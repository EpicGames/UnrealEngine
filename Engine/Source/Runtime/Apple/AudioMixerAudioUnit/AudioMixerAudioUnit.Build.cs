// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMixerAudioUnit : ModuleRules
{
	public AudioMixerAudioUnit(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		PrivateIncludePathModuleNames.Add("TargetPlatform");
		PublicIncludePaths.Add("Runtime/AudioMixer/Public");
		PrivateIncludePaths.Add("Runtime/AudioMixer/Private");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"AudioMixerCore"
			}
			);

		PrecompileForTargets = PrecompileTargetsType.None;

        if (Target.bCompileAgainstEngine)
        {
			// Engine module is required for CompressedAudioInfo implementations.
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target,
                "UEOgg",
                "Vorbis",
                "VorbisFile"
            );
        }

		PublicFrameworks.AddRange(new string[]
		{
			"AudioToolbox",
			"CoreAudio"
		});
		
		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PublicFrameworks.Add("AVFoundation");
		}


		PublicDefinitions.Add("WITH_OGGVORBIS=1");

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
