// Copyright Epic Games, Inc. All Rights Reserved.

/**
	Concrete implementation of FAudioDevice for Apple's CoreAudio

*/

#include "AudioMixer.h"
#include "AudioMixerPlatformAudioUnit.h"
#include "Modules/ModuleManager.h"

class FAudioMixerModuleAudioUnit : public IAudioDeviceModule
{
public:

	virtual void StartupModule() override
	{
		IAudioDeviceModule::StartupModule();

		FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
	}

	virtual bool IsAudioMixerModule() const override { return true; }

    virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() override
    {
        return new Audio::FMixerPlatformAudioUnit();
    }
};

IMPLEMENT_MODULE(FAudioMixerModuleAudioUnit, AudioMixerAudioUnit);
