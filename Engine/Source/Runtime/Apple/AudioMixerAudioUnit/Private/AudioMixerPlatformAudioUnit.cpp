// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerPlatformAudioUnit.h"
#include "AudioMixerPlatformAudioUnitUtils.h"
#include "Modules/ModuleManager.h"
#include "AudioMixer.h"
#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"

/*
 This implementation only depends on the audio units API which allows it to run on MacOS, iOS and tvOS.
 
 For now just assume an iOS configuration (only 2 left and right channels on a single device)
 */

/**
 * CoreAudio System Headers
 */
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>
#include <AVFoundation/AVAudioSession.h>


static int32 SuspendCounter = 0;

DECLARE_LOG_CATEGORY_EXTERN(LogAudioMixerAudioUnit, Log, All);
DEFINE_LOG_CATEGORY(LogAudioMixerAudioUnit);

namespace Audio
{
	static const int32 DefaultBufferSize = 512;

	static const double DefaultSampleRate = 48000.0;
    
    void AUDIOMIXERAUDIOUNIT_API IncrementIOSAudioMixerPlatformSuspendCounter()
    {
        FMixerPlatformAudioUnit::IncrementSuspendCounter();
    }
    
    void AUDIOMIXERAUDIOUNIT_API DecrementIOSAudioMixerPlatformSuspendCounter()
    {
        FMixerPlatformAudioUnit::DecrementSuspendCounter();
    }
	
	FMixerPlatformAudioUnit::FMixerPlatformAudioUnit()
	: bSuspended(false)
    , bInitialized(false)
	, bInCallback(false)
	, SubmittedBufferPtr(nullptr)
	, RemainingBytesInCurrentSubmittedBuffer(0)
	, BytesPerSubmittedBuffer(0)
	, GraphSampleRate(DefaultSampleRate)
	, NumSamplesPerRenderCallback(0)
	, NumSamplesPerDeviceCallback(0)
	{
	}
	
	FMixerPlatformAudioUnit::~FMixerPlatformAudioUnit()
	{
		if (bInitialized)
		{
			TeardownHardware();
		}
	}
	
	int32 FMixerPlatformAudioUnit::GetNumFrames(const int32 InNumReqestedFrames)
	{
		return AlignArbitrary(InNumReqestedFrames, 4);
	}
	
	bool FMixerPlatformAudioUnit::InitializeHardware()
	{
		if (bInitialized)
		{
			return false;
		}
		
		bSupportsBackgroundAudio = false;
		GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsBackgroundAudio"), bSupportsBackgroundAudio, GEngineIni);

		OSStatus Status;
		GraphSampleRate = (double) InternalPlatformSettings.SampleRate;
		UInt32 BufferSize = (UInt32) GetNumFrames(InternalPlatformSettings.CallbackBufferFrameSize);

		const int32 NumChannels = 2;
		
		if (GraphSampleRate == 0)
		{
			GraphSampleRate = DefaultSampleRate;
		}
		
		if (BufferSize == 0)
		{
			BufferSize = DefaultBufferSize;
		}
		
		BytesPerSubmittedBuffer = BufferSize * NumChannels * sizeof(float);
		check(BytesPerSubmittedBuffer != 0);
		
		NSError* error;
		
		AVAudioSession* AudioSession = [AVAudioSession sharedInstance];
		
		// this sample rate is currently gotten from AudioSession in GetPlatformSettings, so there should be no issue
		bool Success = [AudioSession setPreferredSampleRate:GraphSampleRate error:&error];
		
		if (!Success)
		{
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Error setting sample rate.");
		}
		
#if PLATFORM_VISIONOS
		Success = [AudioSession setIntendedSpatialExperience:AVAudioSessionSpatialExperienceBypassed options:nil error:&error];
		if (!Success)
		{
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Error while setting spatial experience to bypass for VisionOS.");
		}
#endif

		// By calling setPreferredIOBufferDuration, we indicate that we would prefer that the buffer size not change if possible.
		float AudioMixerBufferSizeInSec = InternalPlatformSettings.CallbackBufferFrameSize / GraphSampleRate;
		Success = [AudioSession setPreferredIOBufferDuration:AudioMixerBufferSizeInSec error: &error];
		
		int32 FinalBufferSize = [AudioSession IOBufferDuration] * GraphSampleRate;
		int32 FinalPreferredBufferSize = [AudioSession preferredIOBufferDuration] * GraphSampleRate;
		
		BytesPerSubmittedBuffer = FinalBufferSize * NumChannels * sizeof(float);
		check(BytesPerSubmittedBuffer != 0);
		
		UE_LOGF(LogAudioMixerAudioUnit, Display, "Device Sample Rate: %f", GraphSampleRate);
		check(GraphSampleRate != 0);
		
		Success = [AudioSession setActive:true error:&error];
		
		if (!Success)
		{
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Error starting audio session.");
		}
		
		UE_LOGF(LogAudioMixerAudioUnit, Display, "Bytes per submitted buffer: %d", BytesPerSubmittedBuffer);
		
		// Linear PCM stream format
		OutputFormat.mFormatID         = kAudioFormatLinearPCM;
		OutputFormat.mFormatFlags       = kAudioFormatFlagIsFloat | kAudioFormatFlagsNativeEndian | kAudioFormatFlagIsPacked;
		OutputFormat.mChannelsPerFrame = 2;
		OutputFormat.mBytesPerFrame    = sizeof(float) * OutputFormat.mChannelsPerFrame;
		OutputFormat.mFramesPerPacket  = 1;
		OutputFormat.mBytesPerPacket   = OutputFormat.mBytesPerFrame * OutputFormat.mFramesPerPacket;
		OutputFormat.mBitsPerChannel   = 8 * sizeof(float);
		OutputFormat.mSampleRate       = GraphSampleRate;
		
		Status = NewAUGraph(&AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to create audio unit graph!"));
			return false;
		}
		
		AudioComponentDescription UnitDescription;
		
		// Setup audio output unit
		UnitDescription.componentType         = kAudioUnitType_Output;
		
		//On iOS, we'll use the RemoteIO AudioUnit.
		UnitDescription.componentSubType      = kAudioUnitSubType_RemoteIO;
		UnitDescription.componentManufacturer = kAudioUnitManufacturer_Apple;
		UnitDescription.componentFlags        = 0;
		UnitDescription.componentFlagsMask    = 0;
		Status = AUGraphAddNode(AudioUnitGraph, &UnitDescription, &OutputNode);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to initialize audio output node!"), true);
			return false;
		}
		
		Status = AUGraphOpen(AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to open audio unit graph"), true);
			return false;
		}
		
		Status = AUGraphNodeInfo(AudioUnitGraph, OutputNode, nullptr, &OutputUnit);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to retrieve output unit reference!"), true);
			return false;
		}
		
		Status = AudioUnitSetProperty(OutputUnit,
									  kAudioUnitProperty_StreamFormat,
									  kAudioUnitScope_Input,
									  0,
									  &OutputFormat,
									  sizeof(AudioStreamBasicDescription));
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to set output format!"), true);
			return false;
		}
		

		AudioStreamInfo.DeviceInfo = GetPlatformDeviceInfo();
		
		AURenderCallbackStruct InputCallback;
		InputCallback.inputProc = &AudioRenderCallback;
		InputCallback.inputProcRefCon = this;
		Status = AUGraphSetNodeInputCallback(AudioUnitGraph,
											 OutputNode,
											 0,
											 &InputCallback);
		UE_CLOGF(Status != noErr, LogAudioMixerAudioUnit, Error, "Failed to set input callback for audio output node");
		
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;
		
		bInitialized = true;
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::CheckAudioDeviceChange()
	{
		//TODO
		return false;
	}
	
	bool FMixerPlatformAudioUnit::TeardownHardware()
	{
		if(!bInitialized)
		{
			return true;
		}
		
		StopAudioStream();
		CloseAudioStream();
		
		DisposeAUGraph(AudioUnitGraph);
		
		AudioUnitGraph = nullptr;
		OutputNode = -1;
		OutputUnit = nullptr;
		
		bInitialized = false;
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::IsInitialized() const
	{
		return bInitialized;
	}
	
	bool FMixerPlatformAudioUnit::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		OutNumOutputDevices = 1;
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{
		OutInfo = AudioStreamInfo.DeviceInfo;
		return true;
	}
	
	bool FMixerPlatformAudioUnit::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		OutDefaultDeviceIndex = 0;
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			return false;
		}
		
		OpenStreamParams = Params;
		//todo: AudioStreamInfo.SampleRate = OpenStreamParams.SampleRate;
		AudioStreamInfo.Reset();
		AudioStreamInfo.OutputDeviceIndex = OpenStreamParams.OutputDeviceIndex;
		AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
		AudioStreamInfo.NumBuffers = OpenStreamParams.NumBuffers;
		AudioStreamInfo.AudioMixer = OpenStreamParams.AudioMixer;
		AudioStreamInfo.DeviceInfo = GetPlatformDeviceInfo();
		
		// Set up circular buffer between our rendering buffer size and the device's buffer size.
		// Since we are only using this circular buffer on a single thread, we do not need to add extra slack.
		NumSamplesPerRenderCallback = AudioStreamInfo.NumOutputFrames * AudioStreamInfo.DeviceInfo.NumChannels;
		NumSamplesPerDeviceCallback = InternalPlatformSettings.CallbackBufferFrameSize * AudioStreamInfo.DeviceInfo.NumChannels;
		
		// initial circular buffer capacity is zero, so this initializes it.
		GrowCircularBufferIfNeeded(NumSamplesPerRenderCallback, NumSamplesPerDeviceCallback);
		
		// Initialize the audio unit graph
		OSStatus Status = AUGraphInitialize(AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to initialize audio graph!"), true);
			return false;
		}
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;
        
        FCoreDelegates::ApplicationWillDeactivateDelegate.AddRaw(this, &FMixerPlatformAudioUnit::SuspendContext);
        FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FMixerPlatformAudioUnit::ResumeContext);
		FCoreDelegates::AudioInterruptionDelegate.AddRaw(this, &FMixerPlatformAudioUnit::InterruptContext);

		return true;
	}
	
	bool FMixerPlatformAudioUnit::CloseAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}
		
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;
        
		FCoreDelegates::AudioInterruptionDelegate.RemoveAll(this);
		FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);
        FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::StartAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}
		
		BeginGeneratingAudio();
		
		// This will start the render audio callback
		OSStatus Status = AUGraphStart(AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to start audio graph!"), true);
			return false;
		}
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::StopAudioStream()
	{
		if(!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Running)
		{
			return false;
		}
		
        StopGeneratingAudio();
		AUGraphStop(AudioUnitGraph);
		
		return true;
	}
	
	bool FMixerPlatformAudioUnit::MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId)
	{
		//TODO
		
		return false;
	}
	
	FAudioPlatformDeviceInfo FMixerPlatformAudioUnit::GetPlatformDeviceInfo() const
	{
		FAudioPlatformDeviceInfo DeviceInfo;
		
		AVAudioSession* AudioSession = [AVAudioSession sharedInstance];
		double SampleRate = [AudioSession preferredSampleRate];
		DeviceInfo.SampleRate = (int32)SampleRate;
		DeviceInfo.NumChannels = 2;
		DeviceInfo.Format = EAudioMixerStreamDataFormat::Float;
		DeviceInfo.OutputChannelArray.SetNum(2);
		DeviceInfo.OutputChannelArray[0] = EAudioMixerChannel::FrontLeft;
		DeviceInfo.OutputChannelArray[1] = EAudioMixerChannel::FrontRight;
		DeviceInfo.bIsSystemDefault = true;
		
		return DeviceInfo;
	}
	
	void FMixerPlatformAudioUnit::SubmitBuffer(const uint8* Buffer)
	{
		if (!Buffer || bIsUsingNullDevice)
		{
			return;
		}

		const int32 BytesToSubmitToAudioMixer = NumSamplesPerRenderCallback * sizeof(float);
		
		int32 PushResult = CircularOutputBuffer.Push((const int8*)Buffer, BytesToSubmitToAudioMixer);
		check(PushResult == BytesToSubmitToAudioMixer);
	}

	FString FMixerPlatformAudioUnit::GetDefaultDeviceName()
	{
		return FString();
	}
	
	FAudioPlatformSettings FMixerPlatformAudioUnit::GetPlatformSettings() const
	{
		InternalPlatformSettings = FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
		
		// Check for command line overrides
		FString TempString;
		
		// Buffer Size
		if(FParse::Value(FCommandLine::Get(), TEXT("-ForceIOSAudioMixerBufferSize="), TempString))
		{
			InternalPlatformSettings.CallbackBufferFrameSize = FCString::Atoi(*TempString);
		}
		
		// NumBuffers
		if(FParse::Value(FCommandLine::Get(), TEXT("-ForceIOSAudioMixerNumBuffers="), TempString))
		{
			InternalPlatformSettings.NumBuffers = FCString::Atoi(*TempString);
		}
		
		AVAudioSession* AudioSession = [AVAudioSession sharedInstance];
		double PreferredBufferSizeInSec = [AudioSession preferredIOBufferDuration];
		double BufferSizeInSec = [AudioSession IOBufferDuration];
		double SampleRate = [AudioSession preferredSampleRate];
		
		int32 NumFrames;
		
		if (BufferSizeInSec == 0.0)
		{
			NumFrames = DefaultBufferSize;
		}
		else
		{
			NumFrames = (int32)(SampleRate * BufferSizeInSec);
		}

		if (FMath::IsNearlyZero(SampleRate))
		{
			SampleRate = DefaultSampleRate;
		}

		InternalPlatformSettings.SampleRate = SampleRate;

		return InternalPlatformSettings;
	}
    
    void FMixerPlatformAudioUnit::IncrementSuspendCounter()
    {
        if(SuspendCounter == 0)
        {
            FPlatformAtomics::InterlockedIncrement(&SuspendCounter);
        }
    }
    
    void FMixerPlatformAudioUnit::DecrementSuspendCounter()
    {
        if(SuspendCounter > 0)
        {
            FPlatformAtomics::InterlockedDecrement(&SuspendCounter);
        }
    }
	
	void FMixerPlatformAudioUnit::ResumeContext()
	{
		if (SuspendCounter > 0)
		{
			FPlatformAtomics::InterlockedDecrement(&SuspendCounter);
			
			if (bIsUsingNullDevice)
			{
				StopRunningNullDevice();
			}
            
            if (AudioUnitGraph != NULL)
            {
                AUGraphStart(AudioUnitGraph);
            }
            
            if (OutputUnit != NULL)
            {
                AudioOutputUnitStart(OutputUnit);
            }
			
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Resuming Audio");
			bSuspended = false;
		}
	}
	
	void FMixerPlatformAudioUnit::SuspendContext()
	{
#if PLATFORM_IOS
        if (bSupportsBackgroundAudio)
        {
            return;
        }
#endif
		if (SuspendCounter == 0)
		{
			FPlatformAtomics::InterlockedIncrement(&SuspendCounter);
            
            if(AudioUnitGraph != NULL)
            {
                AUGraphStop(AudioUnitGraph);
            }
            
            if (OutputUnit != NULL)
            {
                AudioOutputUnitStop(OutputUnit);
            }
			
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Suspending Audio");
			bSuspended = true;
		}
	}
	
	void FMixerPlatformAudioUnit::HandleError(const TCHAR* InLogOutput, bool bTeardown)
	{
		UE_LOGF(LogAudioMixerAudioUnit, Log, "%ls", InLogOutput);
		if (bTeardown)
		{
			TeardownHardware();
		}
	}
	
	void FMixerPlatformAudioUnit::GrowCircularBufferIfNeeded(const int32 InNumSamplesPerRenderCallback, const int32 InNumSamplesPerDeviceCallback)
	{
		const int32 MaxCircularBufferCapacity = 2 * sizeof(float) * FMath::Max<int32>(NumSamplesPerRenderCallback, NumSamplesPerDeviceCallback);
		
		if (CircularOutputBuffer.GetCapacity() < MaxCircularBufferCapacity)
		{
			// SetCapacity also zeros-out data
			CircularOutputBuffer.SetCapacity(MaxCircularBufferCapacity);
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Growing iOS circular buffer to %i bytes.", MaxCircularBufferCapacity);
		}
	}
	
	void FMixerPlatformAudioUnit::InterruptContext(bool Interrupted)
	{
		if (Interrupted)
		{
			StartRunningNullDevice();
		}
		else
		{
			StopRunningNullDevice();
		}
	}

	bool FMixerPlatformAudioUnit::PerformCallback(AudioBufferList* OutputBufferData)
	{
		bInCallback = true;
		
		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
		{
			// How many bytes we have left over from previous callback
			BytesPerSubmittedBuffer = OutputBufferData->mBuffers[0].mDataByteSize;
			uint8* OutputBufferPtr = (uint8*)OutputBufferData->mBuffers[0].mData;
			
			// Check to see if the system has requested a larger callback size
			NumSamplesPerDeviceCallback = BytesPerSubmittedBuffer / static_cast<float>(sizeof(float));
			GrowCircularBufferIfNeeded(NumSamplesPerRenderCallback, NumSamplesPerDeviceCallback);
			
			while (CircularOutputBuffer.Num() < BytesPerSubmittedBuffer)
			{
				ReadNextBuffer();
			}
			
			int32 PopResult = CircularOutputBuffer.Pop((int8*)OutputBufferPtr, BytesPerSubmittedBuffer);
			check(PopResult == BytesPerSubmittedBuffer);
		}
		else // AudioStreamInfo.StreamState != EAudioOutputStreamState::Running
		{
			for (uint32 bufferItr = 0; bufferItr < OutputBufferData->mNumberBuffers; ++bufferItr)
			{
				memset(OutputBufferData->mBuffers[bufferItr].mData, 0, OutputBufferData->mBuffers[bufferItr].mDataByteSize);
			}
		}
		
		bInCallback = false;
		
		return true;
	}
	
	OSStatus FMixerPlatformAudioUnit::AudioRenderCallback(void* RefCon, AudioUnitRenderActionFlags* ActionFlags,
														  const AudioTimeStamp* TimeStamp, UInt32 BusNumber,
														  UInt32 NumFrames, AudioBufferList* IOData)
	{
		// Get the user data and cast to our FMixerPlatformCoreAudio object
		FMixerPlatformAudioUnit* me = (FMixerPlatformAudioUnit*) RefCon;
		
		me->PerformCallback(IOData);
		
		return noErr;
	}
}
