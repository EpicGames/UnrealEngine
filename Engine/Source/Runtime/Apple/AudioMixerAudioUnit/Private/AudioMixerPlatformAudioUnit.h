// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "DSP/Dsp.h"
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>

// Any platform defines
namespace Audio
{
	
	class FMixerPlatformAudioUnit : public IAudioMixerPlatformInterface
	{
		
	public:
		
		FMixerPlatformAudioUnit();
		~FMixerPlatformAudioUnit();
		
		//~ Begin IAudioMixerPlatformInterface
		virtual FString GetPlatformApi() const override { return TEXT("AudioUnit"); }
		virtual bool InitializeHardware() override;
		virtual bool CheckAudioDeviceChange() override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual bool MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId) override;
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		virtual void SubmitBuffer(const uint8* Buffer) override;
		virtual FString GetDefaultDeviceName() override;
		virtual FAudioPlatformSettings GetPlatformSettings() const override;
		virtual int32 GetNumFrames(const int32 InNumReqestedFrames) override;
		virtual void ResumeContext() override;
		virtual void SuspendContext() override;
		
		//~ End IAudioMixerPlatformInterface
        
        static void IncrementSuspendCounter();
        static void DecrementSuspendCounter();
		
	private:
		AudioStreamBasicDescription OutputFormat;

		bool	bSuspended;

		/** True if the connection to the device has been initialized */
		bool	bInitialized;
		
		/** True if execution is in the callback */
		bool	bInCallback;

		AUGraph     AudioUnitGraph;
		AUNode      OutputNode;
		AudioUnit    OutputUnit;
		uint8*      SubmittedBufferPtr;
		int32 SubmittedBytes = 0;
		
		int32       RemainingBytesInCurrentSubmittedBuffer;
		int32       BytesPerSubmittedBuffer;
		
		double GraphSampleRate;
        
        bool bSupportsBackgroundAudio;
		
		// We may have to grow the circular buffer capacity since Audio Unit callback size is not guaranteed to be constant
		// Currently, this just zero's-out and reallocates, so it will pop. (We always keep largest capacity)
		void GrowCircularBufferIfNeeded(const int32 InNumSamplesPerRenderCallback, const int32 InNumSamplesPerDeviceCallback);
		
		// This buffer is pushed to and popped from in the SubmitBuffer callback.
		// This is required for devices that require frame counts per callback that are not powers of two.
		Audio::TCircularAudioBuffer<int8> CircularOutputBuffer;
		
		int32 NumSamplesPerRenderCallback;
		int32 NumSamplesPerDeviceCallback;
		mutable bool bInternalPlatformSettingsInitialized{ false };
		mutable FAudioPlatformSettings InternalPlatformSettings;
		
		void InterruptContext(bool Interrupted);

		bool PerformCallback(AudioBufferList* OutputBufferData);
		void HandleError(const TCHAR* InLogOutput, bool bTeardown = true);
		static OSStatus AudioRenderCallback(void* RefCon, AudioUnitRenderActionFlags* ActionFlags,
														  const AudioTimeStamp* TimeStamp, UInt32 BusNumber,
														  UInt32 NumFrames, AudioBufferList* IOData);

	};

}
