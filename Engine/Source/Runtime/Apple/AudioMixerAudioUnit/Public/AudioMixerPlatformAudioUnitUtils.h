// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace Audio
{
    void AUDIOMIXERAUDIOUNIT_API IncrementIOSAudioMixerPlatformSuspendCounter();
    void AUDIOMIXERAUDIOUNIT_API DecrementIOSAudioMixerPlatformSuspendCounter();
}
