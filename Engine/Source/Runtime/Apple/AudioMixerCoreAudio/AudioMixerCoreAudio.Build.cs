// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class AudioMixerCoreAudio : ModuleRules
{
	public AudioMixerCoreAudio(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");
		PrivateIncludePathModuleNames.Add("AudioMixer");
		PrivateIncludePaths.Add(Path.Combine(GetModuleDirectory("AudioMixer"), "Private")); // TODO: Adding private include path from other module


		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"AudioMixerCore",
			}
		);

		PrecompileForTargets = PrecompileTargetsType.None;

		if (Target.bCompileAgainstEngine)
		{
			// Engine module is required for CompressedAudioInfo implementations.
			PrivateDependencyModuleNames.Add("Engine");

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"UEOgg",
				"Vorbis",
				"VorbisFile"
			);
		}

		PublicFrameworks.AddRange(new string[]
		{
			"AudioToolbox",
			"CoreAudio"
		});

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			bRequiresPlatformSDK = true;
			PublicFrameworks.Add("AudioUnit");
		}

		PublicDefinitions.Add("WITH_OGGVORBIS=1");

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}

		PublicDefinitions.Add("WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG=0");
	}
}
