// Copyright Epic Games, Inc. All Rights Reserved.
/**
	Concrete implementation of FAudioDevice for Apple's CoreAudio
*/
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "AudioMixerPlatformCoreAudio.h"


class FAudioMixerModuleCoreAudio : public IAudioDeviceModule
{
public:

	virtual void StartupModule() override
	{
		IAudioDeviceModule::StartupModule();

		FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
	}

	virtual bool IsAudioMixerModule() const override { return true; }

	virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() override
	{
		return new Audio::FMixerPlatformCoreAudio();
	}
};
IMPLEMENT_MODULE(FAudioMixerModuleCoreAudio, AudioMixerCoreAudio);
