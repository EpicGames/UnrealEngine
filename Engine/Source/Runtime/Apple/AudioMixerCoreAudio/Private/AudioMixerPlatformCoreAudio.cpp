// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerPlatformCoreAudio.h"
#include "Modules/ModuleManager.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#if WITH_ENGINE
#include "AudioDevice.h"
#endif
/**
* CoreAudio System Headers
*/
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>
#include <CoreAudio/AudioHardware.h>

DECLARE_LOG_CATEGORY_EXTERN(LogAudioMixerAudioUnit, Log, All);
DEFINE_LOG_CATEGORY(LogAudioMixerAudioUnit);

namespace Audio
{
	static const int32 DefaultBufferSize = 1024;
	static const int32 AUBufferSize = 256;
	static const double DefaultSampleRate = 48000.0;

	static int32 SuspendCounter = 0;

	FMixerPlatformCoreAudio::FMixerPlatformCoreAudio()
		: bInitialized(false)
		, bInCallback(false)
		, SubmittedBufferPtr(nullptr)
		, RemainingBytesInCurrentSubmittedBuffer(0)
		, BytesPerSubmittedBuffer(0)
		, GraphSampleRate(DefaultSampleRate)
	{
	}

	FMixerPlatformCoreAudio::~FMixerPlatformCoreAudio()
	{
		if (bInitialized)
		{
			TeardownHardware();
		}
	}

	int32 FMixerPlatformCoreAudio::GetNumFrames(const int32 InNumReqestedFrames)
	{
	   //On MacOS, we hardcode buffer sizes.
		return DefaultBufferSize;
	}

	bool FMixerPlatformCoreAudio::InitializeHardware()
	{
		if (bInitialized)
		{
			return false;
		}

		OSStatus Status;
		GraphSampleRate = (double) AudioStreamInfo.DeviceInfo.SampleRate;
		UInt32 BufferSize = (UInt32) GetNumFrames(OpenStreamParams.NumFrames);
		const int32 NumChannels = 2;

		if (GraphSampleRate == 0)
		{
			GraphSampleRate = DefaultSampleRate;
		}

		if (BufferSize == 0)
		{
			BufferSize = DefaultBufferSize;
		}

		BytesPerSubmittedBuffer = BufferSize * NumChannels * sizeof(float);
		check(BytesPerSubmittedBuffer != 0);
		UE_LOGF(LogAudioMixerAudioUnit, Display, "Bytes per submitted buffer: %d", BytesPerSubmittedBuffer);

		AudioObjectID DeviceAudioObjectID;
		AudioObjectPropertyAddress DevicePropertyAddress;
		UInt32 AudioDeviceQuerySize;

		//Get Audio Device ID- this will be used throughout initialization to query the audio hardware.
		DevicePropertyAddress.mSelector = kAudioHardwarePropertyDefaultOutputDevice;
		DevicePropertyAddress.mScope = kAudioObjectPropertyScopeGlobal;
		DevicePropertyAddress.mElement = 0;
		AudioDeviceQuerySize = sizeof(AudioDeviceID);
		Status = AudioObjectGetPropertyData(kAudioObjectSystemObject, &DevicePropertyAddress, 0, nullptr, &AudioDeviceQuerySize, &DeviceAudioObjectID);

		if(Status != 0)
		{
			UE_LOGF(LogAudioMixerAudioUnit, Warning, "Error querying Audio Device ID: %i", Status);
		}

		Status = AudioObjectGetPropertyData(DeviceAudioObjectID, &DevicePropertyAddress, 0, nullptr, &AudioDeviceQuerySize, &GraphSampleRate);

		if(Status == 0)
		{
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Sample Rate: %f", GraphSampleRate);
		}
		else
		{
			UE_LOGF(LogAudioMixerAudioUnit, Warning, "Error querying Sample Rate: %i", Status);
		}


		// Linear PCM stream format
		OutputFormat.mFormatID         = kAudioFormatLinearPCM;
		OutputFormat.mFormatFlags	   = kAudioFormatFlagIsFloat | kAudioFormatFlagsNativeEndian | kAudioFormatFlagIsPacked;
		OutputFormat.mChannelsPerFrame = 2;
		OutputFormat.mBytesPerFrame    = sizeof(float) * OutputFormat.mChannelsPerFrame;
		OutputFormat.mFramesPerPacket  = 1;
		OutputFormat.mBytesPerPacket   = OutputFormat.mBytesPerFrame * OutputFormat.mFramesPerPacket;
		OutputFormat.mBitsPerChannel   = 8 * sizeof(float);
		OutputFormat.mSampleRate       = GraphSampleRate;

		Status = NewAUGraph(&AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to create audio unit graph!"));
			return false;
		}

		AudioComponentDescription UnitDescription;

		// Setup audio output unit
		UnitDescription.componentType         = kAudioUnitType_Output;
		//On MacOS, we'll use the DefaultOutput AudioUnit.
		UnitDescription.componentSubType      = kAudioUnitSubType_DefaultOutput;
		UnitDescription.componentManufacturer = kAudioUnitManufacturer_Apple;
		UnitDescription.componentFlags        = 0;
		UnitDescription.componentFlagsMask    = 0;
		Status = AUGraphAddNode(AudioUnitGraph, &UnitDescription, &OutputNode);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to initialize audio output node!"), true);
			return false;
		}

		Status = AUGraphOpen(AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to open audio unit graph"), true);
			return false;
		}

		Status = AUGraphNodeInfo(AudioUnitGraph, OutputNode, nullptr, &OutputUnit);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to retrieve output unit reference!"), true);
			return false;
		}

		Status = AudioUnitSetProperty(OutputUnit,
									  kAudioUnitProperty_StreamFormat,
									  kAudioUnitScope_Input,
									  0,
									  &OutputFormat,
									  sizeof(AudioStreamBasicDescription));
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to set output format!"), true);
			return false;
		}

		DevicePropertyAddress.mSelector = kAudioDevicePropertyBufferFrameSize;
		DevicePropertyAddress.mScope = kAudioObjectPropertyScopeGlobal;
		DevicePropertyAddress.mElement = 0;
		AudioDeviceQuerySize = sizeof(AUBufferSize);
		Status = AudioObjectSetPropertyData(DeviceAudioObjectID, &DevicePropertyAddress, 0, nullptr, AudioDeviceQuerySize, &AUBufferSize);
		if(Status != 0)
		{
			HandleError(TEXT("Failed to set buffer size!"), true);
			return false;
		}

		AudioStreamInfo.NumOutputFrames = BufferSize;

		AudioStreamInfo.DeviceInfo = GetPlatformDeviceInfo();

		AURenderCallbackStruct InputCallback;
		InputCallback.inputProc = &AudioRenderCallback;
		InputCallback.inputProcRefCon = this;
		Status = AUGraphSetNodeInputCallback(AudioUnitGraph,
											 OutputNode,
											 0,
											 &InputCallback);
		UE_CLOGF(Status != noErr, LogAudioMixerAudioUnit, Error, "Failed to set input callback for audio output node");

		OpenStreamParams.NumFrames = BufferSize;
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;

		bInitialized = true;

		return true;
	}

	bool FMixerPlatformCoreAudio::CheckAudioDeviceChange()
	{
		//TODO
		return false;
	}

	bool FMixerPlatformCoreAudio::TeardownHardware()
	{
		if(!bInitialized)
		{
			return true;
		}

		StopAudioStream();
		CloseAudioStream();

		DisposeAUGraph(AudioUnitGraph);

		AudioUnitGraph = nullptr;
		OutputNode = -1;
		OutputUnit = nullptr;

		bInitialized = false;

		return true;
	}

	bool FMixerPlatformCoreAudio::IsInitialized() const
	{
		return bInitialized;
	}

	bool FMixerPlatformCoreAudio::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		OutNumOutputDevices = 1;

		return true;
	}

	bool FMixerPlatformCoreAudio::GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo)
	{
		OutInfo = AudioStreamInfo.DeviceInfo;
		return true;
	}

	bool FMixerPlatformCoreAudio::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		OutDefaultDeviceIndex = 0;

		return true;
	}

	bool FMixerPlatformCoreAudio::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			return false;
		}

		AudioStreamInfo.OutputDeviceIndex = Params.OutputDeviceIndex;
		AudioStreamInfo.AudioMixer = Params.AudioMixer;

		OpenStreamParams = Params;

		// Initialize the audio unit graph
		OSStatus Status = AUGraphInitialize(AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to initialize audio graph!"), true);
			return false;
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;

		return true;
	}

	bool FMixerPlatformCoreAudio::CloseAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;

		return true;
	}

	bool FMixerPlatformCoreAudio::StartAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}

		BeginGeneratingAudio();

		// This will start the render audio callback
		OSStatus Status = AUGraphStart(AudioUnitGraph);
		if (Status != noErr)
		{
			HandleError(TEXT("Failed to start audio graph!"), true);
			return false;
		}

		return true;
	}

	bool FMixerPlatformCoreAudio::StopAudioStream()
	{
		if(!bInitialized)
		{
			return false;
		}

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped && AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			AUGraphStop(AudioUnitGraph);

			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
			{
				StopGeneratingAudio();
			}

			check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
		}


		return true;
	}

	bool FMixerPlatformCoreAudio::MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId)
	{
		//TODO
		return false;
	}

	FAudioPlatformDeviceInfo FMixerPlatformCoreAudio::GetPlatformDeviceInfo() const
	{
		FAudioPlatformDeviceInfo DeviceInfo;

		DeviceInfo.SampleRate = GraphSampleRate;
		DeviceInfo.NumChannels = 2;
		DeviceInfo.Format = EAudioMixerStreamDataFormat::Float;
		DeviceInfo.OutputChannelArray.SetNum(2);
		DeviceInfo.OutputChannelArray[0] = EAudioMixerChannel::FrontLeft;
		DeviceInfo.OutputChannelArray[1] = EAudioMixerChannel::FrontRight;
		DeviceInfo.bIsSystemDefault = true;

		return DeviceInfo;
	}

	void FMixerPlatformCoreAudio::SubmitBuffer(const uint8* Buffer)
	{
		SubmittedBufferPtr = (uint8*) Buffer;
		SubmittedBytes = 0;
		RemainingBytesInCurrentSubmittedBuffer = BytesPerSubmittedBuffer;
	}

	FString FMixerPlatformCoreAudio::GetDefaultDeviceName()
	{
		return FString();
	}

	FAudioPlatformSettings FMixerPlatformCoreAudio::GetPlatformSettings() const
	{
		FAudioPlatformSettings Settings;
		Settings.NumBuffers = 2;
		Settings.SampleRate = GraphSampleRate;
		Settings.CallbackBufferFrameSize = DefaultBufferSize;

		return Settings;
	}

	void FMixerPlatformCoreAudio::ResumeContext()
	{
		if (SuspendCounter > 0)
		{
			FPlatformAtomics::InterlockedDecrement(&SuspendCounter);
			AUGraphStart(AudioUnitGraph);
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Resuming Audio");
			bSuspended = false;
		}
	}

	void FMixerPlatformCoreAudio::SuspendContext()
	{
		if (SuspendCounter == 0)
		{
			FPlatformAtomics::InterlockedIncrement(&SuspendCounter);
			AUGraphStop(AudioUnitGraph);
			UE_LOGF(LogAudioMixerAudioUnit, Display, "Suspending Audio");
			bSuspended = true;
		}
	}

	void FMixerPlatformCoreAudio::HandleError(const TCHAR* InLogOutput, bool bTeardown)
	{
		UE_LOGF(LogAudioMixerAudioUnit, Log, "%ls", InLogOutput);
		if (bTeardown)
		{
			TeardownHardware();
		}
	}

	bool FMixerPlatformCoreAudio::PerformCallback(AudioBufferList* OutputBufferData)
	{
		bInCallback = true;

		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
		{
			if (!SubmittedBufferPtr)
			{
				ReadNextBuffer();
			}

			// How many bytes we have left over from previous callback
			int32 SubmittedBufferBytesLeft = BytesPerSubmittedBuffer - SubmittedBytes;
			int32 OutputBufferBytesLeft = OutputBufferData->mBuffers[0].mDataByteSize;
			uint8* OutputBufferPtr = (uint8*) OutputBufferData->mBuffers[0].mData;
			while (OutputBufferBytesLeft > 0)
			{
				const int32 BytesToCopy = FMath::Min(SubmittedBufferBytesLeft, OutputBufferBytesLeft);

				FMemory::Memcpy((void*) OutputBufferPtr, SubmittedBufferPtr + SubmittedBytes, BytesToCopy);

				OutputBufferBytesLeft -= BytesToCopy;
				SubmittedBufferBytesLeft -= BytesToCopy;

				if (SubmittedBufferBytesLeft <= 0)
				{
					ReadNextBuffer();
					SubmittedBytes = 0;
					SubmittedBufferBytesLeft = BytesPerSubmittedBuffer;
				}
				else
				{
					SubmittedBytes += BytesToCopy;
				}

				if (OutputBufferBytesLeft <= 0)
				{
					break;
				}
				else
				{
					OutputBufferPtr += BytesToCopy;
				}
			}
		}
		else
		{
			for (uint32 bufferItr = 0; bufferItr < OutputBufferData->mNumberBuffers; ++bufferItr)
			{
				memset(OutputBufferData->mBuffers[bufferItr].mData, 0, OutputBufferData->mBuffers[bufferItr].mDataByteSize);
			}
		}

		bInCallback = false;

		return true;
	}

	OSStatus FMixerPlatformCoreAudio::AudioRenderCallback(void* RefCon, AudioUnitRenderActionFlags* ActionFlags,
														  const AudioTimeStamp* TimeStamp, UInt32 BusNumber,
														  UInt32 NumFrames, AudioBufferList* IOData)
	{
		// Get the user data and cast to our FMixerPlatformCoreAudio object
		FMixerPlatformCoreAudio* me = (FMixerPlatformCoreAudio*) RefCon;

		me->PerformCallback(IOData);

		return noErr;
	}
}
