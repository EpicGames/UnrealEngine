// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"

#include "Mac/MacSystemIncludes.h"
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>

// Any platform defines
namespace Audio
{

	class FMixerPlatformCoreAudio : public IAudioMixerPlatformInterface
	{

	public:

		FMixerPlatformCoreAudio();
		~FMixerPlatformCoreAudio();

		//~ Begin IAudioMixerPlatformInterface
		virtual FString GetPlatformApi() const override { return TEXT("CoreAudio"); }
		virtual bool InitializeHardware() override;
		virtual bool CheckAudioDeviceChange() override;
		virtual bool TeardownHardware() override;
		virtual bool IsInitialized() const override;
		virtual bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		virtual bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		virtual bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		virtual bool CloseAudioStream() override;
		virtual bool StartAudioStream() override;
		virtual bool StopAudioStream() override;
		virtual bool MoveAudioStreamToNewAudioDevice(const FString& InNewDeviceId) override;
		virtual FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		virtual void SubmitBuffer(const uint8* Buffer) override;
		virtual FString GetDefaultDeviceName() override;
		virtual FAudioPlatformSettings GetPlatformSettings() const override;
        virtual int32 GetNumFrames(const int32 InNumReqestedFrames) override;
        virtual void ResumeContext() override;
        virtual void SuspendContext() override;
        //~ End IAudioMixerPlatformInterface
        
	private:
		AudioStreamBasicDescription OutputFormat;

		bool	bSuspended;

		/** True if the connection to the device has been initialized */
		bool	bInitialized;
		
		/** True if execution is in the callback */
		bool	bInCallback;

		AUGraph     AudioUnitGraph;
		AUNode      OutputNode;
		AudioUnit	OutputUnit;
        uint8*      SubmittedBufferPtr;
        int32 SubmittedBytes = 0;

        int32       RemainingBytesInCurrentSubmittedBuffer;
        int32       BytesPerSubmittedBuffer;
        
        double GraphSampleRate;
        
		bool PerformCallback(AudioBufferList* OutputBufferData);
		void HandleError(const TCHAR* InLogOutput, bool bTeardown = true);
		static OSStatus AudioRenderCallback(void* RefCon, AudioUnitRenderActionFlags* ActionFlags,
														  const AudioTimeStamp* TimeStamp, UInt32 BusNumber,
														  UInt32 NumFrames, AudioBufferList* IOData);

	};

}
