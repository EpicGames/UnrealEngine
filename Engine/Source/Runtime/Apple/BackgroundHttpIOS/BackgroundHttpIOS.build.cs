// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("IOS", "TVOS", "VisionOS")]
public class BackgroundHttpIOS : ModuleRules
{
    public BackgroundHttpIOS(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
				"BackgroundHTTPFileHash"
            }
        );

		PublicIncludePathModuleNames.AddRange(
            new string[] {
                "BackgroundHTTP",
            }
        );

		PrivateDependencyModuleNames.Add("BuildSettings");
    }
}
