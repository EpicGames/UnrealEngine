// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef UE_DNLD_SANDBOX

#include "BackgroundURLSessionHandler.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"

#include "BuildSettings.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "BackgroundHttpRequestImpl.h"

static bool GBackgroundHttpUseExistingNonResumableTasks = true;
static FAutoConsoleVariableRef CVarBackgroundHttpUseExistingNonResumableTasks(
	TEXT("ios.BackgroundHttp.bUseExistingNonResumableTasks"),
	GBackgroundHttpUseExistingNonResumableTasks,
	TEXT("Make getTasksWithCompletionHandler use existing non resumable tasks"),
	ECVF_Default
);

// This is now disabled as there is a known race condition wrt async cancellation
// and any other code managing the same download id task, will be re-enabled when
// that is fixed.
static bool GBackgroundHttpUseCancelByProducingResumeData = false;
static FAutoConsoleVariableRef CVarBackgroundHttpUseCancelByProducingResumeData(
	TEXT("ios.BackgroundHttp.bUseCancelByProducingResumeData"),
	GBackgroundHttpUseCancelByProducingResumeData,
	TEXT("Use cancelByProducingResumeData in RecreateDownload to try reuse existing downloaded data"),
	ECVF_Default
);

static bool GBackgroundHttpReorderCDNsByReachability = true;
static FAutoConsoleVariableRef CVarBackgroundHttpReorderCDNsByReachability(
	TEXT("ios.BackgroundHttp.bReorderCDNsByReachability"),
	GBackgroundHttpReorderCDNsByReachability,
	TEXT("Reorder CDNs by reachability at runtime (true, by default). Otherwise, use Request URLList order."),
	ECVF_Default
);

#endif

#include <future>

// Force cancel all pending downloads. See ClearPreExistingRequestsAtStartup()
static bool bCancelExistingDownloads = false;

// Always report via NSLog, useful for debugging.
static constexpr bool bReportToNSLog = false;

DEFINE_LOG_CATEGORY_STATIC(LogIOSBackgroundDownload, Log, All);

static void LogIOSBackgroundDownloadMessage(NSString* Message)
{
	if (bReportToNSLog)
	{
		NSLog(@"LogIOSBackgroundDownload %@", Message);
	}
	else if (UE_LOG_ACTIVE(LogIOSBackgroundDownload, Log))
	{
		const FString MessageWrapper(Message);
		UE_LOGF(LogIOSBackgroundDownload, Log, "%ls", *MessageWrapper);
	}
}

#define UE_DNLD_LOG(...) LogIOSBackgroundDownloadMessage([NSString stringWithFormat:__VA_ARGS__])

// --------------------------------------------------------------------------------------------------------------------

// We need additional state in NSURLSessionDownloadTask to implement CDN failover.
// Convential way to implement this would be to set a delegate on NSURLSessionDownloadTask and add properties to the delegate.
//
// In case of background downloads API prohibits setting a delegate or setting NSObject values.
// Additionally we want to keep state in NSURLSessionDownloadTask itself to avoid pitfalls of maintaining separate state elsewhere.
//
// [NSURLSessionDownloadTask taskDescription] is one such property that can be used to track state and API maintains state between app sessions.
@interface FBackgroundNSURLSessionDownloadTaskData : NSObject

@property (nonatomic, retain) NSMutableArray<__kindof NSURL*>* URLs;
@property (nonatomic, retain) NSMutableArray<__kindof NSNumber*>* RetryCountPerURL;
@property (nonatomic) uint64 ExpectedResultSize;
@property (nonatomic) uint64 ExpectedResultSizeFromHeadRequest;
@property (nonatomic, retain) NSString* RequestID;

// assumes all URL's have same content path but different domain
+ (FBackgroundNSURLSessionDownloadTaskData* _Nonnull)TaskDataWithURLs:(NSArray<__kindof NSURL*>*)URLs WithRetryCount:(NSInteger)RetryCount WithExpectedResultSize:(uint64)ExpectedResultSize WithRequestID:(NSString*)RequestID;
+ (FBackgroundNSURLSessionDownloadTaskData* _Nullable)TaskDataFromSerializedString:(NSString*)SerializedData;

- (NSString*)ToSerializedString;

- (NSURL* _Nonnull)GetFirstURL;
- (NSURL* _Nullable)GetNextURL;
- (void)ResetRetryCount:(NSInteger)RetryCount;
- (void)Cancel;

@end

// --------------------------------------------------------------------------------------------------------------------

@implementation FBackgroundNSURLSessionDownloadTaskData

NSString* const SerializationKeyProtocolVersion = @"v";
NSString* const SerializationKeyCDNs = @"c";
NSString* const SerializationKeyPath = @"p";
NSString* const SerializationKeyRetryCountPerURL = @"r";
NSString* const SerializationKeyExpectedResultSize = @"s";
NSString* const SerializationKeyExpectedResultSizeFromHeaderRequest = @"h";
NSString* const SerializationKeyRequestID = @"i";

+ (FBackgroundNSURLSessionDownloadTaskData* _Nonnull)TaskDataWithURLs:(NSArray<__kindof NSURL*>*)URLs WithRetryCount:(NSInteger)RetryCount WithExpectedResultSize:(uint64)ExpectedResultSize WithRequestID:(NSString*)RequestID
{
	FBackgroundNSURLSessionDownloadTaskData* Data = [[FBackgroundNSURLSessionDownloadTaskData alloc] init];

	Data.URLs = [NSMutableArray arrayWithArray:URLs];
	Data.RetryCountPerURL = [NSMutableArray arrayWithCapacity:URLs.count];
	for (NSURL* URL in URLs)
	{
		(void)URL;
		[Data.RetryCountPerURL addObject:[NSNumber numberWithInteger:RetryCount]];
	}
	NSAssert(Data.URLs.count > 0, @"URLs should be non empty");
	NSAssert(Data.URLs.count == Data.RetryCountPerURL.count, @"URLs and RetryCountPerURL arrays should have same size");
	Data.ExpectedResultSize = ExpectedResultSize;
	Data.RequestID = RequestID;

	return [Data autorelease];
}

+ (FBackgroundNSURLSessionDownloadTaskData* _Nullable)TaskDataFromSerializedString:(NSString*)SerializedData
{
	if (SerializedData == nil)
	{
		return nil;
	}

	NSData* SerializedDataEncoded = [SerializedData dataUsingEncoding:NSUTF8StringEncoding];
	if (SerializedDataEncoded == nil)
	{
		return nil;
	}

	FBackgroundNSURLSessionDownloadTaskData* Data = [[FBackgroundNSURLSessionDownloadTaskData alloc] init];

	NSError* Error = nil;
	NSDictionary* Dict = [NSJSONSerialization JSONObjectWithData:SerializedDataEncoded options:0 error:&Error];

	NSNumber* Version = [Dict valueForKey:SerializationKeyProtocolVersion];
	NSMutableArray<__kindof NSString*>* CDNs = [Dict valueForKey:SerializationKeyCDNs];
	NSString* Path = [Dict valueForKey:SerializationKeyPath];
	NSArray<__kindof NSNumber*>* RetryCountPerURL = [Dict valueForKey:SerializationKeyRetryCountPerURL];
	NSNumber* ExpectedResultSize = [Dict valueForKey:SerializationKeyExpectedResultSize]; // can be nil
	NSNumber* ExpectedResultSizeFromHeadRequest = [Dict valueForKey:SerializationKeyExpectedResultSizeFromHeaderRequest];
	NSString* RequestID = [Dict valueForKey:SerializationKeyRequestID];

	if (Error != nil || Version == nil || Version.intValue != 1 || CDNs == nil || Path == nil || RetryCountPerURL == nil || RequestID == nil)
	{
		UE_DNLD_LOG(@"Failed to deserialize task state '%@' due to '%@', %u, %u, %u, %u, %u, %u, %u, %u",
			SerializedData,
			Error != nil ? Error.localizedDescription : @"",
			Version == nil ? 1 : 0,
			Version.intValue,
			CDNs == nil ? 1 : 0,
			Path == nil ? 1 : 0,
			RetryCountPerURL == nil ? 1 : 0,
			ExpectedResultSize == nil ? 1 : 0,
			ExpectedResultSizeFromHeadRequest == nil ? 1 : 0,
			RequestID == nil ? 1 : 0
		);
		[Data release];
		return nil;
	}

	Data.URLs = [NSMutableArray arrayWithCapacity:CDNs.count];
	Data.RetryCountPerURL = [NSMutableArray arrayWithArray:RetryCountPerURL];
	for (NSString* CDN in CDNs)
	{
		[Data.URLs addObject:[NSURL URLWithString:[NSString stringWithFormat:@"%@%@", CDN, Path]]];
	}
	NSAssert(Data.URLs.count > 0, @"URLs should be non empty");
	NSAssert(Data.URLs.count == Data.RetryCountPerURL.count, @"URLs and RetryCountPerURL arrays should have same size");
	Data.ExpectedResultSize = ExpectedResultSize != nil ? ExpectedResultSize.unsignedLongLongValue : 0;
	Data.ExpectedResultSizeFromHeadRequest = ExpectedResultSizeFromHeadRequest != nil ? ExpectedResultSizeFromHeadRequest.unsignedLongLongValue : 0;
	Data.RequestID = RequestID;

	return [Data autorelease];
}

- (void)dealloc
{
	[_URLs release];
	[_RetryCountPerURL release];
	[_RequestID release];
	[super dealloc];
}

- (NSString *)ToSerializedString
{
	NSMutableDictionary* Dict = [NSMutableDictionary dictionaryWithCapacity:4];

	NSString* Path = [[self.URLs firstObject] path];
	NSString* Query = [[self.URLs firstObject] query];
	if (Query && Query.length > 0)
	{
		Path = [[Path stringByAppendingString:@"?"] stringByAppendingString:Query];
	}

	// Decompose URL's that we assume to always have the same path+query into just a path(with query attached if present),
	// and all the different CDN's. This reduces the total data stored in the session.
	NSMutableArray<__kindof NSString*>* CDNs = [NSMutableArray arrayWithCapacity:self.URLs.count];
	for (NSURL* URL in self.URLs)
	{
		NSString* URLString = URL.absoluteString;
		checkf([URLString hasSuffix:Path], TEXT("Expected all URLs have same path(+query), but have %s vs %s"), UTF8_TO_TCHAR([Path UTF8String]), UTF8_TO_TCHAR([URLString UTF8String]));

		NSString* Domain = [[URLString componentsSeparatedByString:URL.path] firstObject];
		[CDNs addObject:Domain];
	}

	[Dict setValue:[NSNumber numberWithInt:1] forKey:SerializationKeyProtocolVersion];
	[Dict setValue:CDNs forKey:SerializationKeyCDNs];
	[Dict setValue:Path forKey:SerializationKeyPath];
	[Dict setValue:self.RetryCountPerURL forKey:SerializationKeyRetryCountPerURL];
	if (self.ExpectedResultSize != 0)
	{
		[Dict setValue:[NSNumber numberWithUnsignedLongLong:self.ExpectedResultSize] forKey:SerializationKeyExpectedResultSize];
	}
	if (self.ExpectedResultSizeFromHeadRequest != 0)
	{
		[Dict setValue:[NSNumber numberWithUnsignedLongLong:self.ExpectedResultSizeFromHeadRequest] forKey:SerializationKeyExpectedResultSizeFromHeaderRequest];
	}
	[Dict setValue:self.RequestID forKey:SerializationKeyRequestID];

	NSError* Error = nil;
	NSData* Data = [NSJSONSerialization dataWithJSONObject:Dict options:NSJSONWritingSortedKeys error:&Error];

	NSString* String = [[NSString alloc] initWithData:Data encoding:NSUTF8StringEncoding];
	return [String autorelease];
}

- (NSURL* _Nonnull)GetFirstURL
{
	return self.URLs.firstObject;
}

- (NSURL* _Nullable)GetNextURL
{
	for (NSUInteger i = 0; i < self.RetryCountPerURL.count; ++i)
	{
		NSInteger RetryValue = [self.RetryCountPerURL objectAtIndex:i].integerValue;
		if (RetryValue == 0)
		{
			continue;
		}
		else if (RetryValue <= -1) // special case for infinitely retrying same URL
		{
			return [self.URLs objectAtIndex:i];
		}

		RetryValue--;
		[self.RetryCountPerURL replaceObjectAtIndex:i withObject:[NSNumber numberWithInteger:RetryValue]];
		return [self.URLs objectAtIndex:i];
	}

	return nil;
}

- (void)ResetRetryCount:(NSInteger)RetryCount
{
	for (NSUInteger i = 0; i < self.RetryCountPerURL.count; ++i)
	{
		[self.RetryCountPerURL replaceObjectAtIndex:i withObject:[NSNumber numberWithInteger:RetryCount]];
	}
}

- (void)Cancel
{
	[self ResetRetryCount:0];
}

@end

// --------------------------------------------------------------------------------------------------------------------

enum class EBackgroundNSURLCDNInfoResponse : uint32 // Beware these constants define sorting order in SortingKeyWith
{
	// CDN responded with a valid HTTP response with a code smaller than HTTPStatusCodeErrorBadRequest (400)
	Ok = 1,

	// CDN request timed outed or was cancelled
	Timeout = 2,

	// CDN or networking responded with error, e.g. DNS resolution error, etc
	Error = 3
};

// NSURLSession CDN info
@interface FBackgroundNSURLCDNInfo : NSObject

@property (nonatomic, retain) NSString* CDNHost;
@property (nonatomic, retain) NSString* CDNAbsoluteURL;
@property (nonatomic) EBackgroundNSURLCDNInfoResponse Response;
@property (nonatomic) NSTimeInterval ResponseTime;
@property (nonatomic) NSUInteger ProvidedOrder;

- (void)SetFromURL:(NSURL*)URL;
- (double)SortingKeyWith:(BOOL)bSortByResponseTime;

@end

@implementation FBackgroundNSURLCDNInfo

- (void)SetFromURL:(NSURL*)URL
{
	[self setCDNHost:URL.host];

	NSURLComponents* Components = [NSURLComponents componentsWithURL:URL resolvingAgainstBaseURL:NO];
	Components.fragment = nil;
	Components.path = nil;
	Components.query = nil;

	NSString* AbsoluteURL = Components.string;
	if (![AbsoluteURL hasSuffix:@"/"])
	{
		// Trailing / is not part of RFC 3986, but is used in CDN configs,
		// append it to keep CDN string formatting.
		AbsoluteURL = [AbsoluteURL stringByAppendingString:@"/"];
	}

	[self setCDNAbsoluteURL:AbsoluteURL];
}

- (double)SortingKeyWith:(BOOL)bSortByResponseTime
{
	double Key = (double)self.Response * 100000.0;

	if (bSortByResponseTime && self.Response == EBackgroundNSURLCDNInfoResponse::Ok)
	{
		// sort by response time only if response was valid
		Key += self.ResponseTime;
	}
	else
	{
		// all other cases should be sorted by provided order
		Key += self.ProvidedOrder;
	}

	return Key;
}

@end

// --------------------------------------------------------------------------------------------------------------------

// NSURLSession wrapper focused on background downloading and CDN failover
@interface FBackgroundNSURLSession : NSObject<NSURLSessionDelegate, NSURLSessionTaskDelegate, NSURLSessionDownloadDelegate>

@property (atomic) BOOL AllowCellular;
@property (atomic) BOOL AllowConstrained;

+ (FBackgroundNSURLSession*)Shared;
+ (NSUInteger)GetInvalidDownloadId;
+ (NSString*)GetNSURLSessionIdentifier;

- (void)Initialize;
- (void)SetFileHashHelper:(BackgroundHttpFileHashHelperRef)HelperRef;
- (BackgroundHttpFileHashHelperRef)GetFileHashHelper;
- (void)SaveFileHashHelperState;
- (NSString*)GetTempPathForURL:(NSURL* _Nonnull)URL WithRequestID:(NSString*)RequestID;
- (NSMutableArray<__kindof NSURL*>*)ReorderCDNsByReachability:(NSMutableArray<__kindof NSURL*>*)URLs;
- (NSMutableArray<__kindof NSURL*>*)OrderURLs:(NSArray<__kindof NSURL*>*)Candidates ByCDNs:(NSArray<__kindof FBackgroundNSURLCDNInfo*>*)CDNs;
- (NSArray<__kindof FBackgroundNSURLCDNInfo*>*)GetCDNInfo;
- (NSURLSessionDownloadTask*)CreateDownloadForURL:(NSURL* _Nonnull)URL WithPriority:(float)Priority WithTaskData:(FBackgroundNSURLSessionDownloadTaskData* _Nonnull)TaskData;
- (NSURLSessionDownloadTask*)CreateDownloadForResumeData:(NSData* _Nonnull)ResumeData WithPriority:(float)Priority WithTaskData:(FBackgroundNSURLSessionDownloadTaskData* _Nonnull)TaskData;

- (NSUInteger)CreateOrFindDownloadForURLs:(NSArray<__kindof NSString*>*)URLStrings WithPriority:(float)Priority WithExpectedResultSize:(uint64)ExpectedResultSize WithRequestID:(NSString*)RequestID;
- (void)PauseDownload:(NSUInteger)DownloadId;
- (void)ResumeDownload:(NSUInteger)DownloadId;
- (void)CancelDownload:(NSUInteger)DownloadId;
- (void)SetPriority:(float)Priority ForDownload:(NSUInteger)DownloadId;
- (void)SetCurrentDownloadedBytes:(uint64)DownloadedBytes ForTask:(NSURLSessionDownloadTask*)Task;
- (uint64)GetCurrentDownloadedBytes:(NSUInteger)DownloadId;
- (void)GetActiveDownloadIDs:(TSet<uint64>&)OutActiveIDs;
- (void)RecreateDownload:(NSUInteger)DownloadId ShouldResetRetryCount:(bool)ResetRetryCount;
- (void)RecreateDownloads;
- (void)SwitchToForegroundSession;
- (BOOL)IsForegroundSession;
#if !UE_BUILD_SHIPPING
- (NSString*)GetDownloadDebugText:(NSUInteger)DownloadId;
#endif

- (void)StartCheckingForStaleDownloads;
- (void)StopCheckingForStaleDownloads;
- (void)CheckForStaleDownloads:(NSTimer*)Timer;

- (void)HandleDidEnterBackground;
- (void)HandleWillEnterForeground;

- (NSURLSessionDownloadTask*)FindDownloadTaskFor:(NSUInteger)DownloadId;
- (NSUInteger)FindDownloadIdForTask:(NSURLSessionDownloadTask*)Task;
- (NSUInteger)EnsureTaskIsTracked:(NSURLSessionDownloadTask*)Task;
- (void)ReplaceTrackedTaskWith:(NSURLSessionDownloadTask*)NewTask ForDownloadId:(NSUInteger)DownloadId;
- (void)EnsureTaskIsNotTracked:(NSURLSessionDownloadTask*)Task;

- (void)SetDownloadResult:(NSInteger)HTTPCode IsDiskSpaceError:(BOOL)bIsDiskSpaceError WithTempFile:(NSString*)TempFile ForDownload:(NSURLSessionDownloadTask*)Task;
- (NSString* _Nullable)GetDownloadResult:(NSUInteger)DownloadId OutStatus:(BOOL* _Nonnull)OutStatus OutStatusCode:(NSInteger*)OutStatusCode;

// From NSURLSessionDelegate
- (void)URLSessionDidFinishEventsForBackgroundURLSession:(NSURLSession*)Session;

// From NSURLSessionTaskDelegate
- (void)URLSession:(NSURLSession*)Session task:(NSURLSessionTask*)Task didCompleteWithError:(NSError*)Error;
//- (void)URLSession:(NSURLSession*)Session taskIsWaitingForConnectivity:(NSURLSessionTask *)Task;
//- (void)URLSession:(NSURLSession*)Session task:(NSURLSessionTask *)Task willBeginDelayedRequest:(NSURLRequest *)Request completionHandler:(void (^)(NSURLSessionDelayedRequestDisposition Disposition, NSURLRequest* NewRequest))CompletionHandler;

// From NSURLSessionDownloadDelegate
- (void)URLSession:(NSURLSession*)Session downloadTask:(NSURLSessionDownloadTask*)Task didFinishDownloadingToURL:(NSURL*)Location;
- (void)URLSession:(NSURLSession*)Session downloadTask:(NSURLSessionDownloadTask*)Task didWriteData:(int64_t)BytesWritten totalBytesWritten:(int64_t)TotalBytesWritten totalBytesExpectedToWrite:(int64_t)TotalBytesExpectedToWrite;
//- (void)URLSession:(NSURLSession*)Session downloadTask:(NSURLSessionDownloadTask*)Task didResumeAtOffset:(int64_t)FileOffset expectedTotalBytes:(int64_t)ExpectedTotalBytes;

@end

// --------------------------------------------------------------------------------------------------------------------

@implementation FBackgroundNSURLSession
{
	NSURLSession* _Session;
	NSURLSession* _HeadRequestSession;		// Used by CDN reordering and HEAD request for expected size
	NSMutableDictionary<__kindof NSNumber*, __kindof NSURLSessionDownloadTask*>* _AllDownloads;
	NSUInteger _NextDownloadId;
	std::promise<void> _AllDownloadsPromise;
	std::future<void> _AllDownloadsFuture;
	NSMutableArray<__kindof FBackgroundNSURLCDNInfo*>* _CDNInfo;
	NSObject* _CDNProbeGate; // Serializes HEAD probes in ReorderCDNsByReachability
	NSTimer* _ForegroundStaleDownloadCheckTimer;
	BackgroundHttpFileHashHelperPtr _HelperPtr;
	int32 _MaximumConnectionsPerHost;
	int32 _RetryResumeDataLimit;
	int32 _HeadRequestTimeout;
	bool _bCDNReorderByPingTime;
	bool _bUseExistingNonResumableTasks;
	bool _bUseCancelByProducingResumeData;
	double _CheckForForegroundStaleDownloadsWithInterval;
	double _ForegroundStaleDownloadTimeout;
	std::atomic<bool> _bAnyTaskDidCompleteWithError;
}

static constexpr NSUInteger InvalidDownloadId = 0;

NSString* const NSURLSessionIdentifier = @"com.epicgames.backgrounddownloads";

NSProgressUserInfoKey const NSProgressDownloadLastUpdateTime = @"com.epicgames.nsprogress.lastupdatetime";
NSProgressUserInfoKey const NSProgressDownloadCompletedBytes = @"com.epicgames.nsprogress.completedbytes";
NSProgressUserInfoKey const NSProgressDownloadResultStatusCode = @"com.epicgames.nsprogress.resultstatuscode";
NSProgressUserInfoKey const NSProgressDownloadResultTempFilePath = @"com.epicgames.nsprogress.tempfilepath";
#if !UE_BUILD_SHIPPING
NSProgressUserInfoKey const NSProgressDownloadStartTime = @"com.epicgames.nsprogress.starttime";
#endif

static constexpr NSInteger HTTPStatusCodeSuccessCreated = 201;
static constexpr NSInteger HTTPStatusCodeErrorBadRequest = 400;
static constexpr NSInteger HTTPStatusCodeErrorServer = 500;
static constexpr NSInteger TypicalSizeForDownloadResponseHeader = 400; // bytes

+ (FBackgroundNSURLSession*)Shared
{
	static FBackgroundNSURLSession* Shared = nil;
	static dispatch_once_t Once = {};
	dispatch_once(&Once, ^{
		Shared = [[self alloc] init];
	});
	return Shared;
}

+ (NSUInteger)GetInvalidDownloadId;
{
	return InvalidDownloadId;
}

+ (NSString*)GetNSURLSessionIdentifier
{
	return NSURLSessionIdentifier;
}

- (id)init
{
	if (self = [super init])
	{
		[self Initialize];
	}
	return self;
}

- (void)Initialize
{
	bool bUseForegroundSession = false;
	bool bDiscretionary = true;	// Let iOS schedule background downloads, now that we have the expected sizes filled in
	bool bShouldSendLaunchEvents = true;
	_MaximumConnectionsPerHost = 6;
	double TimeoutIntervalForRequest = 120.0; // Note, ignored in background sessions (if bUseForegroundSession is false).
	double TimeoutIntervalForResource = 60.0 * 60.0;
	_RetryResumeDataLimit = 3;
	_HeadRequestTimeout = 400;
	_bCDNReorderByPingTime = false;
	_bUseExistingNonResumableTasks = GBackgroundHttpUseExistingNonResumableTasks; // pass getTasksWithCompletionHandler tasks even if they are not in resumable state
	_bUseCancelByProducingResumeData = GBackgroundHttpUseCancelByProducingResumeData; // use [task cancelByProducingResumeData...] to restart tasks if true, use [task cancel] if false
	_CheckForForegroundStaleDownloadsWithInterval = 1.0; // how often to check for stale downloads, <=0.0 to disable
	_ForegroundStaleDownloadTimeout = 30.0; // If download hasn't received any bytes for this duration, cancel and retry if possible

#ifndef UE_DNLD_SANDBOX
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bUseForegroundSession"), bUseForegroundSession, GEngineIni);
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bDiscretionary"), bDiscretionary, GEngineIni);
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bShouldSendLaunchEvents"), bShouldSendLaunchEvents, GEngineIni);
	GConfig->GetInt(TEXT("BackgroundHttp.iOSSettings"), TEXT("MaximumConnectionsPerHost"), _MaximumConnectionsPerHost, GEngineIni);
	GConfig->GetDouble(TEXT("BackgroundHttp.iOSSettings"), TEXT("BackgroundReceiveTimeout"), TimeoutIntervalForRequest, GEngineIni);
	GConfig->GetDouble(TEXT("BackgroundHttp.iOSSettings"), TEXT("BackgroundHttpResourceTimeout"), TimeoutIntervalForResource, GEngineIni);
	GConfig->GetInt(TEXT("BackgroundHttp.iOSSettings"), TEXT("RetryResumeDataLimit"), _RetryResumeDataLimit, GEngineIni);
	GConfig->GetInt(TEXT("BackgroundHttp.iOSSettings"), TEXT("CDNReorderingTimeout"), _HeadRequestTimeout, GEngineIni);
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bCDNReorderByPingTime"), _bCDNReorderByPingTime, GEngineIni);
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bUseExistingNonResumableTasks"), _bUseExistingNonResumableTasks, GEngineIni);
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bUseCancelByProducingResumeData"), _bUseCancelByProducingResumeData, GEngineIni);
	GConfig->GetDouble(TEXT("BackgroundHttp.iOSSettings"), TEXT("CheckForForegroundStaleDownloadsWithInterval"), _CheckForForegroundStaleDownloadsWithInterval, GEngineIni);
	GConfig->GetDouble(TEXT("BackgroundHttp.iOSSettings"), TEXT("ForegroundStaleDownloadTimeout"), _ForegroundStaleDownloadTimeout, GEngineIni);
#endif

	UE_DNLD_LOG(@"bUseForegroundSession=%u", bUseForegroundSession ? 1 : 0);
	UE_DNLD_LOG(@"bDiscretionary=%u", bDiscretionary ? 1 : 0);
	UE_DNLD_LOG(@"bShouldSendLaunchEvents=%u", bShouldSendLaunchEvents ? 1 : 0);
	UE_DNLD_LOG(@"MaximumConnectionsPerHost=%i", _MaximumConnectionsPerHost);
	UE_DNLD_LOG(@"TimeoutIntervalForRequest=%f", TimeoutIntervalForRequest);
	UE_DNLD_LOG(@"TimeoutIntervalForResource=%f", TimeoutIntervalForResource);
	UE_DNLD_LOG(@"RetryResumeDataLimit=%i", _RetryResumeDataLimit);
	UE_DNLD_LOG(@"HeadRequestTimeout=%i", _HeadRequestTimeout);
	UE_DNLD_LOG(@"bCDNReorderByPingTime=%u", _bCDNReorderByPingTime ? 1 : 0);
	UE_DNLD_LOG(@"bUseExistingNonResumableTasks=%u", _bUseExistingNonResumableTasks ? 1 : 0);
	UE_DNLD_LOG(@"bUseCancelByProducingResumeData=%u", _bUseCancelByProducingResumeData ? 1 : 0);
	UE_DNLD_LOG(@"CheckForForegroundStaleDownloadsWithInterval=%f", _CheckForForegroundStaleDownloadsWithInterval);
	UE_DNLD_LOG(@"ForegroundStaleDownloadTimeout=%f", _ForegroundStaleDownloadTimeout);

	_AllDownloads = [NSMutableDictionary new];
	_AllDownloadsFuture = _AllDownloadsPromise.get_future();
	_NextDownloadId = InvalidDownloadId + 1;
	_CDNInfo = [NSMutableArray new];
	_CDNProbeGate = [NSObject new];
	_ForegroundStaleDownloadCheckTimer = nil;
	_bAnyTaskDidCompleteWithError = false;

	// Never allow cellular & constrained downloads unless we get explicit opt-in from the user.
	self.AllowCellular    = NO;
	self.AllowConstrained = NO;

	NSURLSessionConfiguration* Configuration = bUseForegroundSession ?
		[NSURLSessionConfiguration defaultSessionConfiguration] :
		[NSURLSessionConfiguration backgroundSessionConfigurationWithIdentifier: NSURLSessionIdentifier];

	// iOS will schedule downloads on it's own if true,
	// otherwise all downloads will be scheduled ASAP if false
	Configuration.discretionary = bDiscretionary;

	// In case if our app gets killed in background, iOS will launch it and report finished downloads via handleEventsForBackgroundURLSession.
	// This will help us to retry/fail-over downloads in background without waiting for user to open the game again.
	// Note that this behavior can be disabled via Background App Refresh set to No in iOS settings.
	Configuration.sessionSendsLaunchEvents = bShouldSendLaunchEvents;

	// Set session to allow cellular and instead control this on NSMutableURLRequest level because this value cannot be changed after session is created.
	Configuration.allowsCellularAccess = YES;
	Configuration.allowsExpensiveNetworkAccess = YES;
	Configuration.allowsConstrainedNetworkAccess = YES;

	Configuration.networkServiceType = bUseForegroundSession ? NSURLNetworkServiceTypeDefault : NSURLNetworkServiceTypeBackground;

	// this api is not available on any other Apple platform other than iOS
	// as this file is compiled for tvOS as well, we need the #if !PLATFORM_TVOS
#if !PLATFORM_TVOS
	Configuration.multipathServiceType = NSURLSessionMultipathServiceTypeAggregate;
#endif // !PLATFORM_TVOS

	Configuration.HTTPMaximumConnectionsPerHost = _MaximumConnectionsPerHost;

	Configuration.timeoutIntervalForRequest = TimeoutIntervalForRequest;

	Configuration.timeoutIntervalForResource = TimeoutIntervalForResource;

	_Session = [[NSURLSession sessionWithConfiguration:Configuration delegate:self delegateQueue:nil] retain];
	UE_DNLD_LOG(@"sessionWithConfiguration '%@'", Configuration.identifier);

	[_Session getTasksWithCompletionHandler:^(NSArray<__kindof NSURLSessionDataTask*>*, NSArray<__kindof NSURLSessionUploadTask*>*, NSArray<__kindof NSURLSessionDownloadTask*>* Downloads)
	{
		UE_DNLD_LOG(@"getTasksWithCompletionHandler block with %lu tasks", (unsigned long)[Downloads count]);
		
		if (Downloads != nil)
		{
			for (NSURLSessionDownloadTask* ExistingTask in Downloads)
			{
				const bool bCanRestartTask =
					(ExistingTask.state == NSURLSessionTaskStateRunning) ||
					(ExistingTask.state == NSURLSessionTaskStateSuspended);

				if (_bUseExistingNonResumableTasks == false && !bCanRestartTask)
				{
					UE_DNLD_LOG(@"Skipping tracking for existing download task with taskIdentifier %lu because it's not in resumable state (%lu)", ExistingTask.taskIdentifier, ExistingTask.state);
					continue;
				}

				const NSUInteger DownloadId = [self EnsureTaskIsTracked:ExistingTask];

				if (bCancelExistingDownloads)
				{
					UE_DNLD_LOG(@"Canceling existing download task with taskIdentifier %lu", ExistingTask.taskIdentifier);
					[self CancelDownload:DownloadId];
				}

				UE_DNLD_LOG(@"Existing task '%@' with taskIdentifier %lu with DownloadId %lu and state %lu", ExistingTask.taskDescription, ExistingTask.taskIdentifier, DownloadId, ExistingTask.state);
			}
		}

		_AllDownloadsPromise.set_value();
	}];
	
	Configuration = [NSURLSessionConfiguration ephemeralSessionConfiguration];	// doesn’t store caches, credential stores, or any session-related data to disk
	Configuration.discretionary = NO;
	// Don't go over cellular here, during first download attempt we would try to over non-cellular connections first.
	// Hence it makes more sense to prioritize CDN's that are reachable via non-cellular connections.
	Configuration.allowsCellularAccess = NO;
	Configuration.networkServiceType = NSURLNetworkServiceTypeResponsiveData;
	Configuration.timeoutIntervalForRequest = (double)_HeadRequestTimeout / 1000 ;
	Configuration.timeoutIntervalForResource = (double)_HeadRequestTimeout / 1000;
	Configuration.HTTPMaximumConnectionsPerHost = _MaximumConnectionsPerHost;
	_HeadRequestSession = [[NSURLSession sessionWithConfiguration:Configuration] retain];

	const FString& DirectoryPath = FBackgroundHttpFileHashHelper::GetTemporaryRootPath();
	if (ensureAlwaysMsgf(!DirectoryPath.IsEmpty(), TEXT("Invalid FBackgroundHttpFileHashHelper::GetTemporaryRootPath()")))
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (bCancelExistingDownloads)
		{
			PlatformFile.DeleteDirectory(*DirectoryPath);
		}

		PlatformFile.CreateDirectory(*DirectoryPath);

		if (!PlatformFile.DirectoryExists(*DirectoryPath))
		{
			ensureAlwaysMsgf(false, TEXT("Failed to create temporary directory for background downloads"));
		}
	}
}

- (void)dealloc
{
	[_Session release];
	_Session = nil;
	
	[_HeadRequestSession release];
	_HeadRequestSession = nil;

	[_AllDownloads release];
	_AllDownloads = nil;

	if (_CDNInfo != nil)
	{
		[_CDNInfo release];
		_CDNInfo = nil;
	}

	if (_CDNProbeGate != nil)
	{
		[_CDNProbeGate release];
		_CDNProbeGate = nil;
	}

	if (_ForegroundStaleDownloadCheckTimer != nil)
	{
		[_ForegroundStaleDownloadCheckTimer release];
		_ForegroundStaleDownloadCheckTimer = nil;
	}

	[super dealloc];
}

- (void)SetFileHashHelper:(BackgroundHttpFileHashHelperRef)HelperRef
{
	_HelperPtr = HelperRef.ToSharedPtr();
}

- (BackgroundHttpFileHashHelperRef)GetFileHashHelper
{
	// initialize a new instance in case if we get here from handleEventsForBackgroundURLSession
	if (!_HelperPtr.IsValid())
	{
		_HelperPtr = MakeShared<FBackgroundHttpFileHashHelper, ESPMode::ThreadSafe>();
		_HelperPtr->LoadData();
	}

	return _HelperPtr.ToSharedRef();
}

- (void)SaveFileHashHelperState
{
	@synchronized (self)
	{
		[self GetFileHashHelper]->SaveData();
	}
}

- (NSString*)GetTempPathForURL:(NSURL* _Nonnull)URL WithRequestID:(NSString*)RequestID
{
	@synchronized (self)
	{
		BackgroundHttpFileHashHelperRef HelperRef = [self GetFileHashHelper];

		const FString TaskURL(URL.absoluteString);
		const FString& TempFileName = HelperRef->FindOrAddTempFilenameMappingForURL(TaskURL, RequestID);
		const FString& DestinationPath = HelperRef->GetFullPathOfTempFilename(TempFileName);
		
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		const FString ConvertedPath = PlatformFile.ConvertToAbsolutePathForExternalAppForWrite(*DestinationPath);

		const FString DestinationDir = FPaths::GetPath(ConvertedPath);
		PlatformFile.CreateDirectory(*DestinationDir);

		return ConvertedPath.GetNSString();
	}
}

// Orders candidate URLs by their matching CDN, just appending any candidate without a match.
- (NSMutableArray<__kindof NSURL*>*)OrderURLs:(NSArray<__kindof NSURL*>*)Candidates ByCDNs:(NSArray<__kindof FBackgroundNSURLCDNInfo*>*)CDNs
{
	// Array sizes are assumed small enough that hashmap is not needed

	NSMutableArray<__kindof NSURL*>* Result = [NSMutableArray arrayWithCapacity:[Candidates count]];

	for (FBackgroundNSURLCDNInfo* Info in CDNs)
	{
		for (NSURL* URL in Candidates)
		{
			if ([Info.CDNHost isEqualToString:URL.host])
			{
				[Result addObject:URL];
				break;
			}
		}
	}

	// Add CDN's that weren't present at first lookup
	for (NSURL* URL in Candidates)
	{
		if (![Result containsObject:URL])
		{
			[Result addObject:URL];
		}
	}

	return Result;
}

- (NSMutableArray<__kindof NSURL*>*)ReorderCDNsByReachability:(NSMutableArray<__kindof NSURL*>*)URLs
{
	if (_HeadRequestTimeout == 0 || [URLs count] == 0)
	{
		return URLs;
	}

	// Fast path: re-uses existing CDN info.
	@synchronized (_CDNInfo)
	{
		if ([_CDNInfo count] != 0)
		{
			return [self OrderURLs:URLs ByCDNs:_CDNInfo];
		}
	}

	// We only allow a single-fligh CDN info update.
	@synchronized (_CDNProbeGate)
	{
		// A prior gate holder may have already published the info.
		@synchronized (_CDNInfo)
		{
			if ([_CDNInfo count] != 0)
			{
				return [self OrderURLs:URLs ByCDNs:_CDNInfo];
			}
		}

		UE_DNLD_LOG(@"Starting to check for CDN reachability");

		// Requests will put results into a local Pending state, which is later
		// published into _CDNInfo.
		NSMutableArray<__kindof FBackgroundNSURLCDNInfo*>* Pending = [NSMutableArray array];

		std::shared_ptr<std::atomic<int32>> InflightCount = std::make_shared<std::atomic<int32>>((int32)URLs.count);
		std::shared_ptr<std::promise<void>> Done = std::make_shared<std::promise<void>>();
		std::shared_ptr<std::atomic<bool>> AcceptWrites = std::make_shared<std::atomic<bool>>(true);

		NSDate* StartTime = [NSDate date];

		for (NSURL* URL in URLs)
		{
			NSMutableURLRequest* Request = [NSMutableURLRequest requestWithURL:URL];
			// Use HEAD request because we want the smallest response possible from the CDN to see if the connection works at all.
			[Request setHTTPMethod:@"HEAD"];

			UE_DNLD_LOG(@"Create data task for '%@'", Request.URL.absoluteString);

			// Note, completion handler might be invoked after end of this method.
			NSURLSessionDataTask* Task = [_HeadRequestSession dataTaskWithRequest:Request completionHandler:^(NSData* _Nullable Data, NSURLResponse* _Nullable Response, NSError* _Nullable Error)
			{
				// Fast reject — handler may fire after wait_for; skipping the decrement is fine since Done is no longer awaited.
				if (!AcceptWrites->load())
				{
					return;
				}

				const NSTimeInterval ResponseTime = -[StartTime timeIntervalSinceNow];

				bool bIsOk = false;
				const bool bIsTimeout = (Error != nil) ? (Error.code == NSURLErrorTimedOut || Error.code == NSURLErrorCancelled) : false;

				if (Response != nil && [Response isKindOfClass:[NSHTTPURLResponse class]])
				{
					NSHTTPURLResponse* HTTPResponse = (NSHTTPURLResponse*)Response;
					UE_DNLD_LOG(@"Finished data task for '%@' (host '%@') with status code %li and response time %f", Request.URL.absoluteString, Request.URL.host, HTTPResponse.statusCode, ResponseTime);

					if (HTTPResponse.statusCode < HTTPStatusCodeErrorBadRequest)
					{
						bIsOk = true;
					}
				}
				else
				{
					UE_DNLD_LOG(@"Finished data task for '%@' with error '%@' (%i, %i) and response time %f", Request.URL.absoluteString, (Error != nil ? Error.localizedDescription : @"nil"), (Error != nil ? (int32)Error.code : 0), bIsTimeout ? 1 : 0, ResponseTime);
				}

				FBackgroundNSURLCDNInfo* Info = [[FBackgroundNSURLCDNInfo alloc] init];
				[Info SetFromURL:Request.URL];
				if (bIsOk)
				{
					[Info setResponse:EBackgroundNSURLCDNInfoResponse::Ok];
				}
				else if (bIsTimeout)
				{
					[Info setResponse:EBackgroundNSURLCDNInfoResponse::Timeout];
				}
				else
				{
					[Info setResponse:EBackgroundNSURLCDNInfoResponse::Error];
				}
				[Info setResponseTime:ResponseTime];

				@synchronized (Pending)
				{
					if (AcceptWrites->load())
					{
						[Pending addObject:Info];
					}
				}
				[Info release];

				if (InflightCount->fetch_sub(1) == 1)
				{
					UE_DNLD_LOG(@"Finished all data tasks for CDN reachability");
					Done->set_value();
				}
			}];
			[Task setPriority:NSURLSessionTaskPriorityHigh];
			[Task resume];
		}

		Done->get_future().wait_for(std::chrono::milliseconds(_HeadRequestTimeout));
		UE_DNLD_LOG(@"Finished waiting for CDN reachability");

		AcceptWrites->store(false);
		[_HeadRequestSession resetWithCompletionHandler:^{}];

		// AcceptWrites was flipped above, so any handler entering Pending now reads
		// false and skips. So either task added info or won't touch it (in a case of
		// timeout).
		@synchronized (Pending)
		{
			for (NSUInteger URLIndex = 0; URLIndex < [URLs count]; URLIndex++)
			{
				NSURL* URL = [URLs objectAtIndex:URLIndex];
				bool bFoundCDNInfo = false;

				for (FBackgroundNSURLCDNInfo* Info in Pending)
				{
					if ([Info.CDNHost isEqualToString:URL.host])
					{
						[Info setProvidedOrder:URLIndex];
						bFoundCDNInfo = true;
						break;
					}
				}

				if (!bFoundCDNInfo)
				{
					FBackgroundNSURLCDNInfo* Info = [[FBackgroundNSURLCDNInfo alloc] init];
					[Info SetFromURL:URL];
					// If cdn/networking hasn't provided us with any info, consider request timed out
					[Info setResponse:EBackgroundNSURLCDNInfoResponse::Timeout];
					[Info setProvidedOrder:URLIndex];
					[Pending addObject:Info];
					[Info release];
				}
			}

			[Pending sortUsingComparator:^NSComparisonResult(FBackgroundNSURLCDNInfo* _Nonnull A, FBackgroundNSURLCDNInfo* _Nonnull B)
			{
				const double KeyA = [A SortingKeyWith:_bCDNReorderByPingTime];
				const double KeyB = [B SortingKeyWith:_bCDNReorderByPingTime];

				if (KeyA < KeyB)
				{
					return NSOrderedAscending;
				}
				else if (KeyA > KeyB)
				{
					return NSOrderedDescending;
				}
				else
				{
					return NSOrderedSame;
				}
			}];

			for (NSUInteger i = 0; i < [Pending count]; i++)
			{
				FBackgroundNSURLCDNInfo* Info = [Pending objectAtIndex:i];
				UE_DNLD_LOG(@"%lu CDN '%@' AbsoluteURL '%@' Response:%u ResponseTime:%f ProvidedOrder:%lu SortingKey:%f", i, Info.CDNHost, Info.CDNAbsoluteURL, Info.Response, Info.ResponseTime, (unsigned long)Info.ProvidedOrder, [Info SortingKeyWith:_bCDNReorderByPingTime]);
			}
		}

		NSMutableArray<__kindof NSURL*>* Result = [self OrderURLs:URLs ByCDNs:Pending];

		// _CDNInfo is write-once, published under the  _CDNProbeGate.
		@synchronized (_CDNInfo)
		{
			[_CDNInfo addObjectsFromArray:Pending];
		}

		return Result;
	}
}

- (NSArray<__kindof FBackgroundNSURLCDNInfo*>*)GetCDNInfo
{
	@synchronized (_CDNInfo)
	{
		return [[_CDNInfo copy] autorelease];
	}
}

- (NSURLSessionDownloadTask*)CreateDownloadForURL:(NSURL* _Nonnull)URL WithPriority:(float)Priority WithTaskData:(FBackgroundNSURLSessionDownloadTaskData* _Nonnull)TaskData;
{
	NSMutableURLRequest* URLRequest = [NSMutableURLRequest requestWithURL:URL];
	URLRequest.allowsCellularAccess = self.AllowCellular;
	URLRequest.allowsConstrainedNetworkAccess = self.AllowConstrained;
	URLRequest.allowsExpensiveNetworkAccess = self.AllowCellular;
#if !UE_BUILD_SHIPPING
	if ([URL.host hasSuffix:@".akamaized.net"])
	{
		[URLRequest setValue:@"akamai-x-cache-on, akamai-x-cache-remote-on, akamai-x-check-cacheable" forHTTPHeaderField:@"Pragma"];
	}
#endif

	NSURLSessionDownloadTask* Task = [_Session downloadTaskWithRequest:URLRequest];
	[Task setPriority:Priority];
	[Task setTaskDescription:[TaskData ToSerializedString]];
	if (TaskData.ExpectedResultSizeFromHeadRequest > 0)
	{
		if (TaskData.ExpectedResultSizeFromHeadRequest != TaskData.ExpectedResultSize)
		{
			UE_DNLD_LOG(@"Mismatch expectedSize from '%@' between HEAD request %llu and config %llu", URL.absoluteString, TaskData.ExpectedResultSizeFromHeadRequest, TaskData.ExpectedResultSize);
		}
		[Task setCountOfBytesClientExpectsToReceive:TaskData.ExpectedResultSizeFromHeadRequest + TypicalSizeForDownloadResponseHeader];
	}
	else
	{
		[Task setCountOfBytesClientExpectsToReceive:TaskData.ExpectedResultSize + TypicalSizeForDownloadResponseHeader];
	}

	UE_DNLD_LOG(@"CreateDownloadForURL '%@' with taskIdentifier %lu", Task.taskDescription, Task.taskIdentifier);
	return Task;
}

- (NSURLSessionDownloadTask*)CreateDownloadForResumeData:(NSData* _Nonnull)ResumeData WithPriority:(float)Priority WithTaskData:(FBackgroundNSURLSessionDownloadTaskData* _Nonnull)TaskData
{
	NSURLSessionDownloadTask* Task = [_Session downloadTaskWithResumeData:ResumeData];
	[Task setPriority:Priority];
	[Task setTaskDescription:[TaskData ToSerializedString]];

	UE_DNLD_LOG(@"CreateDownloadForResumeData '%@' with taskIdentifier %lu", Task.taskDescription, Task.taskIdentifier);
	return Task;
}

- (NSUInteger)CreateOrFindDownloadForURLs:(NSArray<__kindof NSString*>*)URLStrings WithPriority:(float)Priority WithExpectedResultSize:(uint64)ExpectedResultSize WithRequestID:(NSString*)RequestID
{
	if (_AllDownloadsFuture.wait_for(std::chrono::seconds(0)) != std::future_status::ready)
	{
		UE_DNLD_LOG(@"Starting wait for existing downloads status");
		
		_AllDownloadsFuture.wait();

		UE_DNLD_LOG(@"Done waiting for existing downloads status");
	}

	// To be able to store less state we assume all strings have same asset path suffix
	NSString* AssetPath = nil;
	NSMutableArray<__kindof NSURL*>* URLs = [NSMutableArray arrayWithCapacity:URLStrings.count];
	for (NSString* URLString in URLStrings)
	{
		NSURL* URLValue = [NSURL URLWithString:URLString];
		if (AssetPath == nil)
		{
			AssetPath = URLValue.path;
			NSString* Query = URLValue.query;
			if (Query != nil)
			{
				AssetPath = [[AssetPath stringByAppendingString:@"?"] stringByAppendingString:Query];
			}
		}
		[URLs addObject:URLValue];
	}

	if (GBackgroundHttpReorderCDNsByReachability)
	{
		URLs = [self ReorderCDNsByReachability:URLs];
	}

	// Serialize current settings
	FBackgroundNSURLSessionDownloadTaskData* TaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataWithURLs:URLs WithRetryCount:_RetryResumeDataLimit WithExpectedResultSize:ExpectedResultSize WithRequestID:RequestID];

	// Check for existing download task, could be from previous app session.
	UE_DNLD_LOG(@"Trying to find existing download for asset path '%@'", AssetPath);
	if (AssetPath != nil)
	{
		@synchronized(_AllDownloads)
		{
			__block NSUInteger ExistingDownloadId = InvalidDownloadId;
			__block NSURLSessionDownloadTask* ExistingTask = nil;
			[_AllDownloads enumerateKeysAndObjectsUsingBlock:^(NSNumber* IterKey, NSURLSessionDownloadTask* IterTask, BOOL* IterStop)
			 {
				if ((IterTask.originalRequest != nil && [IterTask.originalRequest.URL.path isEqualToString:AssetPath])
					|| (IterTask.currentRequest != nil && [IterTask.currentRequest.URL.path isEqualToString:AssetPath]))
				{
					FBackgroundNSURLSessionDownloadTaskData* IterTaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataFromSerializedString:IterTask.taskDescription];
					if (TaskData == nil || [IterTaskData.RequestID isEqualToString:RequestID])
					{
						ExistingDownloadId = IterKey.unsignedIntegerValue;
						ExistingTask = [IterTask retain]; // Retain in case if task gets killed in another thread.
						*IterStop = YES;
					}
				}
			}];

			if (ExistingDownloadId != InvalidDownloadId && ExistingTask != nil)
			{				
				UE_DNLD_LOG(@"Found existing download task for path '%@' requestID '%@' with DownloadId %lu", AssetPath, RequestID, ExistingDownloadId);
				
				// Update existing task state to new one, to reset retry counters, cdn links, etc.
				[ExistingTask setTaskDescription:[TaskData ToSerializedString]];
				
				[ExistingTask resume]; // Resume task just in case if it was not running before.
				[ExistingTask release];
				
				return ExistingDownloadId;
			}
		}
	}

	NSURL* URL = [TaskData GetFirstURL];
	
	NSURLSessionDownloadTask* DownloadTask = [self CreateDownloadForURL:URL WithPriority:Priority WithTaskData:TaskData];
	const NSUInteger DownloadId = [self EnsureTaskIsTracked:DownloadTask];
	
	// Do not start the Download yet, use HEAD request to get more accurate expected size first, also may trigger CDN cache fetching
	NSMutableURLRequest* Request = [NSMutableURLRequest requestWithURL:URL];
	[Request setHTTPMethod:@"HEAD"];

	UE_DNLD_LOG(@"Create HEAD request for '%@'", Request.URL.absoluteString);

	NSURLSessionDataTask* HeadRequestTask = [_HeadRequestSession dataTaskWithRequest:Request completionHandler:^(NSData* _Nullable Data, NSURLResponse* _Nullable Response, NSError* _Nullable Error)
	{
		if (Response != nil && [Response isKindOfClass:[NSHTTPURLResponse class]])
		{
			NSHTTPURLResponse* HTTPResponse = (NSHTTPURLResponse*)Response;
			[TaskData setExpectedResultSizeFromHeadRequest: (NSUInteger)[[HTTPResponse valueForHTTPHeaderField:@"Content-Length"] integerValue]];
			UE_DNLD_LOG(@"Finished HEAD request for '%@' (host '%@') with status code %li and Content-Length %llu", Request.URL.absoluteString, Request.URL.host, HTTPResponse.statusCode, TaskData.ExpectedResultSizeFromHeadRequest);
		}
		else
		{
			UE_DNLD_LOG(@"Finished HEAD request for '%@' with error '%@' %i", Request.URL.absoluteString, (Error != nil ? Error.localizedDescription : @"nil"), (Error != nil ? (int32)Error.code : 0));
		}
		[DownloadTask setTaskDescription:[TaskData ToSerializedString]];
		
		if (TaskData.ExpectedResultSizeFromHeadRequest > 0)
		{
			if (TaskData.ExpectedResultSizeFromHeadRequest != TaskData.ExpectedResultSize)
			{
				UE_DNLD_LOG(@"Mismatch expectedSize from '%@' between HEAD request %llu and config %llu", URL.absoluteString, TaskData.ExpectedResultSizeFromHeadRequest, TaskData.ExpectedResultSize);
			}
			[DownloadTask setCountOfBytesClientExpectsToReceive:TaskData.ExpectedResultSizeFromHeadRequest + TypicalSizeForDownloadResponseHeader];
		}
		else
		{
			UE_DNLD_LOG(@"ExpectedSize from HEAD request is missing for '%@', using size from DownloadConfig %llu", URL.absoluteString, TaskData.ExpectedResultSize);
			[DownloadTask setCountOfBytesClientExpectsToReceive:TaskData.ExpectedResultSize + TypicalSizeForDownloadResponseHeader];
		}
		
		[DownloadTask resume];
	}];
	
	[HeadRequestTask setPriority:NSURLSessionTaskPriorityHigh];
	[HeadRequestTask resume];

	return DownloadId;
}

- (void)PauseDownload:(NSUInteger)DownloadId
{
	UE_DNLD_LOG(@"PauseDownload for DownloadId %lu", DownloadId);

	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task != nil)
	{
		[Task suspend];
	}
}

- (void)ResumeDownload:(NSUInteger)DownloadId
{
	UE_DNLD_LOG(@"ResumeDownload for DownloadId %lu", DownloadId);

	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task != nil)
	{
		[Task resume];
	}
}

- (void)CancelDownload:(NSUInteger)DownloadId
{
	UE_DNLD_LOG(@"CancelDownload for DownloadId %lu", DownloadId);

	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task == nil)
	{
		return;
	}

	FBackgroundNSURLSessionDownloadTaskData* TaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataFromSerializedString:Task.taskDescription];
	if (TaskData != nil)
	{
		// Remove task data from this download task, otherwise didCompleteWithError might retry the request.
		[TaskData Cancel];
		Task.taskDescription = [TaskData ToSerializedString];
	}

	// Must retain otherwise EnsureTaskIsNotTracked will free the object.
	[Task retain];

	// We're done with this task.
	[self EnsureTaskIsNotTracked:Task];

	// Will invoke didCompleteWithError if task is incomplete.
	[Task cancel];
	
	// Finally release the object.
	[Task release];
}

- (void)SetPriority:(float)Priority ForDownload:(NSUInteger)DownloadId
{
	UE_DNLD_LOG(@"SetPriority %f for DownloadId %lu", Priority, DownloadId);

	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task != nil)
	{
		[Task setPriority:Priority];
	}
}

- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didFinishCollectingMetrics:(NSURLSessionTaskMetrics *)metrics {
#if !UE_BUILD_SHIPPING
	NSDateFormatter *formatter = [[NSDateFormatter alloc] init];
    [formatter setDateFormat:@"yyyy-MM-dd'T'HH:mm:ss.SSSZZZZZ"];
    [formatter setLocale:[NSLocale localeWithLocaleIdentifier:@"en_US_POSIX"]];
    
    // Calculate task interval
    NSURLSessionTaskTransactionMetrics *firstMetric = metrics.transactionMetrics.firstObject;
    NSURLSessionTaskTransactionMetrics *lastMetric = metrics.transactionMetrics.lastObject;
	UE_DNLD_LOG(@"Task %lu got metrics", task.taskIdentifier);
    if (firstMetric && lastMetric) {
        NSTimeInterval taskInterval = [lastMetric.requestEndDate timeIntervalSinceDate:firstMetric.requestStartDate];
        UE_DNLD_LOG(@"Task Interval: %f", taskInterval);
    }

    // Get redirect count
    NSInteger redirectCount = metrics.transactionMetrics.count - 1; // The first metric is the initial request, redirects are additional metrics
    UE_DNLD_LOG(@"Redirect Count: %ld", (long)redirectCount);
#endif

	for (NSURLSessionTaskTransactionMetrics *metric in metrics.transactionMetrics) {
#if !UE_BUILD_SHIPPING
		UE_DNLD_LOG(@"Network Protocol Name: %@", metric.networkProtocolName);
		UE_DNLD_LOG(@"Reused Connection: %@", metric.reusedConnection ? @"Yes" : @"No");
		UE_DNLD_LOG(@"Proxy Connection: %@", metric.proxyConnection ? @"Yes": @"No");

        UE_DNLD_LOG(@"Fetch Start Date: %@", [formatter stringFromDate:metric.fetchStartDate]);
        UE_DNLD_LOG(@"Request Start Date: %@", [formatter stringFromDate:metric.requestStartDate]);
        UE_DNLD_LOG(@"Response Start Date: %@", [formatter stringFromDate:metric.responseStartDate]);
        UE_DNLD_LOG(@"Request End Date: %@", [formatter stringFromDate:metric.requestEndDate]);
        UE_DNLD_LOG(@"Response End Date: %@", [formatter stringFromDate:metric.responseEndDate]);
#endif

		// Calculate and log response duration
        if (metric.responseStartDate && metric.responseEndDate) {
            NSTimeInterval responseDuration = [metric.responseEndDate timeIntervalSinceDate:metric.responseStartDate];
#if !UE_BUILD_SHIPPING
            UE_DNLD_LOG(@"Response Duration: %f seconds", responseDuration);
#endif
			// Calculate and log download speed
            if (responseDuration > 0 && metric.countOfResponseBodyBytesReceived > 0) 
			{
				FBackgroundHttpRequestMetricsExtended MetricsExtended;

				MetricsExtended.TotalBytesDownloaded = (int32)metric.countOfResponseBodyBytesReceived;
				MetricsExtended.DownloadDuration = (float)responseDuration;
				MetricsExtended.FetchStartTimeUTC = FDateTime::FromUnixTimestampDecimal([metric.requestStartDate timeIntervalSince1970]);
				MetricsExtended.FetchEndTimeUTC = FDateTime::FromUnixTimestampDecimal([metric.responseEndDate timeIntervalSince1970]);

				FBackgroundURLSessionHandler::OnDownloadMetricsExtended.Broadcast((uint64)task.taskIdentifier, MetricsExtended);
				
#if !UE_BUILD_SHIPPING
				// bytes per second
				double downloadSpeed = (double)metric.countOfResponseBodyBytesReceived / responseDuration;
				NSString *formattedSpeed = [self formattedSpeed:downloadSpeed];
				UE_DNLD_LOG(@"Download Speed: %@", formattedSpeed);
            } 
			else 
			{
                UE_DNLD_LOG(@"Download Speed: Not Available");
#endif
            }
        }
#if !UE_BUILD_SHIPPING
		else 
		{
            UE_DNLD_LOG(@"Response Duration: Not Available");
        }
#endif
    }
#if !UE_BUILD_SHIPPING
	UE_DNLD_LOG(@"-------------------------");
#endif
}

#if !UE_BUILD_SHIPPING
- (NSString *)formattedSpeed:(double)speedInBytesPerSecond {
    NSArray *units = @[@"bytes/second", @"KB/s", @"MB/s", @"GB/s"];
    double speed = speedInBytesPerSecond;
    NSInteger unitIndex = 0;

    while (speed >= 1024 && unitIndex < units.count - 1) {
        speed /= 1024;
        unitIndex++;
    }
    
    return [NSString stringWithFormat:@"%.2f %@", speed, units[unitIndex]];
}
#endif

- (void)SetCurrentDownloadedBytes:(uint64)DownloadedBytes ForTask:(NSURLSessionDownloadTask*)Task
{
	if (Task != nil)
	{
#if !UE_BUILD_SHIPPING // only needed for GetDownloadDebugText
		if ([Task.progress.userInfo objectForKey:NSProgressDownloadStartTime] == nil)
		{
			[Task.progress setUserInfoObject:[NSNumber numberWithDouble:[[NSDate date] timeIntervalSince1970]] forKey:NSProgressDownloadStartTime];
		}
#endif

		[Task.progress setUserInfoObject:[NSNumber numberWithDouble:[[NSDate date] timeIntervalSince1970]] forKey:NSProgressDownloadLastUpdateTime];
		[Task.progress setUserInfoObject:[NSNumber numberWithLongLong:DownloadedBytes] forKey:NSProgressDownloadCompletedBytes];
	}
}

- (uint64)GetCurrentDownloadedBytes:(NSUInteger)DownloadId
{
	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task != nil)
	{
		NSNumber* CompletedBytes = [Task.progress.userInfo objectForKey:NSProgressDownloadCompletedBytes];
		if (CompletedBytes != nil)
		{
			return CompletedBytes.unsignedLongLongValue;
		}
		else // if didWriteData was not invoked yet
		{
			// TODO maybe we can use this for everything?
			return Task.countOfBytesReceived;
		}
	}

	return 0;
}

- (void)GetActiveDownloadIDs:(TSet<uint64>&)OutActiveIDs
{
	@synchronized(_AllDownloads)
	{
		[_AllDownloads enumerateKeysAndObjectsUsingBlock:^(NSNumber* Key, NSURLSessionDownloadTask* Task, BOOL* IterStop)
		{
			// NSURLSessionTaskStateRunning means "started, not suspended/cancelled" — it does NOT mean
			// the task has an active TCP connection. With MaximumConnectionsPerHost, NSURLSession internally
			// queues tasks beyond the connection limit and still reports them as Running.
			// Use two signals to determine if a task has a real connection:
			//   - countOfBytesReceived > 0: body data is flowing
			//   - countOfBytesExpectedToReceive > 0 and != unknown: response headers received (connection
			//     established, server responded) even if no body data yet
			const bool bHasReceivedData = Task.countOfBytesReceived > 0;
			const bool bHasReceivedHeaders = Task.countOfBytesExpectedToReceive > 0 && Task.countOfBytesExpectedToReceive != NSURLSessionTransferSizeUnknown;
			if (Task.state == NSURLSessionTaskStateRunning && (bHasReceivedData || bHasReceivedHeaders))
			{
				OutActiveIDs.Add(Key.unsignedLongLongValue);
			}
		}];
	}
}

- (void)RecreateDownload:(NSUInteger)DownloadId ShouldResetRetryCount:(bool)ResetRetryCount
{
	UE_DNLD_LOG(@"RecreateDownload for DownloadId %lu", DownloadId);

	NSURLSessionDownloadTask* OldTask = [self FindDownloadTaskFor:DownloadId];
	const float OldTaskPriority = OldTask.priority;
	const NSURLSessionTaskState OldTaskState = OldTask.state;

	FBackgroundNSURLSessionDownloadTaskData* OldTaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataFromSerializedString:OldTask.taskDescription];
	FBackgroundNSURLSessionDownloadTaskData* NewTaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataFromSerializedString:OldTask.taskDescription];
	if (OldTaskData == nil || NewTaskData == nil)
	{
		return;
	}
	
	if (ResetRetryCount)
	{
		[NewTaskData ResetRetryCount:_RetryResumeDataLimit];
	}

	// Remove task data from this download task, otherwise didCompleteWithError might retry the request.
	[OldTaskData Cancel];
	OldTask.taskDescription = [OldTaskData ToSerializedString];

	// Must retain otherwise EnsureTaskIsNotTracked will free the object.
	[OldTask retain];

	// We're done with this task.
	[self EnsureTaskIsNotTracked:OldTask];

	// If permissions changed, we need to recreate task from scratch skipping the resume data.
	// That is unfortunate, but otherwise we might recreate a task _after_ user allowed us to
	// use cellular/constrained network, which would have old permissions and not be able to
	// download.
	NSURLRequest* Request = OldTask.currentRequest != nil ? OldTask.currentRequest : OldTask.originalRequest;
	const bool bIsResumable =
		(Request == nil) || (
		Request.allowsCellularAccess           == self.AllowCellular    &&
		Request.allowsConstrainedNetworkAccess == self.AllowConstrained &&
		Request.allowsExpensiveNetworkAccess   == self.AllowCellular);

	if (_bUseCancelByProducingResumeData && bIsResumable)
	{
		// Will invoke didCompleteWithError if task is incomplete.
		[OldTask cancelByProducingResumeData:^(NSData * _Nullable ResumeData)
		{
			if (ResumeData != nil)
			{
				UE_DNLD_LOG(@"RecreateDownload OldTask cancelByProducingResumeData for DownloadId %lu returned resume data", DownloadId);
				
				NSURLSessionDownloadTask* NewTask = [self CreateDownloadForResumeData:ResumeData WithPriority:OldTaskPriority WithTaskData:NewTaskData];
				[self ReplaceTrackedTaskWith:NewTask ForDownloadId:DownloadId];
				
				if (OldTaskState != NSURLSessionTaskStateSuspended)
				{
					[NewTask resume];
				}
			}
			else
			{
				UE_DNLD_LOG(@"RecreateDownload OldTask cancelByProducingResumeData for DownloadId %lu did not return resume data", DownloadId);
				
				// start without resume data, will drop any already downloaded data
				NSURLSessionDownloadTask* NewTask = [self CreateDownloadForURL:[NewTaskData GetFirstURL] WithPriority:OldTaskPriority WithTaskData:NewTaskData];
				[self ReplaceTrackedTaskWith:NewTask ForDownloadId:DownloadId];
				
				if (OldTaskState != NSURLSessionTaskStateSuspended)
				{
					[NewTask resume];
				}
			}
		}];

		// Finally release the object.
		[OldTask release];

		OldTask = nil;
	}
	else
	{
		// Will invoke didCompleteWithError if task is incomplete.
		[OldTask cancel];

		// Finally release the object.
		[OldTask release];

		OldTask = nil;

		// start without resume data, will drop any already downloaded data
		NSURLSessionDownloadTask* NewTask = [self CreateDownloadForURL:[NewTaskData GetFirstURL] WithPriority:OldTaskPriority WithTaskData:NewTaskData];
		[self ReplaceTrackedTaskWith:NewTask ForDownloadId:DownloadId];
				
		if (OldTaskState != NSURLSessionTaskStateSuspended)
		{
			[NewTask resume];
		}
	}
}

- (void)RecreateDownloads
{
	UE_DNLD_LOG(@"RecreateDownloads started");

	// Copy keys to avoid deadlocking in case if cancel/resume/etc will call delegates in-place
	NSArray<__kindof NSNumber*>* AllKeys = nil;
	@synchronized(_AllDownloads)
	{
		AllKeys = [_AllDownloads allKeys];
	}

	for (NSNumber* IterKey in AllKeys)
	{
		const NSUInteger DownloadId = IterKey.unsignedIntegerValue;
		[self RecreateDownload:DownloadId ShouldResetRetryCount:true];
	}

	UE_DNLD_LOG(@"RecreateDownloads finished");
}

- (void)SwitchToForegroundSession
{
	UE_DNLD_LOG([self IsForegroundSession]
		? @"SwitchToForegroundSession: already in foreground"
		: @"SwitchToForegroundSession: switching from background");

	bool bDiscretionary = true;
	double TimeoutIntervalForRequest = 120.0;
	double TimeoutIntervalForResource = 60.0 * 60.0;

#ifndef UE_DNLD_SANDBOX
	GConfig->GetBool(TEXT("BackgroundHttp.iOSSettings"), TEXT("bDiscretionary"), bDiscretionary, GEngineIni);
	GConfig->GetDouble(TEXT("BackgroundHttp.iOSSettings"), TEXT("BackgroundReceiveTimeout"), TimeoutIntervalForRequest, GEngineIni);
	GConfig->GetDouble(TEXT("BackgroundHttp.iOSSettings"), TEXT("BackgroundHttpResourceTimeout"), TimeoutIntervalForResource, GEngineIni);
#endif

	UE_DNLD_LOG(@"bDiscretionary=%u", bDiscretionary ? 1 : 0);
	UE_DNLD_LOG(@"TimeoutIntervalForRequest=%f", TimeoutIntervalForRequest);
	UE_DNLD_LOG(@"TimeoutIntervalForResource=%f", TimeoutIntervalForResource);

	NSURLSessionConfiguration* Configuration = [NSURLSessionConfiguration defaultSessionConfiguration];
	Configuration.discretionary = bDiscretionary;
	Configuration.allowsCellularAccess = YES;
	Configuration.allowsExpensiveNetworkAccess = YES;
	Configuration.allowsConstrainedNetworkAccess = YES;
	Configuration.networkServiceType = NSURLNetworkServiceTypeDefault; // always default in foreground
	// TODO enable if entitlement is provided
	// Configuration.multipathServiceType = NSURLSessionMultipathServiceTypeAggregate;
	Configuration.HTTPMaximumConnectionsPerHost = _MaximumConnectionsPerHost; // TODO maybe x2 the number?
	Configuration.timeoutIntervalForRequest = TimeoutIntervalForRequest;
	Configuration.timeoutIntervalForResource = TimeoutIntervalForResource;

	NSURLSession* OldSession = _Session;
	_Session = [[NSURLSession sessionWithConfiguration:Configuration delegate:self delegateQueue:nil] retain];
	UE_DNLD_LOG(@"SwitchToForegroundSession: new foreground session created");

	[self RecreateDownloads];

	[OldSession invalidateAndCancel];
	[OldSession release];
}

- (BOOL)IsForegroundSession
{
	// Default sessions have a nil identifier; background-identifier sessions are non-nil.
	return _Session.configuration.identifier == nil;
}

#if !UE_BUILD_SHIPPING
- (NSString*)GetDownloadDebugText:(NSUInteger)DownloadId
{
	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task == nil)
	{
		return [NSString stringWithFormat:@"%llu is not tracked", (uint64)DownloadId];
	}

	NSString* Description = [NSString stringWithFormat:@"%llu:%@", (uint64)DownloadId, Task.currentRequest.URL.absoluteString];

	NSString* DetailMode = [self IsForegroundSession] ? @"FG" : @"BG";

	NSNumber* ResultStatusCode = [Task.progress.userInfo objectForKey:NSProgressDownloadResultStatusCode];
	NSString* DetailResult = ResultStatusCode != nil ?
		[NSString stringWithFormat:@" Done:%i", (int32)ResultStatusCode.integerValue]
		: @"";

	NSString* DetailCache = @" NoResp";
	if ([Task.response isKindOfClass:[NSHTTPURLResponse class]])
	{
		NSHTTPURLResponse* Response = (NSHTTPURLResponse*)Task.response;

		NSString* Cache  = [Response valueForHTTPHeaderField:@"X-Epic-Cache"]
						?: [Response valueForHTTPHeaderField:@"X-Cache"]
						?: [Response valueForHTTPHeaderField:@"CF-Cache-Status"];
		NSString* Age    = [Response valueForHTTPHeaderField:@"Age"];
		NSString* Via    = [Response valueForHTTPHeaderField:@"Via"];
		NSString* Server = [Response valueForHTTPHeaderField:@"Server"];

		DetailCache = 
			  Cache != nil  ? [NSString stringWithFormat:@" Cache:%@", Cache]
			: Age != nil    ? [NSString stringWithFormat:@" Age:%@s", Age]
			: Via != nil    ? [NSString stringWithFormat:@" Via:%@", Via]
			: Server != nil ? [NSString stringWithFormat:@" Srv:%@", Server]
			: @" NoCacheHdr";
	}

	NSNumber* CompletedBytes = [Task.progress.userInfo objectForKey:NSProgressDownloadCompletedBytes];
	NSString* DetailBytes = CompletedBytes != nil ?
		[NSString stringWithFormat:@" %.2fMB", (double)CompletedBytes.unsignedLongLongValue / (1024.0 * 1024.0)]
		: @"";

	NSNumber* StartTime = [Task.progress.userInfo objectForKey:NSProgressDownloadStartTime];
	NSString* DetailSpeed = @"";
	if (StartTime != nil && CompletedBytes != nil)
	{
		double Elapsed = [[NSDate date] timeIntervalSince1970] - StartTime.doubleValue;
		if (Elapsed > 0.0)
		{
			double MBps = (double)CompletedBytes.unsignedLongLongValue / Elapsed / (1024.0 * 1024.0);
			DetailSpeed = [NSString stringWithFormat:@" %.2fMB/s", MBps];
		}
	}
	
	return [NSString stringWithFormat:@"%@\n%@%@%@%@%@", Description, DetailMode, DetailResult, DetailBytes, DetailSpeed, DetailCache];
}
#endif

- (void)StartCheckingForStaleDownloads
{
	if (_ForegroundStaleDownloadCheckTimer == nil && _CheckForForegroundStaleDownloadsWithInterval > 0.0)
	{
		dispatch_async(dispatch_get_main_queue(), ^
		{
			if (_ForegroundStaleDownloadCheckTimer == nil && _CheckForForegroundStaleDownloadsWithInterval > 0.0)
			{
				_ForegroundStaleDownloadCheckTimer = [[NSTimer
													   scheduledTimerWithTimeInterval:_CheckForForegroundStaleDownloadsWithInterval
													   target:self
													   selector:@selector(CheckForStaleDownloads:)
													   userInfo:nil
													   repeats:YES]
													  retain];
				_ForegroundStaleDownloadCheckTimer.tolerance = _CheckForForegroundStaleDownloadsWithInterval * 0.5;
				[_ForegroundStaleDownloadCheckTimer fire];
				
				UE_DNLD_LOG(@"Start checking for stale downloads");
			}
		});
	}
}

- (void)StopCheckingForStaleDownloads
{
	if (_ForegroundStaleDownloadCheckTimer != nil)
	{
		dispatch_async(dispatch_get_main_queue(), ^
		{
			if (_ForegroundStaleDownloadCheckTimer != nil)
			{
				[_ForegroundStaleDownloadCheckTimer invalidate];
				[_ForegroundStaleDownloadCheckTimer release];
				_ForegroundStaleDownloadCheckTimer = nil;
				
				UE_DNLD_LOG(@"Stop checking for stale downloads");
			}
		});
	}
}

- (void)CheckForStaleDownloads:(NSTimer*)Timer
{
	// Only check for stale downloads in foreground
	if ([UIApplication sharedApplication].applicationState != UIApplicationStateActive)
	{
		return;
	}

	// Copy keys to avoid deadlocking in case if cancel/resume/etc will call delegates in-place
	NSArray<__kindof NSNumber*>* AllKeys = nil;
	@synchronized(_AllDownloads)
	{
		AllKeys = [_AllDownloads allKeys];
	}

	const NSTimeInterval CurrentTime = [[NSDate date] timeIntervalSince1970];

	for (NSNumber* IterKey in AllKeys)
	{
		const NSUInteger DownloadId = IterKey.unsignedIntegerValue;
		NSURLSessionDownloadTask* Task = nil;
		@synchronized(_AllDownloads)
		{
			Task = [_AllDownloads objectForKey:IterKey];
		}
		if (Task == nil)
		{
			continue;
		}

		NSNumber* ResultStatusCode = [Task.progress.userInfo objectForKey:NSProgressDownloadResultStatusCode];
		if (ResultStatusCode != nil)
		{
			// Task is finished downloading, skip.
			continue;
		}

		NSNumber* LastUpdateTimeNumber = [Task.progress.userInfo objectForKey:NSProgressDownloadLastUpdateTime];
		if (LastUpdateTimeNumber == nil)
		{
			// There is no known last update time, skip.
			// As we don't get any notification from background tasks on when they started,
			// we can't distinguish between task pending processing vs task stuck on getting first byte.
			continue;
		}

		const NSTimeInterval TimeSinceLastUpdate = CurrentTime - LastUpdateTimeNumber.doubleValue;
		if (TimeSinceLastUpdate >= _ForegroundStaleDownloadTimeout)
		{
			UE_DNLD_LOG(@"Task '%@' with taskIdentifier %lu is considering stale, retrying", Task.taskDescription, Task.taskIdentifier);

			// Clear last update property to avoid canceling task twice in next tick.
			[Task.progress setUserInfoObject:nil forKey:NSProgressDownloadLastUpdateTime];
			Task = nil;

			[self RecreateDownload:DownloadId ShouldResetRetryCount:false];
		}
	}
}

- (void)HandleDidEnterBackground
{
}

- (void)HandleWillEnterForeground
{
	@synchronized(_AllDownloads)
	{
		[_AllDownloads enumerateKeysAndObjectsUsingBlock:^(NSNumber* Key, NSURLSessionDownloadTask* Task, BOOL* IterStop)
		{
			NSNumber* ResultStatusCode = [Task.progress.userInfo objectForKey:NSProgressDownloadResultStatusCode];

			// If task was not complete
			if (ResultStatusCode == nil)
			{
				NSNumber* LastUpdateTimeNumber = [Task.progress.userInfo objectForKey:NSProgressDownloadLastUpdateTime];

				// But has last update time
				if (LastUpdateTimeNumber != nil)
				{
					// Refresh task update time so stale timer doesn't retry task for first N seconds after app goes to foreground
					[Task.progress setUserInfoObject:[NSNumber numberWithDouble:[[NSDate date] timeIntervalSince1970]] forKey:NSProgressDownloadLastUpdateTime];

					UE_DNLD_LOG(@"Refreshing last update time for task '%@' taskIdentifier %lu", Task.taskDescription, Task.taskIdentifier);
				}
			}
		}];
	}
}

- (NSURLSessionDownloadTask*)FindDownloadTaskFor:(NSUInteger)DownloadId
{
	@synchronized(_AllDownloads)
	{
		return [_AllDownloads objectForKey:[NSNumber numberWithUnsignedInteger:DownloadId]];
	}
}

- (NSUInteger)FindDownloadIdForTask:(NSURLSessionDownloadTask*)Task
{
	// TODO this is slow, optimize if needed.
	__block NSUInteger ExistingDownloadId = InvalidDownloadId;
	@synchronized(_AllDownloads)
	{
		[_AllDownloads enumerateKeysAndObjectsUsingBlock:^(NSNumber* IterKey, NSURLSessionDownloadTask* IterTask, BOOL* IterStop)
		{
			if (IterTask == Task)
			{
				ExistingDownloadId = IterKey.unsignedIntegerValue;
				*IterStop = YES;
			}
		}];
	}

	return ExistingDownloadId;
}

- (NSUInteger)EnsureTaskIsTracked:(NSURLSessionDownloadTask*)Task
{
	const NSUInteger ExistingDownloadId = [self FindDownloadIdForTask:Task];
	if (ExistingDownloadId != InvalidDownloadId)
	{
		return ExistingDownloadId;
	}

	@synchronized(_AllDownloads)
	{
		const NSUInteger DownloadId = _NextDownloadId++;
		[_AllDownloads setObject:Task forKey:[NSNumber numberWithUnsignedInteger:DownloadId]];

		// start timer as long as we have active tasks
		[self StartCheckingForStaleDownloads];

		return DownloadId;
	}
}

- (void)ReplaceTrackedTaskWith:(NSURLSessionDownloadTask*)NewTask ForDownloadId:(NSUInteger)DownloadId
{
	@synchronized(_AllDownloads)
	{
		[_AllDownloads setObject:NewTask forKey:[NSNumber numberWithUnsignedInteger:DownloadId]];

		// in case if all downloads were drained, we need to restart a timer as a new task was inserted
		[self StartCheckingForStaleDownloads];
	}
}

- (void)EnsureTaskIsNotTracked:(NSURLSessionDownloadTask*)Task
{
	const NSUInteger ExistingDownloadId = [self FindDownloadIdForTask:Task];
	if (ExistingDownloadId != InvalidDownloadId)
	{
		@synchronized(_AllDownloads)
		{
			[_AllDownloads removeObjectForKey:[NSNumber numberWithUnsignedInteger:ExistingDownloadId]];

			if (_AllDownloads.count == 0)
			{
				// stop timer when we have no ongoing tasks
				[self StopCheckingForStaleDownloads];
			}
		}
	}
}

- (void)SetDownloadResult:(NSInteger)HTTPCode IsDiskSpaceError:(BOOL)bIsDiskSpaceError WithTempFile:(NSString*)TempFile ForDownload:(NSURLSessionDownloadTask*)Task
{
	const NSUInteger DownloadId = [self FindDownloadIdForTask:Task];
	if (DownloadId == InvalidDownloadId)
	{
		UE_DNLD_LOG(@"Can't find DownloadId for task '%@'", Task.taskDescription);
		return;
	}

	// We don't necessarily care if these values survive between application restarts.
	// Otherwise we need to put them inside FBackgroundNSURLSessionDownloadTaskData.
	[Task.progress setUserInfoObject:[NSNumber numberWithInteger:HTTPCode] forKey:NSProgressDownloadResultStatusCode];
	[Task.progress setUserInfoObject:TempFile forKey:NSProgressDownloadResultTempFilePath];

	const bool bDownloadSuccess = TempFile != nil;
	
	EBackgroundHttpRequestCompletionResult Result;
	if (bIsDiskSpaceError)
		Result = EBackgroundHttpRequestCompletionResult::OutOfDiskSpaceError;
	else if (!bDownloadSuccess)
		Result = EBackgroundHttpRequestCompletionResult::GenericError;
	else
		Result = EBackgroundHttpRequestCompletionResult::Success;
	
	FBackgroundURLSessionHandler::OnDownloadCompletedExtended.Broadcast(DownloadId, Result);
}

- (NSString* _Nullable)GetDownloadResult:(NSUInteger)DownloadId OutStatus:(BOOL* _Nonnull)OutStatus OutStatusCode:(NSInteger*)OutStatusCode
{
	NSURLSessionDownloadTask* Task = [self FindDownloadTaskFor:DownloadId];
	if (Task == nil)
	{
		*OutStatus = NO;
		return nil;
	}

	NSNumber* ResultStatusCode = [Task.progress.userInfo objectForKey:NSProgressDownloadResultStatusCode];
	if (ResultStatusCode == nil)
	{
		*OutStatus = NO;
		return nil;
	}

	*OutStatus = YES;
	*OutStatusCode = ResultStatusCode.integerValue;

	return [Task.progress.userInfo objectForKey:NSProgressDownloadResultTempFilePath];
}

//- (void)URLSession:(NSURLSession*)Session didBecomeInvalidWithError:(NSError*)Error
//{
//	UE_DNLD_LOG(@"didBecomeInvalidWithError");
//}

- (void)URLSessionDidFinishEventsForBackgroundURLSession:(NSURLSession*)Session
{
	UE_DNLD_LOG(@"URLSessionDidFinishEventsForBackgroundURLSession");

	NSString* Id = Session.configuration.identifier;

	if ([FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap objectForKey:Id] == nil)
	{
		return;
	}
	
	void(^CompletionHandler)() = [[FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap objectForKey:Id] retain];
	[FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap removeObjectForKey:Id];

	// Completion handler has to be invoked on the main thread.
	[[NSOperationQueue mainQueue] addOperationWithBlock:^
	{
		UE_DNLD_LOG(@"URLSessionDidFinishEventsForBackgroundURLSession calling completion handler.");

		FBackgroundURLSessionHandler::OnDownloadsCompletedWhileAppWasNotRunning.Broadcast(!_bAnyTaskDidCompleteWithError);
		_bAnyTaskDidCompleteWithError = false;

		[self SaveFileHashHelperState];

		CompletionHandler();
		[CompletionHandler release];
	}];
}

static bool IsOutOfDiskSpaceError(NSError* Error, int Depth = 0)
{
	if (!Error)
	{
		return false;
	}
	
	// prevent theoretical cyclic references that would never end; number is arbitrarily chosen
	if (Depth >= 5)
	{
		return false;
	}
	
	if ([Error.domain isEqualToString:NSCocoaErrorDomain] && Error.code == NSFileWriteOutOfSpaceError)
	{
		return true;
	}
	
	if ([Error.domain isEqualToString:NSPOSIXErrorDomain] && Error.code == ENOSPC)
	{
		return true;
	}
	
	NSError* Underlying = Error.userInfo[NSUnderlyingErrorKey];
	if (Underlying)
	{
		return IsOutOfDiskSpaceError(Underlying, Depth + 1);
	}
	
	return false;
}

- (void)URLSession:(NSURLSession*)Session task:(NSURLSessionTask*)GenericTask didCompleteWithError:(NSError*)Error
{
	if (Error == nil)
	{
		UE_DNLD_LOG(@"didCompleteWithError, task '%@' with taskIdentifier %lu is completed", GenericTask.taskDescription, GenericTask.taskIdentifier);
		return;
	}

	if (![GenericTask isKindOfClass:[NSURLSessionDownloadTask class]])
	{
		UE_DNLD_LOG(@"didCompleteWithError, ignoring task '%@' with taskIdentifier %lu", GenericTask.taskDescription, GenericTask.taskIdentifier);
		return;
	}
	
	const bool bOutOfDiskSpaceError = IsOutOfDiskSpaceError(Error);

	// Set it even if we will retry the download, as the only use of this variable is to report it URLSessionDidFinishEventsForBackgroundURLSession,
	// in that context retrying any download means that all downloads hasn't been completed yet.
	_bAnyTaskDidCompleteWithError = true;

	NSURLSessionDownloadTask* Task = (NSURLSessionDownloadTask*)GenericTask;
	NSString* LocalizedDescription = Error.localizedDescription;

	NSNumber* CancelReason = [Error.userInfo objectForKey:NSURLErrorBackgroundTaskCancelledReasonKey];
	if (CancelReason != nil)
	{
		LocalizedDescription = [NSString stringWithFormat:@"%@ (BackgroundTaskCancelledReason=%i)", LocalizedDescription, CancelReason.intValue];
	}

	const NSUInteger DownloadId = [self FindDownloadIdForTask:Task];
	const bool bIsTrackedTask = DownloadId != InvalidDownloadId;

	FBackgroundNSURLSessionDownloadTaskData* TaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataFromSerializedString:Task.taskDescription];
	
	if (!bOutOfDiskSpaceError && bIsTrackedTask && TaskData != nil)
	{
		NSData* ResumeData = [Error.userInfo objectForKey:NSURLSessionDownloadTaskResumeData];
		const bool bHasResumeData = ResumeData != nil && ResumeData.length > 0;
		
		NSURL* NextURL = [TaskData GetNextURL];
		
		// Continue trying if next URL is available.
		if (NextURL != nil)
		{
			// Create resume request if our URL is the same and we have resume data.
			if (bHasResumeData && NextURL != nil && [NextURL.absoluteString isEqualToString:Task.originalRequest.URL.absoluteString])
			{
				UE_DNLD_LOG(@"didCompleteWithError, task '%@' with taskIdentifier %lu failed due to '%@' and has resume data and next url is the same, retrying", Task.taskDescription, Task.taskIdentifier, LocalizedDescription);

				NSURLSessionDownloadTask* NewTask = [self CreateDownloadForResumeData:ResumeData WithPriority:Task.priority WithTaskData:TaskData];

				Task = nil;
				[self ReplaceTrackedTaskWith:NewTask ForDownloadId:DownloadId];

				[NewTask resume];
				return;
			}
			else
			{
				if (bHasResumeData)
				{
					// It should be possible to patch resume data to point to a new URL. But there is no public API to do that yet.
					UE_DNLD_LOG(@"didCompleteWithError, task '%@' with taskIdentifier %lu failed due to '%@' and has resume data but next url is different, retrying", Task.taskDescription, Task.taskIdentifier, LocalizedDescription);
				}
				else
				{
					UE_DNLD_LOG(@"didCompleteWithError, task '%@' with taskIdentifier %lu failed due to '%@' and has no resume data or next url is different, retrying", Task.taskDescription, Task.taskIdentifier, LocalizedDescription);
				}

				NSURLSessionDownloadTask* NewTask = [self CreateDownloadForURL:NextURL WithPriority:Task.priority WithTaskData:TaskData];

				Task = nil;
				[self ReplaceTrackedTaskWith:NewTask ForDownloadId:DownloadId];

				[NewTask resume];
				return;
			}
		}
	}

	// Can't retry anymore, fail the request
	{
		UE_DNLD_LOG(@"didCompleteWithError, task '%@' with taskIdentifier %lu failed due to [%@:%ld] '%@', has no retry data or no next url, failing request (bOutOfDiskSpaceError=%d)", Task.taskDescription, Task.taskIdentifier, Error.domain, Error.code, LocalizedDescription, bOutOfDiskSpaceError);

		NSURLResponse* GenericResponse = Task.response;
		NSInteger StatusCode = HTTPStatusCodeErrorServer;
		if ([GenericResponse isKindOfClass:[NSHTTPURLResponse class]])
		{
			NSHTTPURLResponse* Response = (NSHTTPURLResponse*)GenericResponse;
			if (Response.statusCode >= HTTPStatusCodeErrorBadRequest)
			{
				StatusCode = Response.statusCode;
			}
		}

		[self SetDownloadResult:StatusCode IsDiskSpaceError:bOutOfDiskSpaceError WithTempFile:nil ForDownload:Task];
	}
}

//- (void)URLSession:(NSURLSession*)Session taskIsWaitingForConnectivity:(NSURLSessionTask*)Task
//{
//	UE_DNLD_LOG(@"taskIsWaitingForConnectivity '%@'", Task.taskDescription);
//}

//- (void)URLSession:(NSURLSession*)Session task:(NSURLSessionTask*)Task willBeginDelayedRequest:(NSURLRequest*)Request completionHandler:(void (^)(NSURLSessionDelayedRequestDisposition Disposition, NSURLRequest* NewRequest))CompletionHandler
//{
//	UE_DNLD_LOG(@"willBeginDelayedRequest '%@' for '%@' with taskIdentifier %lu", Task.taskDescription, Request.debugDescription, Task.taskIdentifier);
//	CompletionHandler(NSURLSessionDelayedRequestContinueLoading, Request);
//}

- (void)URLSession:(NSURLSession*)Session downloadTask:(NSURLSessionDownloadTask*)Task didFinishDownloadingToURL:(NSURL*)Location
{
	// Should not be needed, but ensure this just in case
	[self EnsureTaskIsTracked:Task];

	FBackgroundNSURLSessionDownloadTaskData* TaskData = [FBackgroundNSURLSessionDownloadTaskData TaskDataFromSerializedString:Task.taskDescription];
	NSString* RequestID = TaskData != nil ? TaskData.RequestID : @"";

	NSString* DestinationPath = [self GetTempPathForURL:Task.originalRequest.URL WithRequestID:RequestID];

	[self SaveFileHashHelperState];

	// Try to remove existing file in case if we have a stale file.
	if ([[NSFileManager defaultManager] fileExistsAtPath:DestinationPath])
	{
		[[NSFileManager defaultManager] removeItemAtPath:DestinationPath error:nil];
	}

	// Check file size before attempting to move
	bool bResultSizeIsCorrect = false;
	if (TaskData != nil && TaskData.ExpectedResultSize > 0)
	{
		NSString* LocationPath = Location.path;

		NSError* AttributesError = nil;
		NSDictionary* Attributes = [[NSFileManager defaultManager] attributesOfItemAtPath:LocationPath error:&AttributesError];

		if (Attributes != nil && AttributesError == nil)
		{
			const UInt64 CurrentFileSize = [Attributes fileSize];
			bResultSizeIsCorrect = CurrentFileSize == TaskData.ExpectedResultSize;

			// Fail request if current size doesn't match expected size. 
			if (!bResultSizeIsCorrect)
			{
				[self SetDownloadResult:HTTPStatusCodeErrorServer IsDiskSpaceError:false WithTempFile:nil ForDownload:Task];

				UE_DNLD_LOG(@"didFinishDownloadingToURL task '%@' with taskIdentifier %lu file '%@' size %llu doesn't match expected file size of %llu", Task.taskDescription, Task.taskIdentifier, LocationPath, CurrentFileSize, TaskData.ExpectedResultSize);
				return;
			}
		}
		else
		{
			[self SetDownloadResult:HTTPStatusCodeErrorServer IsDiskSpaceError:false WithTempFile:nil ForDownload:Task];

			UE_DNLD_LOG(@"didFinishDownloadingToURL task '%@' with taskIdentifier %lu can't access file attributes of '%@' due to '%@'", Task.taskDescription, Task.taskIdentifier, LocationPath, (AttributesError != nil ? AttributesError.localizedDescription : @"error is nil"));
			return;
		}
	}

	NSError* Error = nil;
	[[NSFileManager defaultManager] moveItemAtURL:Location toURL:[NSURL fileURLWithPath:DestinationPath] error:&Error];

	// Update task progress just in case didWriteData was not invoked
	const uint64 TotalBytesWritten = [[[NSFileManager defaultManager] attributesOfItemAtPath:DestinationPath error:nil] fileSize];
	[self SetCurrentDownloadedBytes:TotalBytesWritten ForTask:Task];

	if (Error != nil)
	{
		[self SetDownloadResult:HTTPStatusCodeErrorServer IsDiskSpaceError:IsOutOfDiskSpaceError(Error) WithTempFile:nil ForDownload:Task];

		UE_DNLD_LOG(@"didFinishDownloadingToURL task '%@' with taskIdentifier %lu failed to move file to '%@' due to '%@', result size was correct %u", Task.taskDescription, Task.taskIdentifier, DestinationPath, Error.localizedDescription, bResultSizeIsCorrect ? 1 : 0);
	}
	else
	{
		[self SetDownloadResult:HTTPStatusCodeSuccessCreated IsDiskSpaceError:false WithTempFile:DestinationPath ForDownload:Task];

		UE_DNLD_LOG(@"didFinishDownloadingToURL task '%@' with taskIdentifier %lu move file to '%@', result size was correct %u, download finished", Task.taskDescription, Task.taskIdentifier, DestinationPath, bResultSizeIsCorrect ? 1 : 0);
	}
}

- (void)URLSession:(NSURLSession*)Session downloadTask:(NSURLSessionDownloadTask*)Task didWriteData:(int64_t)BytesWritten totalBytesWritten:(int64_t)TotalBytesWritten totalBytesExpectedToWrite:(int64_t)TotalBytesExpectedToWrite
{
	//UE_DNLD_LOG(@"didWriteData taskIdentifier %lu wrote %lli", Task.taskIdentifier, TotalBytesWritten);
	[self SetCurrentDownloadedBytes:TotalBytesWritten ForTask:Task];
}

//- (void)URLSession:(NSURLSession*)Session downloadTask:(NSURLSessionDownloadTask*)Task didResumeAtOffset:(int64_t)FileOffset expectedTotalBytes:(int64_t)ExpectedTotalBytes
//{
//	UE_DNLD_LOG(@"didResumeAtOffset taskIdentifier %lu offset %lli total %lli", Task.taskIdentifier, FileOffset, ExpectedTotalBytes);
//}

@end

const uint64 FBackgroundURLSessionHandler::InvalidDownloadId = [FBackgroundNSURLSession GetInvalidDownloadId];

FBackgroundURLSessionHandler::FOnDownloadCompletedExtended FBackgroundURLSessionHandler::OnDownloadCompletedExtended;
FBackgroundURLSessionHandler::FOnDownloadMetricsExtended FBackgroundURLSessionHandler::OnDownloadMetricsExtended;

FBackgroundURLSessionHandler::FOnDownloadsCompletedWhileAppWasNotRunning FBackgroundURLSessionHandler::OnDownloadsCompletedWhileAppWasNotRunning;

NSMutableDictionary<NSString*,void(^)()>* FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap = [[NSMutableDictionary alloc] init];

void FBackgroundURLSessionHandler::AllowCellular(bool bAllow)
{
	@autoreleasepool
	{
		const BOOL bCurrentValue = [FBackgroundNSURLSession Shared].AllowCellular;
		const BOOL bNewValue = bAllow ? YES : NO;
		if (bCurrentValue == bNewValue)
		{
			return;
		}

		[[FBackgroundNSURLSession Shared] setAllowCellular:bNewValue];
		[[FBackgroundNSURLSession Shared] RecreateDownloads];
	}
}

void FBackgroundURLSessionHandler::AllowConstrained(bool bAllow)
{
	@autoreleasepool
	{
		const BOOL bCurrentValue = [FBackgroundNSURLSession Shared].AllowConstrained;
		const BOOL bNewValue = bAllow ? YES : NO;
		if (bCurrentValue == bNewValue)
		{
			return;
		}

		[[FBackgroundNSURLSession Shared] setAllowConstrained:bNewValue];
		[[FBackgroundNSURLSession Shared] RecreateDownloads];
	}
}

uint64 FBackgroundURLSessionHandler::CreateOrFindDownload(const TArray<FString>& URLs, const float Priority, BackgroundHttpFileHashHelperRef HelperRef, const uint64 ExpectedResultSize, const FString& RequestID)
{
	@autoreleasepool
	{
		NSMutableArray* URLArray = [NSMutableArray arrayWithCapacity:URLs.Num()];
		for (const FString& URL: URLs)
		{
			[URLArray addObject:URL.GetNSString()];
		}

		[[FBackgroundNSURLSession Shared] SetFileHashHelper:HelperRef];
		return [[FBackgroundNSURLSession Shared] CreateOrFindDownloadForURLs:URLArray WithPriority:Priority WithExpectedResultSize:ExpectedResultSize WithRequestID:RequestID.GetNSString()];
	}
}

void FBackgroundURLSessionHandler::PauseDownload(const uint64 DownloadId)
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] PauseDownload:DownloadId];
	}
}

void FBackgroundURLSessionHandler::ResumeDownload(const uint64 DownloadId)
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] ResumeDownload:DownloadId];
	}
}

void FBackgroundURLSessionHandler::CancelDownload(const uint64 DownloadId)
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] CancelDownload:DownloadId];
	}
}

void FBackgroundURLSessionHandler::SetPriority(const uint64 DownloadId, const float Priority)
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] SetPriority:Priority ForDownload:DownloadId];
	}
}

uint64 FBackgroundURLSessionHandler::GetCurrentDownloadedBytes(const uint64 DownloadId)
{
	@autoreleasepool
	{
		return [[FBackgroundNSURLSession Shared] GetCurrentDownloadedBytes:DownloadId];
	}
}

void FBackgroundURLSessionHandler::GetActiveDownloadIDs(TSet<uint64>& OutActiveIDs)
{
	@autoreleasepool
	{
		OutActiveIDs.Reset();
		[[FBackgroundNSURLSession Shared] GetActiveDownloadIDs:OutActiveIDs];
	}
}

bool FBackgroundURLSessionHandler::IsDownloadFinished(const uint64 DownloadId, int32& OutResultHTTPCode, FString& OutTemporaryFilePath)
{
	@autoreleasepool
	{
		BOOL Status = NO;
		NSInteger StatusCode = 0;
		NSString* TempFile = [[FBackgroundNSURLSession Shared] GetDownloadResult:DownloadId OutStatus:&Status OutStatusCode:&StatusCode];
		if (!Status)
		{
			return false;
		}
		
		OutResultHTTPCode = (int32)StatusCode;
		if (TempFile != nil)
		{
			OutTemporaryFilePath = FString(TempFile);
			UE_DNLD_LOG(@"DownloadId %lli finished with status code %li and path '%@'", DownloadId, (long)StatusCode, TempFile);
		}
		else
		{
			UE_DNLD_LOG(@"DownloadId %lli finished with status code %li and no path", DownloadId, (long)StatusCode);
		}
		
		return true;
	}
}

void FBackgroundURLSessionHandler::HandleEventsForBackgroundURLSession(const FString& SessionIdentifier)
{
	@autoreleasepool
	{
		NSString* Identifier = SessionIdentifier.GetNSString();
		if (![[FBackgroundNSURLSession GetNSURLSessionIdentifier] isEqualToString:Identifier])
		{
			UE_DNLD_LOG(@"HandleEventsForBackgroundURLSession ignoring session identifier '%@'", Identifier);
			return;
		}
		
		UE_DNLD_LOG(@"HandleEventsForBackgroundURLSession will initializes session with identifier '%@'", Identifier);
		[FBackgroundNSURLSession Shared];
		// will invoke URLSessionDidFinishEventsForBackgroundURLSession internally.
	}
}

void FBackgroundURLSessionHandler::HandleDidEnterBackground()
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] HandleDidEnterBackground];
	}
}

void FBackgroundURLSessionHandler::HandleWillEnterForeground()
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] HandleWillEnterForeground];
	}
}

void FBackgroundURLSessionHandler::SaveBackgroundHttpFileHashHelperState()
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] SaveFileHashHelperState];
	}
}

TArray<TTuple<FString, uint32, FString>> FBackgroundURLSessionHandler::GetCDNAnalyticsData()
{
	TArray<TTuple<FString, uint32, FString>> Result;

	@autoreleasepool
	{
		NSArray<__kindof FBackgroundNSURLCDNInfo*>* CDNInfo = [[FBackgroundNSURLSession Shared] GetCDNInfo];

		for (FBackgroundNSURLCDNInfo* Info in CDNInfo)
		{
			FString StatusString;
			switch (Info.Response) {
				case EBackgroundNSURLCDNInfoResponse::Ok:
					StatusString = TEXT("OK");
					break;
				case EBackgroundNSURLCDNInfoResponse::Timeout:
					StatusString = TEXT("Timeout");
					break;
				case EBackgroundNSURLCDNInfoResponse::Error:
				default:
					StatusString = TEXT("Error");
					break;
			}
			Result.Emplace(TTuple<FString, uint32, FString>(Info.CDNAbsoluteURL, (uint32)(Info.ResponseTime * MSEC_PER_SEC), StatusString));
		}
	}

	return Result;
}

#if !UE_BUILD_SHIPPING
void FBackgroundURLSessionHandler::GetDownloadDebugText(const uint64 DownloadId, TArray<FString>& Output)
{
	@autoreleasepool
	{
		const FString DebugText([[FBackgroundNSURLSession Shared] GetDownloadDebugText:DownloadId]);
		Output.Add(DebugText);
	}
}
#endif

void FBackgroundURLSessionHandler::CleanupOldTempDownloads()
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString TempPath = FBackgroundHttpFileHashHelper::GetTemporaryRootPath();
	UE_LOGF(LogIOSBackgroundDownload, VeryVerbose, "CleanupOldTempDownloads: cleaning up %ls", *TempPath);
	if (PlatformFile.DirectoryExists(*TempPath)) 
	{
		const FString BranchName = BuildSettings::GetBranchName();
		UE_LOGF(LogIOSBackgroundDownload, VeryVerbose, "CleanupOldTempDownloads: keeping directory: %ls", *BranchName);
		PlatformFile.IterateDirectory(*TempPath, [&BranchName, &PlatformFile](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool {
			UE_LOGF(LogIOSBackgroundDownload, VeryVerbose, "CleanupOldTempDownloads: considering: %ls | %d", FilenameOrDirectory, (int)bIsDirectory);
			if (bIsDirectory)
			{
				const FString DirectoryName = FPaths::GetCleanFilename(FilenameOrDirectory);
				if (DirectoryName != TEXT("URLMap") && DirectoryName != BranchName)
				{
					UE_LOGF(LogIOSBackgroundDownload, VeryVerbose, "CleanupOldTempDownloads: deleting: %ls", FilenameOrDirectory);
					PlatformFile.DeleteDirectoryRecursively(FilenameOrDirectory);
				}
			}
			return true;
		});
	}
}

void FBackgroundURLSessionHandler::ClearPreExistingRequestsAtStartup(bool bEnable)
{
	bCancelExistingDownloads = bEnable;	
}

#if !UE_BUILD_SHIPPING
void BackgroundURLSessionHandler_SwitchToForegroundSession() // not public API at this point
{
	@autoreleasepool
	{
		[[FBackgroundNSURLSession Shared] SwitchToForegroundSession];
	}
}
#endif
