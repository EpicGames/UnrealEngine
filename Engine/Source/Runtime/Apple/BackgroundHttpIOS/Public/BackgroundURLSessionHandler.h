// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"

#include "BackgroundHttpFileHashHelper.h"
#include "BackgroundHttpRequestImpl.h"

//Included for FBackgroundHttpRequestMetricsExtended
#include "BackgroundHttpMetrics.h"

#import <Foundation/NSDictionary.h>
@class NSString;

//Interface for wrapping a NSURLSession configured to support background downloading of NSURLSessionDownloadTasks.
//This exists here as we can have to re-associate with our background session after app launch and need to re-associate with downloads
//right away before the HttpModule is loaded.
class BACKGROUNDHTTPIOS_API FBackgroundURLSessionHandler
{
public:

	// New API for background downloading

	// Value of invalid download id which could be used to compare return value of CreateOrFindDownload. 
	static const uint64 InvalidDownloadId;

	// Will be invoked from didFinishDownloadingToURL or didCompleteWithError.
	UE_DEPRECATED(5.8, "FBackgroundURLSessionHandler::FOnDownloadCompleted is deprecated.  Use FOnDownloadCompletedExtended")
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDownloadCompleted, const uint64 /*DownloadId*/, const bool /*bSuccess*/);
	UE_DEPRECATED(5.8, "FBackgroundURLSessionHandler::FOnDownloadCompleted is deprecated.  Use FOnDownloadCompletedExtended")
	static FOnDownloadCompleted OnDownloadCompleted;
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDownloadCompletedExtended, const uint64 /*DownloadId*/, EBackgroundHttpRequestCompletionResult /*Result*/);
	static FOnDownloadCompletedExtended OnDownloadCompletedExtended;
	
	// Will be invoked from didFinishCollectingMetrics
	UE_DEPRECATED(5.7, "FBackgroundURLSessionHandler::FOnDownloadMetrics is deprecated.  Use FOnDownloadMetricsExtended")
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnDownloadMetrics, const uint64 /*DownloadId*/, const int32 /*TotalBytesDownloaded*/, const float /*DownloadDuration*/)
	UE_DEPRECATED(5.7, "FBackgroundURLSessionHandler::OnDownloadMetrics is deprecated.  Use OnDownloadMetricsExtended")
	static FOnDownloadMetrics OnDownloadMetrics;

	// Will be invoked from didFinishCollectingMetrics
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDownloadMetricsExtended, const uint64 /*DownloadId*/, const FBackgroundHttpRequestMetricsExtended /*MetricsExtended*/);
	static FOnDownloadMetricsExtended OnDownloadMetricsExtended;

	// Will be invoked from handleEventsForBackgroundURLSession application delegate. Needs to be registered very early, e.g. from static constructor.
	// handleEventsForBackgroundURLSession is only invoked if app was killed by OS while in background and then relaunched to notify that downloads were completed.
	// Is not invoked in any other scenario.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDownloadsCompletedWhileAppWasNotRunning, const bool /*bSuccess*/);
	static FOnDownloadsCompletedWhileAppWasNotRunning OnDownloadsCompletedWhileAppWasNotRunning;

	// Will be used by IOSAppDelegate and ApplePlatformHttp
	static NSMutableDictionary<NSString*,void(^)()>* BackgroundSessionEventCompleteDelegateMap;

	// Sets if cellular network is allowed to be used for new downloads.
	// Existing downloads will be recreated to reflect new setting value.
	static void AllowCellular(bool bAllow);
	
	// Sets if constrained network is allowed to be used for new downloads.
	// Existing downloads will be recreated to reflect new setting value.
	static void AllowConstrained(bool bAllow);

	// Creates a new download or finds existing download matching URL path.
	// All URL's should have same path and only differ in domain.
	// Priority is a value between 0.0 to 1.0, see NSURLSessionTaskPriorityDefault.
	// HelperRef is optional shared reference to BackgroundHttpFileHashHelperRef.
	// In case of HandleEventsForBackgroundURLSession this subsystem will create it's own reference.
	static uint64 CreateOrFindDownload(const TArray<FString>& URLs, const float Priority, BackgroundHttpFileHashHelperRef HelperRef, const uint64 ExpectedResultSize, const FString& RequestID);
	UE_DEPRECATED(5.8, "FBackgroundURLSessionHandler::CreateOrFindDownload without RequestID is deprecated.")
	static uint64 CreateOrFindDownload(const TArray<FString>& URLs, const float Priority, BackgroundHttpFileHashHelperRef HelperRef, const uint64 ExpectedResultSize = 0) { return CreateOrFindDownload(URLs, Priority, HelperRef, ExpectedResultSize, {}); }

	static void PauseDownload(const uint64 DownloadId);

	static void ResumeDownload(const uint64 DownloadId);

	// Cancels and invalidates DownloadId.
	static void CancelDownload(const uint64 DownloadId);

	// Priority is a value between 0.0 to 1.0, see NSURLSessionTaskPriorityDefault.
	static void SetPriority(const uint64 DownloadId, const float Priority);

	static uint64 GetCurrentDownloadedBytes(const uint64 DownloadId);

	// Returns download IDs of tasks currently in NSURLSessionTaskStateRunning state
	static void GetActiveDownloadIDs(TSet<uint64>& OutActiveIDs);

	static bool IsDownloadFinished(const uint64 DownloadId, int32& OutResultHTTPCode, FString& OutTemporaryFilePath);

	// To be used by app delegate, call it from handleEventsForBackgroundURLSession.
	static void HandleEventsForBackgroundURLSession(const FString& SessionIdentifier);

	// To be used by app delegate, call it from applicationDidEnterBackground
	static void HandleDidEnterBackground();

	// To be used by app delegate, call it from applicationWillEnterForeground
	static void HandleWillEnterForeground();

	// To be used by ApplePlatformBackgroundHttpManager
	static void SaveBackgroundHttpFileHashHelperState();

	// Returns an ordered list of CDNs used to issue actual downloads, including AbosluteURL, ResponseTime (in ms) and Status
	// A list of URLS provided to CreateOrFindDownload might change order if CDNReorderingTimeout > 0 to ensure better success rate.
	// List is empty before first CreateOrFindDownload call.
	static TArray<TTuple<FString, uint32, FString>> GetCDNAnalyticsData();

#if !UE_BUILD_SHIPPING
	static void GetDownloadDebugText(const uint64 DownloadId, TArray<FString>& Output);
#endif
	
	// Cleans up temp downloads placed in any subdirectory other than the current build one's
	static void CleanupOldTempDownloads();
	
	// Should be called before any other methods to enable clearing any pre-existing downloads
	// when the system starts
	static void ClearPreExistingRequestsAtStartup(bool bEnable);
};
