// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatforms("Mac")]
public class MacMenu : ModuleRules
{
    public MacMenu(ReadOnlyTargetRules Target) : base(Target)
    {
        bRequiresPlatformSDK = true;

        PrivateIncludePaths.Add(Path.Combine(ModuleDirectory, "../../Slate/Private"));

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "InputCore",
                "Slate",
                "SlateCore"
            }
        );
        
        if (Target.bCompileAgainstApplicationCore)
        {
	        PublicDependencyModuleNames.Add("ApplicationCore");
        }

    }
}