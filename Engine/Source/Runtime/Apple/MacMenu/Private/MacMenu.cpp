// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacMenu.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Commands/Commands.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SViewport.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, MacMenu);

struct FMacMenuItemState
{
	TSharedPtr<const FMenuEntryBlock> Block;
	EMultiBlockType Type;
	NSString* Title;
	NSString* KeyEquivalent;
	uint32 KeyModifiers;
	NSImage* Icon;
	bool IsSubMenu;
	bool IsEnabled;
	uint32 State;

	FMacMenuItemState() : Type(EMultiBlockType::None), Title(nil), KeyEquivalent(nil), KeyModifiers(0), Icon(nil), IsSubMenu(false), IsEnabled(false), State(0) {}
	~FMacMenuItemState()
	{
		if (Title) [Title release];
		if (KeyEquivalent) [KeyEquivalent release];
		if (Icon) [Icon release];
	}
};

static TMap<FMacMenu*, TSharedPtr<TArray<FMacMenuItemState>>> GCachedMenuState;
static FCriticalSection GCachedMenuStateCS;

// The FMenuEntryBlock in FMacMenuItem will be deallocated when menu items
// are removed from the NSMenu in FMacMenu::UpdateWithMultiBox() via the Main Thread. But SWidgets should only be destroyed in the Game Thread.
// We save the FMenuEntryBlock ptrs in this queue and release them in the Game Thread
static TArray<TSharedPtr<const FMenuEntryBlock>> GMacMenuEntryBlockDeletionQueue;
static FCriticalSection GMacMenuEntryBlockCS;;

@interface FMacMenuItem : NSMenuItem
@property (assign) TSharedPtr<const FMenuEntryBlock> MenuEntryBlock;
- (void)performAction;
@end

@implementation FMacMenuItem

- (id)initWithMenuEntryBlock:(TSharedPtr< const FMenuEntryBlock >)Block
{
	self = [super initWithTitle:@"" action:nil keyEquivalent:@""];
	self.MenuEntryBlock = Block;
	return self;
}

- (void) dealloc
{
	{
		FScopeLock MacMenuEntryBlockLock(&GMacMenuEntryBlockCS);
		// we save the block to prevent it from being released in the main thread
		// releasing the block in Main Thread will trigger an assertion in FSlateApplicationBase::Get()
		GMacMenuEntryBlockDeletionQueue.Add(self.MenuEntryBlock);
	}
	[super dealloc];
}

- (void)performAction
{
	FCocoaMenu* CocoaMenu = [[self menu] isKindOfClass:[FCocoaMenu class]] ? (FCocoaMenu*)[self menu] : nil;
	if ( !CocoaMenu || ![CocoaMenu isHighlightingKeyEquivalent] )
	{
		FSlateMacMenu::ExecuteMenuItemAction(self.MenuEntryBlock.ToSharedRef());
		FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
	}
}

@end

@implementation FMacMenu

- (id)initWithMenuEntryBlock:(TSharedPtr< const FMenuEntryBlock >)Block
{
	self = [super initWithTitle:@""];
	[self setDelegate:self];
	self.MenuEntryBlock = Block;
	FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);
	GCachedMenuState.Add(self, TSharedPtr<TArray<FMacMenuItemState>>(new TArray<FMacMenuItemState>()));
	return self;
}

- (void)menuNeedsUpdate:(NSMenu*)Menu
{
	FSlateMacMenu::UpdateMenu(self);
}

- (void)menuWillOpen:(NSMenu*)Menu
{
	FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
	
	GameThreadCall(^{
		FSlateApplication::Get().ClearKeyboardFocus( EFocusCause::WindowActivate );
	}, false);

}

- (void)dealloc
{
	FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);
	GCachedMenuState.Remove(self);
	[super dealloc];
}

@end

// Register Mac Specific global menu with optional key bindings
#define LOCTEXT_NAMESPACE "MacMenu"
class FMacMenuCommands : public TCommands<FMacMenuCommands>
{
public:
	FMacMenuCommands() : TCommands<FMacMenuCommands>
	(
		TEXT("MacMenu"),
		NSLOCTEXT("Contexts", "MacMenu", "MacMenu"),
		NAME_None,
		"MacMenu"
	)
	{}
	
	virtual void RegisterCommands() override
	{
		UI_COMMAND( Preferences, "Preferences...", "Preferences", EUserInterfaceActionType::None, FInputChord( EModifierKey::Control, EKeys::Comma ) )
		UI_COMMAND( Help, "Help", "Help", EUserInterfaceActionType::None, FInputChord( EModifierKey::Shift, EKeys::F1 ) )
		UI_COMMAND( Hide, "Hide", "Hide", EUserInterfaceActionType::None, FInputChord( EModifierKey::Control, EKeys::H ) )
		UI_COMMAND( HideOthers, "Hide Others", "Hide Others", EUserInterfaceActionType::None, FInputChord( EModifierKey::Control | EModifierKey::Alt, EKeys::H ) )
		UI_COMMAND( ShowAll, "Show All", "Show All", EUserInterfaceActionType::None, FInputChord() )
		UI_COMMAND( Minimize, "Minimize", "Minimize", EUserInterfaceActionType::None, FInputChord( EModifierKey::Control, EKeys::M ) )
		UI_COMMAND( Close, "Close", "Close", EUserInterfaceActionType::None, FInputChord( FInputChord() ) )
		UI_COMMAND( About, "About", "About", EUserInterfaceActionType::None, FInputChord( FInputChord() ) )
		UI_COMMAND( Quit, "Quit", "Quit", EUserInterfaceActionType::None, FInputChord( EModifierKey::Control, EKeys::Q ) )		
		UI_COMMAND( Zoom, "Zoom", "Zoom", EUserInterfaceActionType::None, FInputChord() )	
		UI_COMMAND( ToggleFullScreen, "Enter Full Screen", "Enter Full Screen", EUserInterfaceActionType::None, FInputChord( EModifierKey::Control | EModifierKey::Command, EKeys::F ) )
		UI_COMMAND( BringAllToFront, "Bring All to Front", "Bring All to Front", EUserInterfaceActionType::None, FInputChord() )
	}
	
	TSharedPtr<FUICommandInfo> Preferences;
	TSharedPtr<FUICommandInfo> Help;
	TSharedPtr<FUICommandInfo> Hide;
	TSharedPtr<FUICommandInfo> HideOthers;
	TSharedPtr<FUICommandInfo> ShowAll;
	TSharedPtr<FUICommandInfo> Minimize;
	TSharedPtr<FUICommandInfo> Close;
	TSharedPtr<FUICommandInfo> About;
	TSharedPtr<FUICommandInfo> Quit;
	TSharedPtr<FUICommandInfo> Zoom;
	TSharedPtr<FUICommandInfo> ToggleFullScreen;
	TSharedPtr<FUICommandInfo> BringAllToFront;
};
#undef LOCTEXT_NAMESPACE

// Generic MacMenu helper functions
namespace MacMenuHelper
{
	// Create a runtime fixed ID per cmd we wish to lookup
	enum
	{
		CmdID_DontCare = 0,
		CmdID_Preferences,
		CmdID_Help,
		CmdID_Hide,
		CmdID_HideOthers,
		CmdID_ShowAll,
		CmdID_Minimize,
		CmdID_Close,
		CmdID_About,
		CmdID_Quit,
		CmdID_Zoom,
		CmdID_ToggleFullScreen,
		CmdID_BringAllToFront,
		CmdID_ServicesMenu
	};
	
	NSString* GetMenuItemKeyEquivalent(const TSharedRef<const FInputChord>& Chord, uint32* OutModifiers)
	{
		*OutModifiers = 0;
		NSString* Result = @"";
		
		if (Chord->IsValidChord())
		{
			if (Chord->NeedsControl())
			{
				*OutModifiers |= NSEventModifierFlagControl;
			}
			if (Chord->NeedsShift())
			{
				*OutModifiers |= NSEventModifierFlagShift;
			}
			if (Chord->NeedsAlt())
			{
				*OutModifiers |= NSEventModifierFlagOption;
			}
			if (Chord->NeedsCommand())
			{
				*OutModifiers |= NSEventModifierFlagCommand;
			}
			
			// Compute Unicode character for Key
			const uint32* KeyCodePtr = nullptr;
			const uint32* CharCodePtr = nullptr;
			FInputKeyManager::Get().GetCodesFromKey(Chord->Key, KeyCodePtr, CharCodePtr);
			
			unichar UniChar = 0;
			if(CharCodePtr != nullptr)
			{
				UniChar = (unichar)(*CharCodePtr);
			}
			else if(KeyCodePtr != nullptr)
			{
				UniChar = FMacApplication::TranslateKeyCodeToUniCode(*KeyCodePtr, *OutModifiers);
			}
			
			if(UniChar != 0)
			{
				Result = [[[NSString alloc] initWithCharacters:&UniChar length:1] autorelease];
			}
			else
			{
				FString KeyString = Chord->GetKeyText().ToString().ToLower();
				Result = KeyString.GetNSString();
			}
		}
		
		return Result;
	}
	
	NSString* GetMenuItemKeyEquivalent(const TSharedRef<const class FMenuEntryBlock>& Block, uint32* OutModifiers)
	{
		if (Block->GetAction().IsValid())
		{
			return GetMenuItemKeyEquivalent(Block->GetAction()->GetFirstValidChord(), OutModifiers);;
		}

		*OutModifiers = 0;
		return @"";
	}
	
	NSMenuItem* CreateNSMenuItemForCommand(TSharedPtr<FUICommandInfo> const& Command, SEL ActionSelector, uint32 TagID = CmdID_DontCare, NSString* OptionalContextStr = nil)
	{
		check(ActionSelector);
		check(Command.IsValid());
		
		NSString* MenuItemTitle = Command->GetLabel().ToString().GetNSString();
		if(OptionalContextStr != nil && OptionalContextStr.length)
		{
			MenuItemTitle = [[[NSString alloc] initWithFormat:@"%@ %@", MenuItemTitle, OptionalContextStr] autorelease];
		}

		uint32 KeyModifier = 0;
		NSString* KeyEquivalentString = GetMenuItemKeyEquivalent(Command->GetFirstValidChord(), &KeyModifier);
		
		NSMenuItem* MenuItem = [[[NSMenuItem alloc] initWithTitle:MenuItemTitle action:ActionSelector keyEquivalent:KeyEquivalentString] autorelease];
		[MenuItem setKeyEquivalentModifierMask:KeyModifier];
		[MenuItem setTag:TagID];

		return MenuItem;
	}
	
	void UpdateNSMenuItemTitle(NSMenuItem* MenuItem, TSharedPtr<FUICommandInfo> const& Command, NSString* OptionalContextStr = nil)
	{
		check(Command.IsValid());
		
		NSString* MenuItemTitle = Command->GetLabel().ToString().GetNSString();
		if(OptionalContextStr != nil && OptionalContextStr.length)
		{
			MenuItemTitle = [[[NSString alloc] initWithFormat:@"%@ %@", MenuItemTitle, OptionalContextStr] autorelease];
		}
		
		[MenuItem setTitle:MenuItemTitle];
	}
	
	NSString* ComputeAppName()
	{
        if (GIsEditor)
        {
            return NSLOCTEXT("UnrealEditor", "ApplicationTitle", "Unreal Editor").ToString().GetNSString();
        }
        
        FText ProjectTitle;
        GConfig->GetText(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("ProjectDisplayedTitle"), ProjectTitle, GGameIni);
        if (!ProjectTitle.IsEmpty())
        {
            return ProjectTitle.ToString().GetNSString();
        }
        
        return FString(FApp::GetProjectName()).GetNSString();
	}
	
	bool GMacPostInitStartupRequested = false;
	bool GMacPostInitStartUpComplete = false;

	void ConditionalDelayPostStartup()
	{
		// In case an obsecure app startup sequence has not managed to finish the menu startup correctly
		// However only do this if the post init has been called
		if(!MacMenuHelper::GMacPostInitStartUpComplete && MacMenuHelper::GMacPostInitStartupRequested)
		{
			FSlateMacMenu::PostInitStartup();
		}
	}
};

// Bind all low-level Application hooks that require to access this high-level MacMenu system which includes NSApp Menu's and slate menus
class FStartupApplicationToMacMenuBinder
{
public:
	FStartupApplicationToMacMenuBinder()
	{
		FMacPlatformApplicationMisc::UpdateCachedMacMenuStateCb = FSlateMacMenu::UpdateCachedState;
		FMacPlatformApplicationMisc::PostInitMacMenuStartupCb = FSlateMacMenu::PostInitStartup;
		FMacPlatformApplicationMisc::UpdateApplicationMenuCb = FSlateMacMenu::UpdateApplicationMenu;
		FMacPlatformApplicationMisc::UpdateWindowMenuCb = FSlateMacMenu::UpdateWindowMenu;
		FMacPlatformApplicationMisc::LanguageChangedCb = FSlateMacMenu::LanguageChanged;
		FMacApplication::MenuBarShutdownFunc = &FSlateMacMenu::CleanupOnShutdown;
	}
};
static FStartupApplicationToMacMenuBinder StaticInitializer;


void FSlateMacMenu::CleanupOnShutdown()
{
	{
		FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);
		GCachedMenuState.Reset();
	}
	{
		GameThreadCall(^{
			FScopeLock MacMenuEntryBlockLock(&GMacMenuEntryBlockCS);
			for(auto& Block : GMacMenuEntryBlockDeletionQueue)
			{
				Block.Reset();
			}
			GMacMenuEntryBlockDeletionQueue.Empty();
		});
	}
}

void FSlateMacMenu::PostInitStartup()
{
	MacMenuHelper::GMacPostInitStartupRequested = true;
	
	// Setup the app menu in menu bar
	const bool bIsBundledApp = [[[NSBundle mainBundle] bundlePath] hasSuffix:@".app"];
	if (!MacMenuHelper::GMacPostInitStartUpComplete && bIsBundledApp && MacApplication)
	{
		MacMenuHelper::GMacPostInitStartUpComplete = true;
		
		// Setup our Mac Specific commands
		FMacMenuCommands::Register();
	
		// Build Default menu's
        MainThreadCall(^{
            FCocoaMenu* MenuBar = [[FCocoaMenu new] autorelease];
            FCocoaMenu* AppMenu = [[FCocoaMenu new] autorelease];
            NSMenuItem* AppMenuItem = [[NSMenuItem new] autorelease];
            [AppMenuItem setTitle:@"AppMenuItem"];
            [MenuBar addItem:AppMenuItem];
            [AppMenuItem setSubmenu:AppMenu];
            [NSApp setMainMenu:MenuBar];

            NSString* AppName = MacMenuHelper::ComputeAppName();
            
            NSMenu* MainMenu = [NSApp mainMenu];
        
            NSMenuItem* PreferencesItem = GIsEditor ? MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Preferences, @selector(showPreferencesWindow:), MacMenuHelper::CmdID_Preferences) : nil;
            NSMenuItem* HideItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Hide, @selector(hide:), MacMenuHelper::CmdID_Hide, AppName);
            NSMenuItem* HideOthersItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().HideOthers, @selector(hideOtherApplications:), MacMenuHelper::CmdID_HideOthers);
            NSMenuItem* ShowAllItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().ShowAll, @selector(unhideAllApplications:), MacMenuHelper::CmdID_ShowAll);

            SEL ShowAboutSelector = [[NSApp delegate] respondsToSelector:@selector(showAboutWindow:)] ? @selector(showAboutWindow:) : @selector(orderFrontStandardAboutPanel:);
            NSMenuItem* AboutItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().About, ShowAboutSelector, MacMenuHelper::CmdID_About, AppName);
            
            SEL RequestQuitSelector = [[NSApp delegate] respondsToSelector:@selector(requestQuit:)] ? @selector(requestQuit:) : @selector(terminate:);
            NSMenuItem* QuitItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Quit, RequestQuitSelector, MacMenuHelper::CmdID_Quit, AppName);
            
            NSMenuItem* ServicesItem = [[NSMenuItem new] autorelease];
            FCocoaMenu* ServicesMenu = [[FCocoaMenu new] autorelease];
            [ServicesItem setTitle:NSLOCTEXT("MainMenu","ServicesMenu","Services").ToString().GetNSString()];
            [ServicesItem setSubmenu:ServicesMenu];
            [ServicesItem setTag:MacMenuHelper::CmdID_ServicesMenu];
            [NSApp setServicesMenu:ServicesMenu];
            [AppMenu addItem:AboutItem];
            [AppMenu addItem:[NSMenuItem separatorItem]];
            if (PreferencesItem)
            {
                [AppMenu addItem:PreferencesItem];
                [AppMenu addItem:[NSMenuItem separatorItem]];
            }
            [AppMenu addItem:ServicesItem];
            [AppMenu addItem:[NSMenuItem separatorItem]];
            [AppMenu addItem:HideItem];
            [AppMenu addItem:HideOthersItem];
            [AppMenu addItem:ShowAllItem];
            [AppMenu addItem:[NSMenuItem separatorItem]];
            [AppMenu addItem:QuitItem];

            if (FApp::IsGame())
            {
                NSMenu* ViewMenu = [[FCocoaMenu new] autorelease];
                [ViewMenu setTitle:NSLOCTEXT("MainMenu","ViewMenu","View").ToString().GetNSString()];
                NSMenuItem* ViewMenuItem = [[NSMenuItem new] autorelease];
                [ViewMenuItem setSubmenu:ViewMenu];
                [[NSApp mainMenu] addItem:ViewMenuItem];

                NSMenuItem* ToggleFullscreenItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().ToggleFullScreen, @selector(toggleFullScreen:), MacMenuHelper::CmdID_ToggleFullScreen);
                [ViewMenu addItem:ToggleFullscreenItem];
            }
            
            NSMenu* WindowMenu = [NSApp windowsMenu];
            if (!WindowMenu)
            {
                WindowMenu = [[FCocoaMenu new] autorelease];
                [WindowMenu setTitle:NSLOCTEXT("MainMenu","WindowMenu","Window").ToString().GetNSString()];
                NSMenuItem* WindowMenuItem = [[NSMenuItem new] autorelease];
                [WindowMenuItem setSubmenu:WindowMenu];
                [[NSApp mainMenu] addItem:WindowMenuItem];
                [NSApp setWindowsMenu:WindowMenu];
            }
            
            NSMenuItem* MinimizeItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Minimize, @selector(miniaturize:), MacMenuHelper::CmdID_Minimize);
            NSMenuItem* ZoomItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Zoom, @selector(zoom:), MacMenuHelper::CmdID_Zoom);
            NSMenuItem* CloseItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Close, @selector(performClose:), MacMenuHelper::CmdID_Close);
            NSMenuItem* BringAllToFrontItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().BringAllToFront, @selector(arrangeInFront:), MacMenuHelper::CmdID_BringAllToFront);
            [WindowMenu addItem:MinimizeItem];
            [WindowMenu addItem:ZoomItem];
            [WindowMenu addItem:CloseItem];
            [WindowMenu addItem:[NSMenuItem separatorItem]];
            [WindowMenu addItem:BringAllToFrontItem];
            [WindowMenu addItem:[NSMenuItem separatorItem]];
        }, false);
	}
}

void FSlateMacMenu::LanguageChanged()
{
    if (!MacMenuHelper::GMacPostInitStartupRequested)
    {
        // if MacMenu not even done init setup yet, ignore LanguageChanged call
        return;
    }
    
	NSMenu* MainMenu = [NSApp mainMenu];
	NSMenuItem* AppMenuItem = [MainMenu itemWithTitle:@"AppMenuItem"];
	NSMenu* AppMenu = [AppMenuItem submenu];

	NSMenuItem* AboutItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_About];
	NSMenuItem* PreferencesItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_Preferences];
	NSMenuItem* HideItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_Hide];
	NSMenuItem* HideOthersItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_HideOthers];
	NSMenuItem* ShowAllItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_ShowAll];
	NSMenuItem* QuitItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_Quit];

	NSString* AppName = MacMenuHelper::ComputeAppName();
	MacMenuHelper::UpdateNSMenuItemTitle(AboutItem, FMacMenuCommands::Get().About, AppName);
	MacMenuHelper::UpdateNSMenuItemTitle(HideItem, FMacMenuCommands::Get().Hide, AppName);
	MacMenuHelper::UpdateNSMenuItemTitle(HideOthersItem, FMacMenuCommands::Get().HideOthers);
	MacMenuHelper::UpdateNSMenuItemTitle(ShowAllItem, FMacMenuCommands::Get().ShowAll);
	MacMenuHelper::UpdateNSMenuItemTitle(QuitItem, FMacMenuCommands::Get().Quit, AppName);
	MacMenuHelper::UpdateNSMenuItemTitle(PreferencesItem, FMacMenuCommands::Get().Preferences);
	
	NSMenuItem* Services = [AppMenu itemWithTag:MacMenuHelper::CmdID_ServicesMenu];
	[Services setTitle:NSLOCTEXT("MainMenu","ServicesMenu","Services").ToString().GetNSString()];
}

void FSlateMacMenu::UpdateApplicationMenu(bool bMacApplicationModalMode)
{
	MacMenuHelper::ConditionalDelayPostStartup();
	
    NSMenu* MainMenu = [NSApp mainMenu];
    NSMenuItem* AppMenuItem = [MainMenu itemWithTitle:@"AppMenuItem"];
    NSMenu* AppMenu = [AppMenuItem submenu];

    NSMenuItem* AboutItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_About];
    NSMenuItem* PreferencesItem = GIsEditor ? [AppMenu itemWithTag:MacMenuHelper::CmdID_Preferences] : nil;
    NSMenuItem* HideItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_Hide];
    NSMenuItem* HideOthersItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_HideOthers];
    NSMenuItem* ShowAllItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_ShowAll];
    NSMenuItem* QuitItem = [AppMenu itemWithTag:MacMenuHelper::CmdID_Quit];

    if(!bMacApplicationModalMode)
    {
    	SEL ShowAboutSelector = [[NSApp delegate] respondsToSelector:@selector(showAboutWindow:)] ? @selector(showAboutWindow:) : @selector(orderFrontStandardAboutPanel:);
        [AboutItem setAction:ShowAboutSelector];
        
        [PreferencesItem setAction:@selector(showPreferencesWindow:)];
        [HideItem setAction:@selector(hide:)];
        [HideOthersItem setAction:@selector(hideOtherApplications:)];
        [ShowAllItem setAction:@selector(unhideAllApplications:)];
        SEL RequestQuitSelector = [[NSApp delegate] respondsToSelector:@selector(requestQuit:)] ? @selector(requestQuit:) : @selector(terminate:);
        [QuitItem setAction:RequestQuitSelector];
    }
    else
    {
        for (NSMenuItem* Item in [AppMenu itemArray])
        {
            if(![Item hasSubmenu])
            {
            	[Item setAction:nil];
            }
        }
    }
    
    [AppMenu update];
    [MainMenu update];
}

void FSlateMacMenu::UpdateWindowMenu(bool bMacApplicationModalMode)
{
	NSMenu* WindowMenu = [NSApp windowsMenu];
	
	NSMenuItem* MinimizeItem = [WindowMenu itemWithTag:MacMenuHelper::CmdID_Minimize];
	NSMenuItem* ZoomItem = [WindowMenu itemWithTag:MacMenuHelper::CmdID_Zoom];
	NSMenuItem* CloseItem = [WindowMenu itemWithTag:MacMenuHelper::CmdID_Close];
	NSMenuItem* BringAllToFrontItem = [WindowMenu itemWithTag:MacMenuHelper::CmdID_BringAllToFront];
    
    if(!bMacApplicationModalMode)
    {
        [MinimizeItem setAction:@selector(miniaturize:)];
        [ZoomItem setAction:@selector(zoom:)];
        [CloseItem setAction:@selector(performClose:)];
        [BringAllToFrontItem setAction:@selector(arrangeInFront:)];
    }
    else
    {
        for (NSMenuItem* Item in [WindowMenu itemArray])
        {
            if(![Item hasSubmenu])
            {
                [Item setAction:nil];
            }
        }
    }
    
    [WindowMenu update];
    [[NSApp mainMenu] update];
}

void FSlateMacMenu::UpdateWithMultiBox(const TSharedPtr< FMultiBox > MultiBox)
{
	MacMenuHelper::ConditionalDelayPostStartup();
	
	// The dispatch block can't handle TSharedPtr correctly, so we use a small trick to pass MultiBox safely
	struct FSafeMultiBoxPass
	{
		TSharedPtr<FMultiBox> MultiBox;
	};
	FSafeMultiBoxPass* SafeMultiBoxPtr = new FSafeMultiBoxPass;
	SafeMultiBoxPtr->MultiBox = MultiBox;

	FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);
	GCachedMenuState.Reset();

	MainThreadCall(^{
		int32 NumItems = [[NSApp mainMenu] numberOfItems];
		FText WindowLabel = NSLOCTEXT("MainMenu", "WindowMenu", "Window");
		for (int32 Index = NumItems - 1; Index > 0; Index--)
		{
			[[NSApp mainMenu] removeItemAtIndex:Index];
		}

		if( SafeMultiBoxPtr->MultiBox.IsValid() )
		{
			const TArray<TSharedRef<const FMultiBlock> >& MenuBlocks = SafeMultiBoxPtr->MultiBox->GetBlocks();

			for (int32 Index = 0; Index < MenuBlocks.Num(); Index++)
			{
				TSharedRef<const FMenuEntryBlock> Block = StaticCastSharedRef<const FMenuEntryBlock>(MenuBlocks[Index]);
				FMacMenu* Menu = [[[FMacMenu alloc] initWithMenuEntryBlock:Block] autorelease];
				NSString* Title = FSlateMacMenu::GetMenuItemTitle(Block);
				[Menu setTitle:Title];

				NSMenuItem* MenuItem = [[NSMenuItem new] autorelease];
				[MenuItem setTitle:Title];
				[[NSApp mainMenu] addItem:MenuItem];
				[MenuItem setSubmenu:Menu];

				const bool bIsWindowMenu = (WindowLabel.ToString().Compare(FString(Title)) == 0);
				if (bIsWindowMenu)
				{
					[NSApp setWindowsMenu:nil];

					[Menu removeAllItems];

					NSMenuItem* MinimizeItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Minimize, @selector(miniaturize:), MacMenuHelper::CmdID_Minimize);
					NSMenuItem* ZoomItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Zoom, @selector(zoom:), MacMenuHelper::CmdID_Zoom);
					NSMenuItem* CloseItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().Close, @selector(performClose:), MacMenuHelper::CmdID_Close);
					NSMenuItem* BringAllToFrontItem = MacMenuHelper::CreateNSMenuItemForCommand(FMacMenuCommands::Get().BringAllToFront, @selector(arrangeInFront:), MacMenuHelper::CmdID_BringAllToFront);

					[Menu addItem:MinimizeItem];
					[Menu addItem:ZoomItem];
					[Menu addItem:CloseItem];
					[Menu addItem:[NSMenuItem separatorItem]];
					[Menu addItem:BringAllToFrontItem];
					[Menu addItem:[NSMenuItem separatorItem]];

					[NSApp setWindowsMenu:Menu];
					[Menu addItem:[NSMenuItem separatorItem]];
				}
			}
		}

		delete SafeMultiBoxPtr;

		FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
	}, false);
	// Now we destroy all the menu blocks in the game thread to ensure ~SWidget() is only called in Game Thread
	GameThreadCall(^{
		FScopeLock MacMenuEntryBlockLock(&GMacMenuEntryBlockCS);
		for(auto& Block : GMacMenuEntryBlockDeletionQueue)
		{
			Block.Reset();
		}
		GMacMenuEntryBlockDeletionQueue.Empty();
	}, false);
}

void FSlateMacMenu::UpdateMenu(FMacMenu* Menu)
{
	MainThreadCall(^{
		FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);

		FText WindowLabel = NSLOCTEXT("MainMenu", "WindowMenu", "Window");
		const bool bIsWindowMenu = (WindowLabel.ToString().Compare(FString([Menu title])) == 0);
		int32 ItemIndexOffset = 0;
		if (bIsWindowMenu)
		{
			int32 SeparatorIndex = 0;
			for (NSMenuItem* Item in [Menu itemArray])
			{
				SeparatorIndex += [Item isSeparatorItem] ? 1 : 0;
				ItemIndexOffset++;
				if (SeparatorIndex == 3)
				{
					break;
				}
			}
		}

		TSharedPtr<TArray<FMacMenuItemState>> MenuState = GCachedMenuState.FindRef(Menu);
		if(MenuState.IsValid())
		{
			int32 ItemIndexAdjust = 0;
			for (int32 Index = 0; Index < MenuState->Num(); Index++)
			{
				FMacMenuItemState& MenuItemState = (*MenuState)[Index];
				const int32 ItemIndex = (bIsWindowMenu ? Index + ItemIndexOffset : Index) - ItemIndexAdjust;
				NSMenuItem* MenuItem = [Menu numberOfItems] > ItemIndex ? [Menu itemAtIndex:ItemIndex] : nil;

				if (MenuItemState.Type == EMultiBlockType::MenuEntry)
				{
					if (MenuItem && (![MenuItem isKindOfClass:[FMacMenuItem class]] || (MenuItemState.IsSubMenu && [MenuItem submenu] == nil) || (!MenuItemState.IsSubMenu && [MenuItem submenu] != nil)))
					{
						[Menu removeItem:MenuItem];
						MenuItem = nil;
					}
					if (!MenuItem)
					{
						MenuItem = [[[FMacMenuItem alloc] initWithMenuEntryBlock:MenuItemState.Block] autorelease];

						if (MenuItemState.IsSubMenu)
						{
							FMacMenu* SubMenu = [[[FMacMenu alloc] initWithMenuEntryBlock:MenuItemState.Block] autorelease];
							[MenuItem setSubmenu:SubMenu];
						}

						if ([Menu numberOfItems] > ItemIndex)
						{
							[Menu insertItem:MenuItem atIndex:ItemIndex];
						}
						else
						{
							[Menu addItem:MenuItem];
						}
					}

					[MenuItem setTitle:MenuItemState.Title];

					[MenuItem setKeyEquivalent:MenuItemState.KeyEquivalent];
					[MenuItem setKeyEquivalentModifierMask:MenuItemState.KeyModifiers];

					if (bIsWindowMenu)
					{
						NSImage* MenuImage = MenuItemState.Icon;
						if(MenuImage)
						{
							[MenuItem setImage:MenuImage];
						}
					}
					else
					{
						[MenuItem setImage:nil];
					}

					[MenuItem setTarget:MenuItem];
					if(!MenuItemState.IsSubMenu)
					{
					   if(MenuItemState.IsEnabled)
						{
							[MenuItem setAction:@selector(performAction)];
						}
						else
						{
							[MenuItem setAction:nil];
						}
					}
					
					if (!MenuItemState.IsSubMenu)
					{
						[MenuItem setState:MenuItemState.State];
					}
				}
				else if (MenuItemState.Type == EMultiBlockType::Separator)
				{
					if (MenuItem && ![MenuItem isSeparatorItem])
					{
						[Menu removeItem:MenuItem];
					}
					else if (!MenuItem)
					{
						if ([Menu numberOfItems] > ItemIndex)
						{
							[Menu insertItem:[NSMenuItem separatorItem] atIndex:ItemIndex];
						}
						else
						{
							[Menu addItem:[NSMenuItem separatorItem]];
						}
					}
				}
				else
				{
					// If it's a type we skip, update ItemIndexAdjust so we can properly calculate item's index in NSMenu
					ItemIndexAdjust++;
				}
			}
		}
	});
}

void FSlateMacMenu::UpdateCachedState()
{
	bool bShouldUpdate = false;

	// @todo: Ideally this would ask global tab manager if there's any active tab, but that cannot be done reliably at the moment
	// so instead we assume that as long as there's any visible, regular window open, we do have some menu to show/update.
	if(!GIsSlowTask)
	{
		MacApplication->GetWindowsArrayMutex().Lock();
		const TArray<TSharedRef<FMacWindow>>&AllWindows = MacApplication->GetAllWindows();
		for (auto Window : AllWindows)
		{
			if (Window->IsRegularWindow() && Window->IsVisible())
			{
				bShouldUpdate = true;
				break;
			}
		}
		MacApplication->GetWindowsArrayMutex().Unlock();
    }
    
    // If PIE Viewport has focus don't update
    if(GIsEditor && FSlateApplication::IsInitialized())
    {
    	TSharedPtr<SViewport> ViewPort = FSlateApplication::Get().GetGameViewport();
		if(ViewPort.IsValid())
		{
			if(ViewPort->HasKeyboardFocus())
			{
				bShouldUpdate = false;
			}
		}
	}

	if (bShouldUpdate)
	{
		TMap<TSharedPtr<const FMenuEntryBlock>, TSharedRef<SWidget>> InvalidMenuEntryBlocks;

		//Step 1: Gather list of null widgets, with GCachedMenuStateCS locked.
		{
			FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);

			//Ensure we will alloc only a single time, even if most of the time it will be bigger than necessary (but always quite small anyway).
			InvalidMenuEntryBlocks.Reserve(GCachedMenuState.Num());

			for (TMap<FMacMenu*, TSharedPtr<TArray<FMacMenuItemState>>>::TIterator It(GCachedMenuState); It; ++It)
			{
				FMacMenu* Menu = It.Key();

				if (!Menu.MultiBox.IsValid())
				{
					TSharedPtr<const FMenuEntryBlock> MenuEntryBlock = Menu.MenuEntryBlock.Pin();
					if (MenuEntryBlock.IsValid())
					{
						InvalidMenuEntryBlocks.Add(MenuEntryBlock, SNullWidget::NullWidget);
					}
				}
			}
		}
		
		//Step 2: Create the widgets, which can be a complex process (like when creating a new window that will try to perform a call on the main thread, that may already have locked GCachedMenuStateCS).
		//So we ensure that GCachedMenuStateCS is not locked for this step.
		for(TPair<TSharedPtr<const FMenuEntryBlock>, TSharedRef<SWidget>>& It : InvalidMenuEntryBlocks)
		{
			TSharedPtr<const FMenuEntryBlock> MenuEntryBlock = It.Key;
			TSharedRef<SWidget> Widget = SNullWidget::NullWidget;

			if (MenuEntryBlock->MenuBuilder.IsBound())
			{
				Widget = MenuEntryBlock->MenuBuilder.Execute();
			}
			else
			{
				const bool bShouldCloseWindowAfterMenuSelection = true;
				FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, MenuEntryBlock->GetActionList(), MenuEntryBlock->Extender);
				{
					// Have the menu fill its contents
					MenuEntryBlock->EntryBuilder.ExecuteIfBound(MenuBuilder);
				}
 
				// Use INT_MAX for MaxHeight to so that SMultiBoxWidget is returned by MakeWidget instead of SVerticalBox
				const int32 MaxHeight = INT_MAX;
				Widget = MenuBuilder.MakeWidget(/*InMakeMultiBoxBuilderOverride=*/ nullptr, MaxHeight);
			}
			
			It.Value = Widget;
		}

		//Step 3: assign the widgets and setup the menu state.
		{
			FScopeLock CachedMenuStateLock(&GCachedMenuStateCS);

			for (TMap<FMacMenu*, TSharedPtr<TArray<FMacMenuItemState>>>::TIterator It(GCachedMenuState); It; ++It)
			{
				FMacMenu* Menu = It.Key();
				TSharedRef<SWidget> Widget = SNullWidget::NullWidget;
				
				if (!Menu.MultiBox.IsValid())
				{
					TSharedPtr<const FMenuEntryBlock> MenuEntryBlock = Menu.MenuEntryBlock.Pin();
					TSharedRef<SWidget>* Value = InvalidMenuEntryBlocks.Find(MenuEntryBlock);
					if (Value!= nullptr)
					{
						Widget = *Value;
						if (Widget->GetType() == FName("SMultiBoxWidget"))
						{
							Menu.MultiBox = TSharedPtr<const FMultiBox>(StaticCastSharedRef<SMultiBoxWidget>(Widget)->GetMultiBox());
						}
						else
						{
							const FName ActionName = MenuEntryBlock->GetAction().IsValid() ? MenuEntryBlock->GetAction()->GetCommandName() : NAME_None;
							UE_LOGF(LogMac, Warning, "Unsupported type of menu widget in FSlateMacMenu::UpdateCachedState(): %ls, %ls, %ls",
								   *Widget->GetType().ToString(), *MenuEntryBlock->GetExtensionHook().ToString(), *ActionName.ToString());
						}
					}
				}

				if (Menu.MultiBox.IsValid())
				{
					TSharedPtr<TArray<FMacMenuItemState>> MenuState = It.Value();
					const TArray<TSharedRef<const FMultiBlock>>& MenuBlocks = Menu.MultiBox.Pin()->GetBlocks();
					MenuState->AddDefaulted(FMath::Max(0, MenuBlocks.Num() - MenuState->Num()));

					for (int32 Index = 0; Index < MenuBlocks.Num(); Index++)
					{
						FMacMenuItemState& ItemState = (*MenuState)[Index];
						ItemState.Type = MenuBlocks[Index]->GetType();

						if (ItemState.Type == EMultiBlockType::MenuEntry)
						{
							TSharedRef<const FMenuEntryBlock> Block = StaticCastSharedRef<const FMenuEntryBlock>(MenuBlocks[Index]);
							ItemState.Block = Block;
							ItemState.Title = [FSlateMacMenu::GetMenuItemTitle(Block) retain];
							ItemState.KeyEquivalent = [MacMenuHelper::GetMenuItemKeyEquivalent(Block, &ItemState.KeyModifiers) retain];
							if (!ItemState.Icon)
							{
								SCOPED_AUTORELEASE_POOL;
								ItemState.Icon = [FSlateMacMenu::GetMenuItemIcon(Block) retain];
							}
							ItemState.IsSubMenu = Block->bIsSubMenu;
							ItemState.IsEnabled = FSlateMacMenu::IsMenuItemEnabled(Block);
							ItemState.State = ItemState.IsSubMenu ? 0 : FSlateMacMenu::GetMenuItemState(Block);
						}
					}
				}
			}
		}
	}
}

void FSlateMacMenu::ExecuteMenuItemAction(const TSharedRef< const class FMenuEntryBlock >& Block)
{
    TSharedPtr< const class FMenuEntryBlock>* MenuBlock = new TSharedPtr< const class FMenuEntryBlock>(Block);
	if (!FPlatformApplicationMisc::bMacApplicationModalMode)
	{
		GameThreadCall(^{
			TSharedPtr< const FUICommandList > ActionList = (*MenuBlock)->GetActionList();
			if (ActionList.IsValid() && (*MenuBlock)->GetAction().IsValid())
			{
				ActionList->ExecuteAction((*MenuBlock)->GetAction().ToSharedRef());
			}
			else
			{
				// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
				(*MenuBlock)->GetDirectActions().Execute();
			}
			delete MenuBlock;
		}, false);
	}
	else
	{
		delete MenuBlock;
	}
}

static const TSharedRef<SWidget> FindTextBlockWidget(TSharedRef<SWidget> Content)
{
	if (Content->GetType() == FName("STextBlock"))
	{
		return Content;
	}

	FChildren* Children = Content->GetChildren();
	int32 NumChildren = Children->Num();

	for (int32 Index=0; Index < NumChildren; ++Index)
	{
		const TSharedRef<SWidget> Found = FindTextBlockWidget(Children->GetChildAt(Index));
		if (Found != SNullWidget::NullWidget)
		{
			return Found;
		}
	}
	return SNullWidget::NullWidget;
}

NSString* FSlateMacMenu::GetMenuItemTitle(const TSharedRef<const FMenuEntryBlock>& Block)
{
	TAttribute<FText> Label;
	if (!Block->LabelOverride.IsBound() && Block->LabelOverride.Get().IsEmpty() && Block->GetAction().IsValid())
	{
		Label = Block->GetAction()->GetLabel();
	}
	else if (!Block->LabelOverride.Get().IsEmpty())
	{
		Label = Block->LabelOverride;
	}
	else if (Block->EntryWidget.IsValid())
	{
		const TSharedRef<SWidget>& TextBlockWidget = FindTextBlockWidget(Block->EntryWidget.ToSharedRef());
		if (TextBlockWidget != SNullWidget::NullWidget)
		{
			Label = StaticCastSharedRef<STextBlock>(TextBlockWidget)->GetText();
		}
	}

	return Label.Get().ToString().GetNSString();
}

NSImage* FSlateMacMenu::GetMenuItemIcon(const TSharedRef<const FMenuEntryBlock>& Block)
{
	NSImage* MenuImage = nil;
	FSlateIcon Icon;
	if (Block->IconOverride.IsSet())
	{
		Icon = Block->IconOverride;
	}
	else if (Block->GetAction().IsValid() && Block->GetAction()->GetIcon().IsSet())
	{
		Icon = Block->GetAction()->GetIcon();
	}
	if (Icon.IsSet())
	{
		if (Icon.GetIcon())
		{
			FSlateBrush const* IconBrush = Icon.GetIcon();
			FName ResourceName = IconBrush->GetResourceName();
			MenuImage = [[[NSImage alloc] initWithContentsOfFile:ResourceName.ToString().GetNSString()] autorelease];
			if (MenuImage)
			{
				[MenuImage setSize:NSMakeSize(16.0f, 16.0f)];
			}
		}
	}
	return MenuImage;
}

bool FSlateMacMenu::IsMenuItemEnabled(const TSharedRef<const class FMenuEntryBlock>& Block)
{
	TSharedPtr<const FUICommandList> ActionList = Block->GetActionList();
	TSharedPtr<const FUICommandInfo> Action = Block->GetAction();
	const FUIAction& DirectActions = Block->GetDirectActions();

	bool bEnabled = true;
	if (ActionList.IsValid() && Action.IsValid())
	{
		bEnabled = ActionList->CanExecuteAction(Action.ToSharedRef());
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		bEnabled = DirectActions.CanExecute();
	}
    
    if(FPlatformApplicationMisc::bMacApplicationModalMode)
    {
        bEnabled = false;
    }

	return bEnabled;
}

int32 FSlateMacMenu::GetMenuItemState(const TSharedRef<const class FMenuEntryBlock>& Block)
{
	TSharedPtr<const FUICommandList> ActionList = Block->GetActionList();
	TSharedPtr<const FUICommandInfo> Action = Block->GetAction();
	const FUIAction& DirectActions = Block->GetDirectActions();

	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if (ActionList.IsValid() && Action.IsValid())
	{
		CheckState = ActionList->GetCheckState(Action.ToSharedRef());
	}
	else
	{
		// There is no action list or action associated with this block via a UI command.  Execute any direct action we have
		CheckState = DirectActions.GetCheckState();
	}

	switch(CheckState)
	{
	case ECheckBoxState::Checked:
		return NSControlStateValueOn;
	case ECheckBoxState::Undetermined:
		return NSControlStateValueMixed;
	default:
		break;
	}
	return NSControlStateValueOff;
}
