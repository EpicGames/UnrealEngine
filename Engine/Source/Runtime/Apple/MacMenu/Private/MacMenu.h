// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/SMenuEntryBlock.h"
#include "Mac/CocoaMenu.h"

@interface FMacMenu : FCocoaMenu <NSMenuDelegate>
@property (assign) TWeakPtr<const FMenuEntryBlock> MenuEntryBlock;
@property (assign) TWeakPtr<const FMultiBox> MultiBox;
@end

class MACMENU_API FSlateMacMenu
{
public:
	static void PostInitStartup();
	static void CleanupOnShutdown();
	static void UpdateWithMultiBox(const TSharedPtr<FMultiBox> MultiBox);
	static void UpdateMenu(FMacMenu* Menu);
	static void UpdateCachedState();
	static void ExecuteMenuItemAction(const TSharedRef<const FMenuEntryBlock>& Block);
	static void UpdateApplicationMenu(bool bMacApplicationModalMode);
	static void UpdateWindowMenu(bool bMacApplicationModalMode);
	static void LanguageChanged();

private:

	static NSString* GetMenuItemTitle(const TSharedRef< const FMenuEntryBlock >& Block);
	static NSImage* GetMenuItemIcon(const TSharedRef<const FMenuEntryBlock>& Block);
	static bool IsMenuItemEnabled(const TSharedRef<const FMenuEntryBlock>& Block);
	static int32 GetMenuItemState(const TSharedRef<const FMenuEntryBlock>& Block);
};
