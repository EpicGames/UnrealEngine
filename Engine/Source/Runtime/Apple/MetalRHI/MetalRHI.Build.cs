// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class MetalRHI : ModuleRules
{	
	public MetalRHI(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"Engine",
				"RHI",
				"RHICore",
				"RenderCore",
				"Projects",
				"TraceLog",
				"Slate",
				"SlateCore"
			}
			);

		PublicIncludePathModuleNames.Add("MetalCPP");

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"MetalCPP"
		);

        AddEngineThirdPartyPrivateStaticDependencies(Target,
            "MetalShaderConverter"
        );   
		
		PublicWeakFrameworks.Add("Metal");

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.Add("QuartzCore");
		}
	}
}
