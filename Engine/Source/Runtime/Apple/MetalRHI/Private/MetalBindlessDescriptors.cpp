// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalBindlessDescriptors.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "MetalRHIContext.h"
#include "UpdateDescriptorHandle.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

#include "MetalDevice.h"
#include "MetalCommandEncoder.h"
#include "MetalDynamicRHI.h"

int32 GBindlessResourceDescriptorHeapSize = 1 << 23; // 8MB
static FAutoConsoleVariableRef CVarBindlessResourceDescriptorHeapSize(
	TEXT("Metal.Bindless.ResourceDescriptorHeapSize"),
	GBindlessResourceDescriptorHeapSize,
	TEXT("Bindless resource descriptor heap size"),
	ECVF_ReadOnly
);

int32 GBindlessSamplerDescriptorHeapSize = 1 << 16; // 64k
static FAutoConsoleVariableRef CVarBindlessSamplerDescriptorHeapSize(
	TEXT("Metal.Bindless.SamplerDescriptorHeapSize"),
	GBindlessSamplerDescriptorHeapSize,
	TEXT("Bindless sampler descriptor heap size"),
	ECVF_ReadOnly
);

static IRDescriptorTableEntry CreateDescriptor(MTL::SamplerState* SamplerState)
{
	IRDescriptorTableEntry DescriptorData{};
	IRDescriptorTableSetSampler(&DescriptorData, SamplerState, 0.0f);
	return DescriptorData;
}

static IRDescriptorTableEntry CreateDescriptor(MTL::Texture* Texture)
{
	IRDescriptorTableEntry DescriptorData{};
	IRDescriptorTableSetTexture(&DescriptorData, Texture, 0.0f, 0u);
	return DescriptorData;
}

static IRDescriptorTableEntry CreateDescriptor(FMetalBuffer* Buffer, const uint32_t ExtraOffset = 0)
{
	IRDescriptorTableEntry DescriptorData{};
	IRDescriptorTableSetBuffer(&DescriptorData, Buffer->GetGPUAddress() + ExtraOffset, Buffer->GetLength());
	return DescriptorData;
}

static IRDescriptorTableEntry CreateDescriptor(const FMetalResourceViewBase::FBufferView& View)
{
	IRDescriptorTableEntry DescriptorData{};
	IRDescriptorTableSetBuffer(&DescriptorData, View.Buffer->GetGPUAddress() + View.Offset, View.Size);
	return DescriptorData;
}

static IRDescriptorTableEntry CreateDescriptor(const FMetalResourceViewBase::FTextureBufferBacked& View)
{
	IRDescriptorTableEntry DescriptorData{};
	
	IRBufferView BufferView;
	BufferView.buffer = View.Buffer->GetMTLBuffer();
	BufferView.bufferOffset = View.Buffer->GetOffset() + View.Offset;
	BufferView.bufferSize = View.Size;
	BufferView.typedBuffer = true;
	BufferView.textureBufferView = View.Texture.get();
	
	uint32 Stride = GPixelFormats[View.Format].BlockBytes;
	uint32 FirstElement = View.Offset / Stride;
	uint32 NumElement = View.Size / Stride;
	
	uint64 BufferVA = View.Buffer->GetGPUAddress() + View.Offset;
	uint64_t ExtraElement = (BufferVA % 16) / Stride;
	
	BufferView.textureViewOffsetInElements = ExtraElement;
	
	IRDescriptorTableSetBufferView(&DescriptorData, &BufferView);
	
	return DescriptorData;
}

#if METAL_RHI_RAYTRACING
static IRDescriptorTableEntry CreateDescriptor(FMetalAccelerationStructure* AccelerationStructure)
{
	IRDescriptorTableEntry DescriptorData{};
	
	// We may not have an indirect argument buffer so just generate an empty descriptor
	if(AccelerationStructure->GetIndirectArgumentBuffer())
	{
		IRDescriptorTableSetAccelerationStructure(&DescriptorData, AccelerationStructure->GetIndirectArgumentBuffer()->GetGPUAddress());
	}
	return DescriptorData;
}
#endif

static IRDescriptorTableEntry CreateDescriptor(FMetalResourceViewBase* Resource)
{
	switch (Resource->GetMetalType())
	{
	case FMetalResourceViewBase::EMetalType::TextureView:
		return CreateDescriptor(Resource->GetTextureView().get());
	case FMetalResourceViewBase::EMetalType::BufferView:
		return CreateDescriptor(Resource->GetBufferView());
	case FMetalResourceViewBase::EMetalType::TextureBufferBacked:
		return CreateDescriptor(Resource->GetTextureBufferBacked());
#if METAL_RHI_RAYTRACING
	case FMetalResourceViewBase::EMetalType::AccelerationStructure:
		return CreateDescriptor(Resource->GetAccelerationStructure());
#endif
	default:
		checkNoEntry();
		return IRDescriptorTableEntry{};
	};
}

FMetalDescriptorHeap::FMetalDescriptorHeap(FMetalDevice& MetalDevice, ERHIDescriptorTypeMask InTypeMask)
	: Device(MetalDevice)
	, TypeMask(InTypeMask)
{
}

FMetalDescriptorHeap::~FMetalDescriptorHeap()
{
}

void FMetalDescriptorHeap::Init(FRHICommandListBase& RHICmdList, int32 HeapSize)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateStructured(TEXT("ResourceHeap"), HeapSize, 1)
		.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::KeepCPUAccessible | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::StructuredBuffer)
		.DetermineInitialState();

	ResourceHeap = new FMetalRHIBuffer(RHICmdList, Device, CreateDesc, nullptr);
	
	Manager = new FRHIHeapDescriptorAllocator(TypeMask, HeapSize / sizeof(IRDescriptorTableEntry), {});
}

FRHIDescriptorHandle FMetalDescriptorHeap::AllocateDescriptor(ERHIDescriptorType DescriptorType)
{
	return Manager->Allocate(DescriptorType);
}

FRHIDescriptorHandle FMetalDescriptorHeap::AllocateDescriptor(ERHIDescriptorType DescriptorType, IRDescriptorTableEntry DescriptorData)
{
	FRHIDescriptorHandle Handle = AllocateDescriptor(DescriptorType);
	UpdateDescriptorImmediately(Handle, DescriptorData);
	return Handle;
}

void FMetalDescriptorHeap::FreeDescriptor(FRHIDescriptorHandle DescriptorHandle)
{
	FMetalDynamicRHI::Get().DeferredDelete([this, DescriptorHandle]()
	{
		Manager->Free(DescriptorHandle);
	});
}

void FMetalDescriptorHeap::UpdateDescriptorImmediately(FRHIDescriptorHandle DescriptorHandle, IRDescriptorTableEntry DescriptorData)
{
	checkf(DescriptorHandle.IsValid(), TEXT("Attemping to update invalid descriptor handle!"));
	
	uint32 DescriptorIndex = DescriptorHandle.GetIndex();
	GetDescriptorMemory()[DescriptorIndex] = DescriptorData;
}

void FMetalDescriptorHeap::BindToEncoder(FMetalCommandEncoder* Encoder, MTL::FunctionType FunctionType, uint32 BindIndex)
{
	FRHIDescriptorAllocatorRange AllocatedRange(0, 0);
	if (Manager && Manager->GetAllocatedRange(AllocatedRange))
	{
		uint32 DescriptorCount = AllocatedRange.Last;
		const uint64 HeapSize = DescriptorCount * sizeof(IRDescriptorTableEntry);
		
		Encoder->SetShaderBuffer(FunctionType, ResourceHeap->GetCurrentBuffer(), 0, HeapSize, BindIndex, MTL::ResourceUsageRead);
	}
}

FMetalBindlessDescriptorManager::FMetalBindlessDescriptorManager(FMetalDevice& MetalDevice)
	: Device(MetalDevice)
	, StandardResources(Device, ERHIDescriptorTypeMask::Standard)
	, SamplerResources(Device, ERHIDescriptorTypeMask::Sampler)
{
}

FMetalBindlessDescriptorManager::~FMetalBindlessDescriptorManager()
{
}

void FMetalBindlessDescriptorManager::Init()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	
	StandardResources.Init(RHICmdList, GBindlessResourceDescriptorHeapSize);
	SamplerResources.Init(RHICmdList, GBindlessSamplerDescriptorHeapSize);

	// Add the types to the global groups
	GRHIGlobals.DescriptorTypeGroups.Emplace(StandardResources.GetTypeMask());
	GRHIGlobals.DescriptorTypeGroups.Emplace(SamplerResources.GetTypeMask());
	
	bIsSupported = true;
}

FRHIDescriptorHandle FMetalBindlessDescriptorManager::AllocateDescriptor(ERHIDescriptorType DescriptorType)
{
	if (StandardResources.HandlesAllocation(DescriptorType))
	{
		return StandardResources.AllocateDescriptor(DescriptorType);
	}
	
	if (SamplerResources.HandlesAllocation(DescriptorType))
	{
		return SamplerResources.AllocateDescriptor(DescriptorType);
	}
	
	return FRHIDescriptorHandle();
}

FRHIDescriptorHandle FMetalBindlessDescriptorManager::AllocateDescriptor(ERHIDescriptorType DescriptorType, MTL::Texture* Texture)
{
	IRDescriptorTableEntry Descriptor = CreateDescriptor(Texture);
	return StandardResources.AllocateDescriptor(DescriptorType, Descriptor);
}

FRHIDescriptorHandle FMetalBindlessDescriptorManager::AllocateDescriptor(ERHIDescriptorType DescriptorType, FMetalBuffer* Buffer, const uint32_t ExtraOffset)
{
	IRDescriptorTableEntry Descriptor = CreateDescriptor(Buffer, ExtraOffset);
	return StandardResources.AllocateDescriptor(DescriptorType, Descriptor);
}

FRHIDescriptorHandle FMetalBindlessDescriptorManager::AllocateDescriptor(MTL::SamplerState* SamplerState)
{
	IRDescriptorTableEntry Descriptor = CreateDescriptor(SamplerState);
	return SamplerResources.AllocateDescriptor(ERHIDescriptorType::Sampler, Descriptor);
}

void FMetalBindlessDescriptorManager::FreeDescriptor(FRHIDescriptorHandle DescriptorHandle)
{
	check(DescriptorHandle.IsValid());

	const ERHIDescriptorType DescriptorType = DescriptorHandle.GetType();

	if (StandardResources.HandlesAllocation(DescriptorType))
	{
		StandardResources.FreeDescriptor(DescriptorHandle);
	}
	else if (SamplerResources.HandlesAllocation(DescriptorType))
	{
		SamplerResources.FreeDescriptor(DescriptorHandle);
	}
	else
	{
		checkNoEntry();
	}
}


void FMetalBindlessDescriptorManager::UpdateDescriptor(FRHICommandListBase& RHICmdList, FRHIDescriptorHandle DescriptorHandle, IRDescriptorTableEntry Descriptor, EMetalDescriptorUpdateType InUpdateType)
{
	const EMetalDescriptorUpdateType UpdateType = !GIsRHIInitialized ? EMetalDescriptorUpdateType::Immediate : InUpdateType;

	RHICmdList.EnqueueLambda([this, UpdateType, Descriptor, DescriptorHandle](FRHICommandListBase& RHICmdList)
	{
		if (UpdateType == EMetalDescriptorUpdateType::Immediate)
		{
			StandardResources.UpdateDescriptorImmediately(DescriptorHandle, Descriptor);
		}
		else
		{
			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
			Context.EnqueueDescriptorUpdate(DescriptorHandle, Descriptor);
		}
	});

	if (RHICmdList.IsTopOfPipe())
	{
		RHICmdList.RHIThreadFence(true);
	}
}

void FMetalBindlessDescriptorManager::UpdateDescriptor(FRHICommandListBase& RHICmdList, FRHIDescriptorHandle DescriptorHandle, MTL::Texture* Texture, EMetalDescriptorUpdateType UpdateType)
{
	IRDescriptorTableEntry Descriptor = CreateDescriptor(Texture);
	UpdateDescriptor(RHICmdList, DescriptorHandle, Descriptor, UpdateType);
}

void FMetalBindlessDescriptorManager::UpdateDescriptor(FRHICommandListBase& RHICmdList, FRHIDescriptorHandle DescriptorHandle, FMetalBuffer* Buffer, EMetalDescriptorUpdateType UpdateType)
{
	IRDescriptorTableEntry Descriptor = CreateDescriptor(Buffer);
	UpdateDescriptor(RHICmdList, DescriptorHandle, Descriptor, UpdateType);
}

void FMetalBindlessDescriptorManager::UpdateDescriptor(FMetalRHICommandContext* Context, FRHIDescriptorHandle DescriptorHandle, FMetalResourceViewBase* Resource, EMetalDescriptorUpdateType UpdateType)
{
	IRDescriptorTableEntry Descriptor = CreateDescriptor(Resource);
	
	if (Context && UpdateType != EMetalDescriptorUpdateType::Immediate)
	{
		check(GIsRHIInitialized);
		Context->EnqueueDescriptorUpdate(DescriptorHandle, Descriptor);
	}
	else
	{
		StandardResources.UpdateDescriptorImmediately(DescriptorHandle, Descriptor);
	}
}

void FMetalBindlessDescriptorManager::FlushPendingDescriptorUpdates(FMetalRHICommandContext& Context, const FMetalPendingDescriptorUpdates& PendingDescriptorUpdates)
{
	const uint32_t NumDescriptors = PendingDescriptorUpdates.Num();
	
	if (!NumDescriptors)
	{
		return;
	}
	
	FMetalDescriptorHeap* Heap = new FMetalDescriptorHeap(Device, ERHIDescriptorTypeMask::Standard);
	
	TRHICommandList_RecursiveHazardous<FMetalRHICommandContext> RHICmdList(&Context);
	Heap->Init(RHICmdList, 256);
	
	TShaderMapRef<FUpdateDescriptorHandleCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	
	FMetalTempAllocator* Allocator = Device.GetUniformAllocator();
	
	// Create the two temporary buffers for descriptor entries and indices
	FMetalBufferPtr DescriptorEntriesBuffer = Allocator->Allocate(NumDescriptors * sizeof(IRDescriptorTableEntry));
	FMemory::Memcpy(DescriptorEntriesBuffer->Contents(), PendingDescriptorUpdates.Descriptors.GetData(), NumDescriptors * sizeof(IRDescriptorTableEntry));
	
	FMetalBufferPtr DescriptorIndiciesBuffer = Allocator->Allocate(NumDescriptors * sizeof(uint32));
	FMemory::Memcpy(DescriptorIndiciesBuffer->Contents(), PendingDescriptorUpdates.Indices.GetData(), NumDescriptors * sizeof(uint32));
	
	// Create descriptor handles and bind buffers to descriptor heap
	FRHIDescriptorHandle DescriptorEntriesHandle = Heap->AllocateDescriptor(ERHIDescriptorType::BufferSRV, CreateDescriptor(DescriptorEntriesBuffer.Get()));
	FRHIDescriptorHandle DescriptorIndicesHandle = Heap->AllocateDescriptor(ERHIDescriptorType::BufferSRV, CreateDescriptor(DescriptorIndiciesBuffer.Get()));
	
	FMetalRHIBuffer* DestBuffer = StandardResources.GetHeap();
	FRHIDescriptorHandle DestBufferHandle = Heap->AllocateDescriptor(ERHIDescriptorType::BufferUAV, CreateDescriptor(DestBuffer->GetCurrentBuffer().Get()));

	// Create a temporary uniform buffer
	uint32* UBDataPtr = reinterpret_cast<uint32*>(FMemory::Malloc(4 * sizeof(uint32)));

	UBDataPtr[0] = NumDescriptors;
	UBDataPtr[1] = DescriptorIndicesHandle.GetIndex();
	UBDataPtr[2] = DescriptorEntriesHandle.GetIndex();
	UBDataPtr[3] = DestBufferHandle.GetIndex();

	Context.GetStateCache().SetComputeShader((FMetalComputeShader*)ShaderRHI);

	FMetalRHIBuffer* Dest = StandardResources.GetHeap();

	Context.GetStateCache().CacheOrSkipResourceResidencyUpdate(DescriptorEntriesBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true, true);
	Context.GetStateCache().CacheOrSkipResourceResidencyUpdate(DescriptorIndiciesBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true, true);
	Context.GetStateCache().CacheOrSkipResourceResidencyUpdate(Dest->GetCurrentBuffer()->GetMTLBuffer(), EMetalShaderStages::Compute, false, true);

	Context.GetStateCache().SetOverriddenDescriptorHeap(Heap);

	FMetalBufferPtr NullBuffer;
	Context.GetStateCache().IRBindPackedUniforms(EMetalShaderStages::Compute, 0, (const uint8*)UBDataPtr, 16, NullBuffer);
	Context.GetStateCache().GetShaderParameters(EMetalShaderStages::Compute).MarkAllDirty();

	Context.RHIDispatchComputeShader(1, 1, 1);
	
	// Insert a compute memory barrier after this write for subsequent compute shader reads of bindless heap
	Context.InsertComputeMemoryBarrier();
	
	Context.GetStateCache().SetOverriddenDescriptorHeap(nullptr);

	FMetalDynamicRHI::Get().DeferredDelete([DescriptorEntriesBuffer, DescriptorIndiciesBuffer, Heap, UBDataPtr]()
	{
		delete Heap;
		FMemory::Free(UBDataPtr);
	});
}

void FMetalBindlessDescriptorManager::BindDescriptorHeapsToEncoder(FMetalCommandEncoder* Encoder, MTL::FunctionType FunctionType, EMetalShaderStages Frequency)
{
	StandardResources.BindToEncoder(Encoder, FunctionType, kIRStandardHeapBindPoint);
	SamplerResources.BindToEncoder(Encoder, FunctionType, kIRSamplerHeapBindPoint);
}

#endif //PLATFORM_SUPPORTS_BINDLESS_RENDERING
