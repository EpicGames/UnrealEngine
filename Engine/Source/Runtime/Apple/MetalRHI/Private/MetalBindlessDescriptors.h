// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
#include "MetalResources.h"
#include "MetalShaderResources.h"
#include "RHIDescriptorAllocator.h"

class FMetalCommandEncoder;
class FMetalDevice;

struct IRDescriptorTableEntry;

enum class EMetalDescriptorUpdateType
{
	Immediate,
	GPU,
};

struct FMetalPendingDescriptorUpdates
{
	TArray<uint32> Indices;
	TArray<IRDescriptorTableEntry> Descriptors;

	int32 Num() const
	{
		return Indices.Num();
	}

	void Add(FRHIDescriptorHandle InHandle, const IRDescriptorTableEntry& InDescriptor)
	{
		if (ensure(InHandle.IsValid()))
		{
			Indices.Emplace(InHandle.GetIndex());
			Descriptors.Emplace(InDescriptor);
		}
	}

	void Empty()
	{
		Indices.Empty();
		Descriptors.Empty();
	}
};

struct FMetalDescriptorHeap
{
	FMetalDescriptorHeap(FMetalDevice& MetalDevice, ERHIDescriptorTypeMask InTypeMask);
	~FMetalDescriptorHeap();

	void Init(FRHICommandListBase& RHICmdList, int32 HeapSize);

	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType);
	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType, IRDescriptorTableEntry DescriptorData);
	void                 FreeDescriptor(FRHIDescriptorHandle DescriptorHandle);

	void UpdateDescriptorImmediately(FRHIDescriptorHandle DescriptorHandle, IRDescriptorTableEntry DescriptorData);
	void BindToEncoder(FMetalCommandEncoder* Encoder, MTL::FunctionType FunctionType, uint32 BindIndex);

	ERHIDescriptorTypeMask GetTypeMask() const
	{
		return TypeMask;
	}

	bool HandlesAllocation(ERHIDescriptorType InType) const
	{
		return EnumHasAnyFlags(GetTypeMask(), RHIDescriptorTypeMaskFromType(InType));
	}

	FMetalRHIBuffer* GetHeap() const
	{
		return ResourceHeap.GetReference();
	}

private:
	IRDescriptorTableEntry* GetDescriptorMemory()
	{
		checkSlow(ResourceHeap != nullptr);
		return reinterpret_cast<IRDescriptorTableEntry*>(ResourceHeap->GetCurrentBuffer()->Contents());
	}

	FMetalDevice& Device;

	FRHIHeapDescriptorAllocator* Manager = nullptr;
	TRefCountPtr<FMetalRHIBuffer> ResourceHeap;

	const ERHIDescriptorTypeMask TypeMask;
};

class FMetalBindlessDescriptorManager
{
public:
	FMetalBindlessDescriptorManager(FMetalDevice& MetalDevice);
	~FMetalBindlessDescriptorManager();

    void Init();

	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType);
	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType, MTL::Texture* Texture);
	FRHIDescriptorHandle AllocateDescriptor(ERHIDescriptorType DescriptorType, FMetalBuffer* Buffer, const uint32_t ExtraOffset = 0);
	FRHIDescriptorHandle AllocateDescriptor(MTL::SamplerState* SamplerState);

	void FreeDescriptor(FRHIDescriptorHandle DescriptorHandle);

	void UpdateDescriptor(FRHICommandListBase& RHICmdList, FRHIDescriptorHandle DescriptorHandle, IRDescriptorTableEntry Descriptor, EMetalDescriptorUpdateType UpdateType);
	void UpdateDescriptor(FRHICommandListBase& RHICmdList, FRHIDescriptorHandle DescriptorHandle, MTL::Texture* Texture, EMetalDescriptorUpdateType UpdateType);
	void UpdateDescriptor(FRHICommandListBase& RHICmdList, FRHIDescriptorHandle DescriptorHandle, FMetalBuffer* Buffer, EMetalDescriptorUpdateType UpdateType);
	void UpdateDescriptor(FMetalRHICommandContext* Context, FRHIDescriptorHandle DescriptorHandle, FMetalResourceViewBase* Resource, EMetalDescriptorUpdateType UpdateType);
	
	void FlushPendingDescriptorUpdates(FMetalRHICommandContext& Context, const FMetalPendingDescriptorUpdates& PendingDescriptorUpdates);
    void BindDescriptorHeapsToEncoder(FMetalCommandEncoder* Encoder, MTL::FunctionType FunctionType, EMetalShaderStages Frequency);

	FMetalRHIBuffer* GetStandardResourceHeap() 
	{
		check(StandardResources.GetHeap());
		return StandardResources.GetHeap();
	}
	
	FMetalRHIBuffer* GetSamplerResourceHeap() 
	{
		check(SamplerResources.GetHeap());
		return SamplerResources.GetHeap();
	}
	
	bool IsSupported() const
	{
		return bIsSupported;
	}

private:
	FMetalDevice& Device;
	
    FMetalDescriptorHeap StandardResources;
    FMetalDescriptorHeap SamplerResources;
	
	bool bIsSupported = false;
};

#endif
