// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalBuffer.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalCommandBuffer.h"
#include "MetalCommandQueue.h"
#include "MetalDevice.h"
#include "MetalLLM.h"
#include "MetalProfiler.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "Templates/AlignmentTemplates.h"
#include <objc/runtime.h>

DECLARE_MEMORY_STAT(TEXT("Used Device Buffer Memory"), STAT_MetalDeviceBufferMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Used Pooled Buffer Memory"), STAT_MetalPooledBufferMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Used Magazine Buffer Memory"), STAT_MetalMagazineBufferMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Used Heap Buffer Memory"), STAT_MetalHeapBufferMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Used Linear Buffer Memory"), STAT_MetalLinearBufferMemory, STATGROUP_MetalRHI);

DECLARE_MEMORY_STAT(TEXT("Unused Pooled Buffer Memory"), STAT_MetalPooledBufferUnusedMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Unused Magazine Buffer Memory"), STAT_MetalMagazineBufferUnusedMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Unused Heap Buffer Memory"), STAT_MetalHeapBufferUnusedMemory, STATGROUP_MetalRHI);
DECLARE_MEMORY_STAT(TEXT("Unused Linear Buffer Memory"), STAT_MetalLinearBufferUnusedMemory, STATGROUP_MetalRHI);

static int32 GMetalHeapBufferBytesToCompact = 0;
static FAutoConsoleVariableRef CVarMetalHeapBufferBytesToCompact(
    TEXT("rhi.Metal.HeapBufferBytesToCompact"),
    GMetalHeapBufferBytesToCompact,
    TEXT("When enabled (> 0) this will force MetalRHI to compact the given number of bytes each frame into older buffer heaps from newer ones in order to defragment memory and reduce wastage.\n")
    TEXT("(Off by default (0))"));

static int32 GMetalResourcePurgeInPool = 0;
static FAutoConsoleVariableRef CVarMetalResourcePurgeInPool(
	TEXT("rhi.Metal.ResourcePurgeInPool"),
	GMetalResourcePurgeInPool,
	TEXT("Use the SetPurgeableState function to allow the OS to reclaim memory from resources while they are unused in the pools. (Default: 0, Off)"));

static int32 GMetalMaxBucketSize = 32 * 1024 * 1024;
static FAutoConsoleVariableRef CVarMetalMaxBucketSize(
	TEXT("rhi.Metal.MaxBucketSize"),
	GMetalMaxBucketSize,
	TEXT("Set the Max Bucket Size for the Pooled Buffer. (Default: 33547255)"));

#if METAL_DEBUG_OPTIONS
extern int32 GMetalBufferScribble;
#endif

extern int32 GMetalResourcePurgeOnDelete;

FMetalBuffer::~FMetalBuffer()
{
    Release();
}

FMetalBuffer::FMetalBuffer(MTL::Buffer* Handle, FreePolicy InFreePolicy)
: Buffer(Handle)
, Allocator(nullptr)
, SubRange(0, Handle->length())
, OnFreePolicy(InFreePolicy)
{
	if(OnFreePolicy == FreePolicy::Owner)
	{
		INC_MEMORY_STAT_BY(STAT_MetalDeviceBufferMemory, GetLength());
		MetalLLM::LogAllocBufferNative(Buffer);
	}
}

FMetalBuffer::FMetalBuffer(MTL::Buffer* Handle, NS::Range Range, IMetalBufferAllocator* InAllocator)
: Buffer(Handle)
, Allocator(InAllocator)
, SubRange(Range)
, OnFreePolicy(FreePolicy::BufferAllocator)
{
}

void FMetalBuffer::Release()
{
    check(!GIsRHIInitialized || (OnFreePolicy != FreePolicy::Owner || bMarkedDeleted));
    
	if (OnFreePolicy == FreePolicy::BufferAllocator)
	{
		check(Allocator);
		Allocator->ReleaseBuffer(this);
		Allocator = nullptr;
	}
	else if(OnFreePolicy == FreePolicy::Owner)
	{
		MetalLLM::LogFreeBufferNative(Buffer);
		
		DEC_MEMORY_STAT_BY(STAT_MetalDeviceBufferMemory, GetLength());
		if (GMetalResourcePurgeOnDelete)
		{
			Buffer->setPurgeableState(MTL::PurgeableStateEmpty);
		}
		Buffer->release();
	}
}

FMetalSubBufferHeap::FMetalSubBufferHeap(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger Alignment, MTL::ResourceOptions Options, FCriticalSection& InPoolMutex)
: Device(MetalDevice)
, PoolMutex(InPoolMutex)
, OutstandingAllocs(0)
, MinAlign(Alignment)
, UsedSize(0)
{
	Options = (MTL::ResourceOptions)FMetalCommandQueue::GetCompatibleResourceOptions(Options);
	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	NS::UInteger FullSize = Align(Size, Alignment);
	
	MTL::StorageMode Storage = (MTL::StorageMode)((Options & MTL::ResourceStorageModeMask) >> MTL::ResourceStorageModeShift);
#if PLATFORM_MAC
	check(Storage != MTL::StorageModeManaged /* Managed memory cannot be safely suballocated! When you overwrite existing data the GPU buffer is immediately disposed of! */);
#endif

	if (bSupportsHeaps && Storage == MTL::StorageModePrivate)
	{
        MTL::HeapDescriptor* Desc = MTL::HeapDescriptor::alloc()->init();
        check(Desc);
        
		Desc->setSize(FullSize);
		Desc->setStorageMode(Storage);
		ParentHeap = NS::TransferPtr(Device.GetDevice()->newHeap(Desc));
        Desc->release();
		check(ParentHeap);
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocHeap(ParentHeap.get());
#endif
	}
	else
	{
        ParentBuffer = Device.GetDevice()->newBuffer(FullSize, Options);
		check(ParentBuffer);
		check(ParentBuffer->length() >= FullSize);
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocBufferNative(ParentBuffer);
#endif
		FreeRanges.Add(NS::Range(0, FullSize));
	}
	INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, FullSize);
	INC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, FullSize);
	
}

FMetalSubBufferHeap::~FMetalSubBufferHeap()
{
	if (ParentHeap)
	{
		DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, ParentHeap->size());
		DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, ParentHeap->size());
	}
	else
	{
		DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, ParentBuffer->length());
		DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, ParentBuffer->length());
	}
	
	if(ParentBuffer)
	{
		MetalLLM::LogFreeBufferNative(ParentBuffer);
		
		if (GMetalResourcePurgeOnDelete)
		{
			ParentBuffer->setPurgeableState(MTL::PurgeableStateEmpty);
		}
		ParentBuffer->release();
		ParentBuffer = nullptr;
	}
}

void FMetalSubBufferHeap::SetOwner(NS::Range const& Range, FMetalRHIBuffer* Owner, bool bIsSwap)
{
	check(Owner == nullptr);
	FScopeLock Lock(&PoolMutex);
	for (uint32 i = 0; i < AllocRanges.Num(); i++)
	{
		if (AllocRanges[i].Range.location == Range.location)
		{
			check(AllocRanges[i].Range.length == Range.length);
			check(AllocRanges[i].Owner == nullptr || Owner == nullptr || bIsSwap);
			AllocRanges[i].Owner = Owner;
			break;
		}
	}
}

void FMetalSubBufferHeap::FreeRange(NS::Range const& Range)
{
	{
		FScopeLock Lock(&PoolMutex);
		for (uint32 i = 0; i < AllocRanges.Num(); i++)
		{
			if (AllocRanges[i].Range.location == Range.location)
			{
				check(AllocRanges[i].Range.length == Range.length);
				AllocRanges.RemoveAt(i);
				break;
			}
		}
	}
	if (ParentHeap)
	{
		INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
		INC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, Range.length);
		DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferMemory, Range.length);
	}
	else
	{
#if METAL_DEBUG_OPTIONS
		if (GIsRHIInitialized)
		{
            assert(Range.location < ParentBuffer->length());
            assert(Range.length && (Range.location + Range.Length) <= ParentBuffer->length());
            
			Device.ValidateIsInactiveBuffer(ParentBuffer, Range);
		}
#endif
    
		FScopeLock Lock(&PoolMutex);
		{
			NS::Range CompactRange = Range;
			for (uint32 i = 0; i < FreeRanges.Num(); )
			{
				if (FreeRanges[i].location == (CompactRange.location + CompactRange.length))
				{
                    NS::Range PrevRange = FreeRanges[i];
					FreeRanges.RemoveAt(i);
					
					CompactRange.length += PrevRange.length;
				}
				else if (CompactRange.location == (FreeRanges[i].location + FreeRanges[i].length))
				{
                    NS::Range PrevRange = FreeRanges[i];
					FreeRanges.RemoveAt(i);
					
					CompactRange.location = PrevRange.location;
					CompactRange.length += PrevRange.length;
				}
				else
				{
					i++;
				}
			}
		
			uint32 i = 0;
			for (; i < FreeRanges.Num(); i++)
			{
				if (FreeRanges[i].length >= CompactRange.length)
				{
					break;
				}
			}
			FreeRanges.Insert(CompactRange, i);
		
			UsedSize -= Range.length;
			
			INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
			INC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, Range.length);
			DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferMemory, Range.length);
		
#if METAL_DEBUG_OPTIONS
			uint64 LostSize = GetSize() - UsedSize;
			for (NS::Range const& FreeRange : FreeRanges)
			{
				LostSize -= FreeRange.length;
			}
			check(LostSize == 0);
#endif
		}
	}
	FPlatformAtomics::InterlockedDecrement(&OutstandingAllocs);
}

void FMetalSubBufferHeap::ReleaseBuffer(FMetalBuffer* Buffer)
{
	if(ParentHeap)
	{
		if (GMetalResourcePurgeOnDelete)
		{
			Buffer->GetMTLBuffer()->setPurgeableState(MTL::PurgeableStateEmpty);
		}
		Buffer->GetMTLBuffer()->release();
	}
	FreeRange(Buffer->GetRange());
}

NS::String* FMetalSubBufferHeap::GetLabel() const
{
	if (ParentHeap)
	{
		return ParentHeap->label();
	}
	else
	{
		return ParentBuffer->label();
	}
}

MTL::Device* FMetalSubBufferHeap::GetDevice() const
{
	if (ParentHeap)
	{
		return ParentHeap->device();
	}
	else
	{
		return ParentBuffer->device();
	}
}

MTL::StorageMode FMetalSubBufferHeap::GetStorageMode() const
{
	if (ParentHeap)
	{
		return ParentHeap->storageMode();
	}
	else
	{
		return ParentBuffer->storageMode();
	}
}

MTL::CPUCacheMode FMetalSubBufferHeap::GetCpuCacheMode() const
{
	if (ParentHeap)
	{
		return ParentHeap->cpuCacheMode();
	}
	else
	{
		return ParentBuffer->cpuCacheMode();
	}
}

NS::UInteger FMetalSubBufferHeap::GetSize() const
{
	if (ParentHeap)
	{
		return ParentHeap->size();
	}
	else
	{
		return ParentBuffer->length();
	}
}

NS::UInteger FMetalSubBufferHeap::GetUsedSize() const
{
	if (ParentHeap)
	{
		return ParentHeap->usedSize();
	}
	else
	{
		return UsedSize;
	}
}

int64 FMetalSubBufferHeap::NumCurrentAllocations() const
{
	return OutstandingAllocs;
}

void FMetalSubBufferHeap::SetLabel(const NS::String* label)
{
	if (ParentHeap)
	{
		ParentHeap->setLabel(label);
	}
	else
	{
		ParentBuffer->setLabel(label);
	}
}

NS::UInteger FMetalSubBufferHeap::MaxAvailableSize() const
{
	if (ParentHeap)
	{
		return ParentHeap->maxAvailableSize(MinAlign);
	}
	else
	{
		if (UsedSize < GetSize())
		{
			return FreeRanges.Last().length;
		}
		else
		{
			return 0;
		}
	}
}

bool FMetalSubBufferHeap::CanAllocateSize(NS::UInteger Size) const
{
	if (ParentHeap)
	{
		NS::UInteger Storage = (NS::UInteger(GetStorageMode()) << MTL::ResourceStorageModeShift);
		NS::UInteger Cache = (NS::UInteger(GetCpuCacheMode()) << MTL::ResourceCpuCacheModeShift);
		MTL::ResourceOptions Opt = MTL::ResourceOptions(Storage | Cache);
		
		NS::UInteger Align = ParentHeap->device()->heapBufferSizeAndAlign(Size, Opt).align;
		return Size <= ParentHeap->maxAvailableSize(Align);
	}
	else
	{
		return Size <= MaxAvailableSize();
	}
}

FMetalBufferPtr FMetalSubBufferHeap::NewBuffer(NS::UInteger length)
{
	NS::UInteger Size = Align(length, MinAlign);
	FMetalBufferPtr Result = nullptr;
	
	if (ParentHeap)
	{
		NS::UInteger Storage = (NS::UInteger(GetStorageMode()) << MTL::ResourceStorageModeShift);
		NS::UInteger Cache = (NS::UInteger(GetCpuCacheMode()) << MTL::ResourceCpuCacheModeShift);
		MTL::ResourceOptions Opt = MTL::ResourceOptions(Storage | Cache);
		
		Result = FMetalBufferPtr(new FMetalBuffer(ParentHeap->newBuffer(Size, Opt), NS::Range(0, Size), this));
		DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Result->GetLength());
		DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, Result->GetLength());
		INC_MEMORY_STAT_BY(STAT_MetalHeapBufferMemory, Result->GetLength());
	}
	else
	{
		check(ParentBuffer);
	
		FScopeLock Lock(&PoolMutex);
		if (MaxAvailableSize() >= Size)
		{
			for (uint32 i = 0; i < FreeRanges.Num(); i++)
			{
				if (FreeRanges[i].length >= Size)
				{
					NS::Range Range = FreeRanges[i];
					FreeRanges.RemoveAt(i);
					
					UsedSize += Range.length;
					
					DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
					DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, Range.length);
					INC_MEMORY_STAT_BY(STAT_MetalHeapBufferMemory, Range.length);
				
					if (Range.length > Size)
					{
						NS::Range Split = NS::Range(Range.location + Size, Range.length - Size);
						FPlatformAtomics::InterlockedIncrement(&OutstandingAllocs);
						FreeRange(Split);
						
						Range.length = Size;
					}
					
#if METAL_DEBUG_OPTIONS
					uint64 LostSize = GetSize() - UsedSize;
					for (NS::Range const& FreeRange : FreeRanges)
					{
						LostSize -= FreeRange.length;
					}
					check(LostSize == 0);
#endif				
                    Result = FMetalBufferPtr(new FMetalBuffer(ParentBuffer, Range, this));
                    Allocation Alloc;
                    Alloc.Range = Range;
                    Alloc.Resource = Result->GetMTLBuffer();
                    Alloc.Owner = nullptr;
                    AllocRanges.Add(Alloc);

					break;
				}
			}
		}
	}
	FPlatformAtomics::InterlockedIncrement(&OutstandingAllocs);
	check(Result);
	return Result;
}

MTL::PurgeableState FMetalSubBufferHeap::SetPurgeableState(MTL::PurgeableState state)
{
	if (ParentHeap)
	{
		return ParentHeap->setPurgeableState(state);
	}
	else
	{
		return ParentBuffer->setPurgeableState(state);
	}
}

#pragma mark --

FMetalSubBufferLinear::FMetalSubBufferLinear(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger Alignment, MTL::ResourceOptions Options, FCriticalSection& InPoolMutex)
: Device(MetalDevice)
, PoolMutex(InPoolMutex)
, MinAlign(Alignment)
, WriteHead(0)
, UsedSize(0)
, FreedSize(0)
{
	Options = (MTL::ResourceOptions)FMetalCommandQueue::GetCompatibleResourceOptions(Options);
	NS::UInteger FullSize = Align(Size, Alignment);
	
	MTL::StorageMode Storage = (MTL::StorageMode)((Options & MTL::ResourceStorageModeMask) >> MTL::ResourceStorageModeShift);
	
    ParentBuffer = Device.GetDevice()->newBuffer(FullSize, Options);
    
	check(ParentBuffer);
	check(ParentBuffer->length() >= FullSize);
    
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
	MetalLLM::LogAllocBufferNative(ParentBuffer);
#endif
    
	INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, FullSize);
	INC_MEMORY_STAT_BY(STAT_MetalLinearBufferUnusedMemory, FullSize);
}

FMetalSubBufferLinear::~FMetalSubBufferLinear()
{
	DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, ParentBuffer->length());
	DEC_MEMORY_STAT_BY(STAT_MetalLinearBufferUnusedMemory, ParentBuffer->length());
	
	if(ParentBuffer)
	{
		MetalLLM::LogFreeBufferNative(ParentBuffer);
		
		if (GMetalResourcePurgeOnDelete)
		{
			ParentBuffer->setPurgeableState(MTL::PurgeableStateEmpty);
		}
		ParentBuffer->release();
		ParentBuffer = nullptr;
	}
}

void FMetalSubBufferLinear::FreeRange(NS::Range const& Range)
{
#if METAL_DEBUG_OPTIONS
	if (GIsRHIInitialized)
	{
        assert(Range.location < ParentBuffer->length());
        assert(Range.length && (Range.location + Range.length) <= ParentBuffer->length());
        
		Device.ValidateIsInactiveBuffer(ParentBuffer, Range);
	}
#endif
	
	FScopeLock Lock(&PoolMutex);
	{
		FreedSize += Range.length;
		INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
		INC_MEMORY_STAT_BY(STAT_MetalLinearBufferUnusedMemory, Range.length);
		DEC_MEMORY_STAT_BY(STAT_MetalLinearBufferMemory, Range.length);
		if (FreedSize == UsedSize)
		{
			UsedSize = 0;
			FreedSize = 0;
			WriteHead = 0;
		}
	}
}

void FMetalSubBufferLinear::ReleaseBuffer(FMetalBuffer* Buffer)
{
	FreeRange(Buffer->GetRange());
}

NS::String* FMetalSubBufferLinear::GetLabel() const
{
	return ParentBuffer->label();
}

MTL::Device* FMetalSubBufferLinear::GetDevice() const
{
	return ParentBuffer->device();
}

MTL::StorageMode FMetalSubBufferLinear::GetStorageMode() const
{
	return ParentBuffer->storageMode();
}

MTL::CPUCacheMode FMetalSubBufferLinear::GetCpuCacheMode() const
{
	return ParentBuffer->cpuCacheMode();
}

NS::UInteger FMetalSubBufferLinear::GetSize() const
{
	return ParentBuffer->length();
}

NS::UInteger FMetalSubBufferLinear::GetUsedSize() const
{
	return UsedSize;
}

void FMetalSubBufferLinear::SetLabel(const NS::String* label)
{
	ParentBuffer->setLabel(label);
}

bool FMetalSubBufferLinear::CanAllocateSize(NS::UInteger Size) const
{
	if (WriteHead < GetSize())
	{
		NS::UInteger Alignment = FMath::Max(NS::UInteger(MinAlign), NS::UInteger(Size & ~(Size - 1llu)));
		NS::UInteger NewWriteHead = Align(WriteHead, Alignment);
		return (GetSize() - NewWriteHead) > Size;
	}
	else
	{
		return 0;
	}
}

FMetalBufferPtr FMetalSubBufferLinear::NewBuffer(NS::UInteger length)
{
	FScopeLock Lock(&PoolMutex);
	NS::UInteger Alignment = FMath::Max(NS::UInteger(MinAlign), NS::UInteger(length & ~(length - 1llu)));
	NS::UInteger Size = Align(length, Alignment);
	NS::UInteger NewWriteHead = Align(WriteHead, Alignment);
	
	FMetalBufferPtr Result = nullptr;
	if ((GetSize() - NewWriteHead) > Size)
	{
		NS::Range Range(NewWriteHead, Size);
		DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
		DEC_MEMORY_STAT_BY(STAT_MetalLinearBufferUnusedMemory, Range.length);
		INC_MEMORY_STAT_BY(STAT_MetalLinearBufferMemory, Range.length);
        
        assert(Range.Location < ParentBuffer->getLength());
        assert(Range.Length && (Range.Location + Range.length) <= ParentBuffer->getLength());
        
        Result = FMetalBufferPtr(new FMetalBuffer(ParentBuffer, Range, this));
		UsedSize += Size;
		WriteHead = NewWriteHead + Size;
	}
	
	return Result;
}

MTL::PurgeableState FMetalSubBufferLinear::SetPurgeableState(MTL::PurgeableState state)
{
	return ParentBuffer->setPurgeableState(state);
}

#pragma mark --

FMetalSubBufferMagazine::FMetalSubBufferMagazine(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger ChunkSize, MTL::ResourceOptions Options)
: Device(MetalDevice)
, MinAlign(ChunkSize)
, BlockSize(ChunkSize)
, OutstandingAllocs(0)
, UsedSize(0)
{
	Options = (MTL::ResourceOptions)FMetalCommandQueue::GetCompatibleResourceOptions(Options);
    static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
    MTL::StorageMode Storage = (MTL::StorageMode)((Options & MTL::ResourceStorageModeMask) >> MTL::ResourceStorageModeShift);
    if (PLATFORM_IOS && bSupportsHeaps && Storage == MTL::StorageModePrivate)
    {
        MinAlign = Device.GetDevice()->heapBufferSizeAndAlign(BlockSize, Options).align;
    }
    
    NS::UInteger FullSize = Align(Size, MinAlign);
	
#if PLATFORM_MAC
	check(Storage != MTL::StorageModeManaged /* Managed memory cannot be safely suballocated! When you overwrite existing data the GPU buffer is immediately disposed of! */);
#endif

	if (bSupportsHeaps && Storage == MTL::StorageModePrivate)
	{
		MTL::HeapDescriptor* Desc = MTL::HeapDescriptor::alloc()->init();
        check(Desc);
        
		Desc->setSize(FullSize);
		Desc->setStorageMode(Storage);
		ParentHeap = NS::TransferPtr(Device.GetDevice()->newHeap(Desc));
        Desc->release();
		check(ParentHeap);
		//METAL_FATAL_ASSERT(ParentHeap, TEXT("Failed to create heap of size %u and resource options %u"), Size, (uint32)Options);
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocHeap(ParentHeap.get());
#endif
	}
	else
	{
        ParentBuffer = Device.GetDevice()->newBuffer(FullSize, Options);
		check(ParentBuffer);
		check(ParentBuffer->length() >= FullSize);
		//METAL_FATAL_ASSERT(ParentBuffer, TEXT("Failed to create heap of size %u and resource options %u"), Size, (uint32)Options);
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocBufferNative(ParentBuffer);
#endif
		
		INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, FullSize);
		INC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, FullSize);
        uint32 BlockCount = FullSize / ChunkSize;
        Blocks.AddZeroed(BlockCount);
	}
}

FMetalSubBufferMagazine::~FMetalSubBufferMagazine()
{
	if (ParentHeap)
	{
		DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, ParentHeap->size());
		DEC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, ParentHeap->size());
	}
	else
	{
        DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, ParentBuffer->length());
		DEC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, ParentBuffer->length());
		
		if(ParentBuffer)
		{
			MetalLLM::LogFreeBufferNative(ParentBuffer);
			
			if (GMetalResourcePurgeOnDelete)
			{
				ParentBuffer->setPurgeableState(MTL::PurgeableStateEmpty);
			}
			ParentBuffer->release();
			ParentBuffer = nullptr;
		}
	}
}

void FMetalSubBufferMagazine::FreeRange(NS::Range const& Range)
{
	FPlatformAtomics::InterlockedDecrement(&OutstandingAllocs);
	if (ParentHeap)
	{
		INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
		INC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, Range.length);
		DEC_MEMORY_STAT_BY(STAT_MetalMagazineBufferMemory, Range.length);
	}
	else
	{
#if METAL_DEBUG_OPTIONS
		if (GIsRHIInitialized)
		{
			Device.ValidateIsInactiveBuffer(ParentBuffer, Range);
		}
#endif
	
		uint32 BlockIndex = Range.location / Range.length;
		FPlatformAtomics::AtomicStore(&Blocks[BlockIndex], 0);
		FPlatformAtomics::InterlockedAdd(&UsedSize, -((int64)Range.length));
		
		INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
		INC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, Range.length);
		DEC_MEMORY_STAT_BY(STAT_MetalMagazineBufferMemory, Range.length);
	}
}

void FMetalSubBufferMagazine::ReleaseBuffer(FMetalBuffer* Buffer)
{
	if(ParentHeap)
	{
		if (GMetalResourcePurgeOnDelete)
		{
			Buffer->GetMTLBuffer()->setPurgeableState(MTL::PurgeableStateEmpty);
		}
		Buffer->GetMTLBuffer()->release();
	}
	FreeRange(Buffer->GetRange());
}

NS::String* FMetalSubBufferMagazine::GetLabel() const
{
	if (ParentHeap)
	{
		return ParentHeap->label();
	}
	else
	{
		return ParentBuffer->label();
	}
}

MTL::Device* FMetalSubBufferMagazine::GetDevice() const
{
	if (ParentHeap)
	{
		return ParentHeap->device();
	}
	else
	{
		return ParentBuffer->device();
	}
}

MTL::StorageMode FMetalSubBufferMagazine::GetStorageMode() const
{
	if (ParentHeap)
	{
		return ParentHeap->storageMode();
	}
	else
	{
		return ParentBuffer->storageMode();
	}
}

MTL::CPUCacheMode FMetalSubBufferMagazine::GetCpuCacheMode() const
{
	if (ParentHeap)
	{
		return ParentHeap->cpuCacheMode();
	}
	else
	{
		return ParentBuffer->cpuCacheMode();
	}
}

NS::UInteger FMetalSubBufferMagazine::GetSize() const
{
	if (ParentHeap)
	{
		return ParentHeap->size();
	}
	else
	{
		return ParentBuffer->length();
	}
}

NS::UInteger FMetalSubBufferMagazine::GetUsedSize() const
{
	if (ParentHeap)
	{
		return ParentHeap->usedSize();
	}
	else
	{
		return (NS::UInteger)FPlatformAtomics::AtomicRead(&UsedSize);
	}
}

NS::UInteger FMetalSubBufferMagazine::GetFreeSize() const
{
	if (ParentHeap)
	{
		return ParentHeap->maxAvailableSize(MinAlign);
	}
	else
	{
		return GetSize() - GetUsedSize();
	}
}

int64 FMetalSubBufferMagazine::NumCurrentAllocations() const
{
	return OutstandingAllocs;
}

bool FMetalSubBufferMagazine::CanAllocateSize(NS::UInteger Size) const
{
	return GetFreeSize() >= Size;
}

void FMetalSubBufferMagazine::SetLabel(const NS::String* label)
{
	if (ParentHeap)
	{
		ParentHeap->setLabel(label);
	}
	else
	{
		ParentBuffer->setLabel(label);
	}
}

FMetalBufferPtr FMetalSubBufferMagazine::NewBuffer()
{
	NS::UInteger Size = BlockSize;
	FMetalBufferPtr Result = nullptr;

	if (ParentHeap)
	{
		NS::UInteger Storage = (NS::UInteger(GetStorageMode()) << MTL::ResourceStorageModeShift);
		NS::UInteger Cache = (NS::UInteger(GetCpuCacheMode()) << MTL::ResourceCpuCacheModeShift);
		MTL::ResourceOptions Opt = MTL::ResourceOptions(Storage | Cache);
		
		Result = FMetalBufferPtr(new FMetalBuffer(ParentHeap->newBuffer(Size, Opt), NS::Range(0, Size), this));
		DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Result->GetLength());
		DEC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, Result->GetLength());
		INC_MEMORY_STAT_BY(STAT_MetalMagazineBufferMemory, Result->GetLength());
	}
	else
	{
		check(ParentBuffer);
		
		for (uint32 i = 0; i < Blocks.Num(); i++)
		{
			if (FPlatformAtomics::InterlockedCompareExchange(&Blocks[i], 1, 0) == 0)
			{
				NS::Range Range(i * BlockSize, BlockSize);
				FPlatformAtomics::InterlockedAdd(&UsedSize, ((int64)Range.length));
                DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Range.length);
				DEC_MEMORY_STAT_BY(STAT_MetalMagazineBufferUnusedMemory, Range.length);
				INC_MEMORY_STAT_BY(STAT_MetalMagazineBufferMemory, Range.length);
                
                Result = FMetalBufferPtr(new FMetalBuffer(ParentBuffer, Range, this));
				break;
			}
		}
	}

	FPlatformAtomics::InterlockedIncrement(&OutstandingAllocs);
	check(Result);
	return Result;
}

MTL::PurgeableState FMetalSubBufferMagazine::SetPurgeableState(MTL::PurgeableState state)
{
	if (ParentHeap)
	{
		return ParentHeap->setPurgeableState(state);
	}
	else
	{
		return ParentBuffer->setPurgeableState(state);
	}
}

FMetalRingBufferRef::FMetalRingBufferRef(FMetalDevice& InDevice, FMetalBufferPtr Buf)
: Device(InDevice)
, Buffer(Buf)
, LastRead(Buffer->GetLength())
{
	MTL_SCOPED_AUTORELEASE_POOL;
    Buffer->GetMTLBuffer()->setLabel(NS::String::string("Ring Buffer", NS::UTF8StringEncoding));
}

FMetalRingBufferRef::~FMetalRingBufferRef()
{
	FMetalDynamicRHI::Get().DeferredDelete(Buffer);
}

FMetalSubBufferRing::FMetalSubBufferRing(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger Alignment, MTL::ResourceOptions InOptions)
: Device(MetalDevice)
, LastFrameChange(0)
, InitialSize(Align(Size, Alignment))
, MinAlign(Alignment)
, CommitHead(0)
, SubmitHead(0)
, WriteHead(0)
, BufferSize(0)
, Options(InOptions)
, Storage((MTL::StorageMode)((Options & MTL::ResourceStorageModeMask) >> MTL::ResourceStorageModeShift))
{
	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	
	// If we are using heaps use shared storage
	if(bSupportsHeaps)
	{
		Storage = MTL::StorageModeShared;
		Options = (Options & ~MTL::ResourceStorageModeMask) | (MTL::ResourceStorageModeShared << MTL::ResourceStorageModeShift);
	}
	else
	{
		Options = (MTL::ResourceOptions)FMetalCommandQueue::GetCompatibleResourceOptions(Options);
	}
#if !defined(WITH_IOS_SIMULATOR) || !WITH_IOS_SIMULATOR
	check(Storage != MTL::StorageModePrivate /* Private memory requires command-buffers and encoders to properly marshal! */);
#endif
	FMemory::Memzero(FrameSize);
}
FMetalSubBufferRing::~FMetalSubBufferRing()
{
}

MTL::Device* FMetalSubBufferRing::GetDevice() const
{
	return RingBufferRef->GetMTLBuffer() ? RingBufferRef->GetMTLBuffer()->device() : nullptr;
}

MTL::StorageMode FMetalSubBufferRing::GetStorageMode() const
{
	return RingBufferRef->GetMTLBuffer() ? RingBufferRef->GetMTLBuffer()->storageMode() : Storage;
}

MTL::CPUCacheMode FMetalSubBufferRing::GetCpuCacheMode() const
{
	return RingBufferRef->GetMTLBuffer() ? RingBufferRef->GetMTLBuffer()->cpuCacheMode() : ((MTL::CPUCacheMode)((Options & MTL::ResourceCpuCacheModeMask) >> MTL::ResourceCpuCacheModeShift));
}

NS::UInteger FMetalSubBufferRing::GetSize() const
{
	return RingBufferRef->GetMTLBuffer() ? RingBufferRef->GetBuffer()->GetLength() : InitialSize;
}

FMetalBufferPtr FMetalSubBufferRing::NewBuffer(NS::UInteger Size, uint32 Alignment)
{
	if (Alignment == 0)
	{
		Alignment = MinAlign;
	}
	else
	{
		Alignment = Align(Alignment, MinAlign);
	}
	
	NS::UInteger FullSize = Align(Size, Alignment);
	
	// Allocate on first use
	if(!RingBufferRef.IsValid())
	{
        RingBufferRef = MakeShared<FMetalRingBufferRef, ESPMode::ThreadSafe>(Device, Device.GetResourceHeap().CreateBuffer(InitialSize, MinAlign, BUF_Dynamic, Options, true));
		BufferSize = InitialSize;
	}
	
	if(RingBufferRef->LastRead <= WriteHead)
	{
		if (WriteHead + FullSize <= RingBufferRef->GetBuffer()->GetLength())
		{
            FMetalBufferPtr NewBuffer = FMetalBufferPtr(new FMetalBuffer(RingBufferRef->GetMTLBuffer(), NS::Range(WriteHead, FullSize), this));
            
			FMemory::Memset(((uint8*)NewBuffer->Contents()), 0x0, FullSize);
			
			WriteHead += FullSize;
			return NewBuffer;
		}
#if PLATFORM_MAC
		else if (Storage == MTL::StorageModeManaged)
		{
			Submit();
            RingBufferRef = MakeShared<FMetalRingBufferRef, ESPMode::ThreadSafe>(Device, Device.GetResourceHeap().CreateBuffer(BufferSize, MinAlign, BUF_Dynamic, Options, true));
			WriteHead = 0;
			CommitHead = 0;
			SubmitHead = 0;
		}
#endif
		else
		{
			WriteHead = 0;
		}
	}
	
	if(WriteHead + FullSize >= RingBufferRef->LastRead || WriteHead + FullSize > BufferSize)
	{
		NS::UInteger NewBufferSize = AlignArbitrary(BufferSize + Size, Align(BufferSize / 4, MinAlign));
		
		UE_LOGF(LogMetal, Verbose, "Reallocating ring-buffer from %d to %d to avoid wrapping write at offset %d into outstanding buffer region %d at frame %lld]", (uint32)BufferSize, (uint32)NewBufferSize, (uint32)WriteHead, (uint32)RingBufferRef->LastRead, (uint64)GFrameCounter);
		
		Submit();
		
        RingBufferRef = MakeShared<FMetalRingBufferRef, ESPMode::ThreadSafe>(Device, Device.GetResourceHeap().CreateBuffer(NewBufferSize, MinAlign, BUF_Dynamic, Options, true));
		BufferSize = NewBufferSize;
		WriteHead = 0;
		CommitHead = 0;
		SubmitHead = 0;
	}
    
	{
        FMetalBufferPtr NewBuffer = FMetalBufferPtr(new FMetalBuffer(RingBufferRef->GetMTLBuffer(), NS::Range(WriteHead, FullSize), this));
		
		AllocatedRanges.Add(NS::Range(WriteHead, FullSize));
		
		FMemory::Memset(((uint8*)NewBuffer->Contents()), 0x0, FullSize);
		
		WriteHead += FullSize;
		// NewBuffer.MarkSingleUse();
		return NewBuffer;
	}
}

void FMetalSubBufferRing::Shrink()
{
	if(RingBufferRef.IsValid())
	{
		NS::UInteger FrameMax = 0;
		for (uint32 i = 0; i < UE_ARRAY_COUNT(FrameSize); i++)
		{
			FrameMax = FMath::Max(FrameMax, FrameSize[i]);
		}
		
		NS::UInteger NecessarySize = FMath::Max(FrameMax, InitialSize);
		NS::UInteger ThreeQuarterSize = Align((BufferSize / 4) * 3, MinAlign);
		
		if ((GFrameNumberRenderThread - LastFrameChange) >= 120 && NecessarySize < ThreeQuarterSize && NecessarySize < BufferSize)
		{
			Submit();
			
			UE_LOGF(LogMetal, Verbose, "Shrinking RingBuffer from %u to %u as max. usage is %u at frame %d]", (uint32)RingBufferRef->GetBuffer()->GetLength(), (uint32)ThreeQuarterSize, (uint32)FrameMax, GFrameNumberRenderThread);
			
            RingBufferRef = MakeShared<FMetalRingBufferRef, ESPMode::ThreadSafe>(Device, Device.GetResourceHeap().CreateBuffer(ThreeQuarterSize, MinAlign, BUF_Dynamic, Options, true));
			BufferSize = ThreeQuarterSize;
			WriteHead = 0;
			CommitHead = 0;
			SubmitHead = 0;
			LastFrameChange = GFrameNumberRenderThread;
		}
		
		FrameSize[GFrameNumberRenderThread % UE_ARRAY_COUNT(FrameSize)] = 0;
	}
}

void FMetalSubBufferRing::Submit()
{
	if (RingBufferRef.IsValid() && WriteHead != SubmitHead)
	{
#if PLATFORM_MAC
		if (Storage == MTL::StorageModeManaged)
		{
			check(SubmitHead < WriteHead);
			NS::Range ModifiedRange(SubmitHead, Align(WriteHead - SubmitHead, MinAlign));
            RingBufferRef->GetMTLBuffer()->didModifyRange(ModifiedRange);
		}
#endif

		SubmitHead = WriteHead;
	}
}

void FMetalSubBufferRing::Commit(FMetalCommandBuffer* CmdBuf)
{
	if (RingBufferRef.IsValid() && WriteHead != CommitHead)
	{
#if PLATFORM_MAC
		check(Storage != MTL::StorageModeManaged || CommitHead < WriteHead);
#endif
		Submit();
		
		NS::UInteger BytesWritten = 0;
		if (CommitHead <= WriteHead)
		{
			BytesWritten = WriteHead - CommitHead;
		}
		else
		{
			NS::UInteger TrailLen = GetSize() - CommitHead;
			BytesWritten = TrailLen + WriteHead;
		}
		
		FrameSize[GFrameNumberRenderThread % UE_ARRAY_COUNT(FrameSize)] += Align(BytesWritten, MinAlign);
		
		TSharedPtr<FMetalRingBufferRef, ESPMode::ThreadSafe> CmdBufferRingBuffer = RingBufferRef;
		FPlatformMisc::MemoryBarrier();
		
		NS::UInteger CommitOffset = CommitHead;
		NS::UInteger WriteOffset = WriteHead;
		
		CommitHead = WriteHead;
		
		TArray<NS::Range> Ranges = MoveTemp(AllocatedRanges);
		
		MTL::HandlerFunction Handler = [CmdBufferRingBuffer, CommitOffset, WriteOffset, Ranges](MTL::CommandBuffer* InBuffer)
		{
#if METAL_DEBUG_OPTIONS
			if (GMetalBufferScribble && CommitOffset != WriteOffset)
			{
				if (CommitOffset < WriteOffset)
				{
					FMemory::Memset(((uint8*)CmdBufferRingBuffer->GetBuffer()->Contents()) + CommitOffset, 0xCD, WriteOffset - CommitOffset);
				}
				else
				{
					uint32 TrailLen = CmdBufferRingBuffer->GetBuffer()->GetLength() - CommitOffset;
					FMemory::Memset(((uint8*)CmdBufferRingBuffer->GetBuffer()->Contents()) + CommitOffset, 0xCD, TrailLen);
					FMemory::Memset(((uint8*)CmdBufferRingBuffer->GetBuffer()->Contents()), 0xCD, WriteOffset);
				}
			}
#endif
			CmdBufferRingBuffer->SetLastRead(WriteOffset);
		};
		CmdBuf->GetMTLCmdBuffer()->addCompletedHandler(Handler);
	}
}

uint32 FMetalBufferPoolPolicyData::GetPoolBucketIndex(CreationArguments Args)
{
	uint32 Size = Args.Size;
	
	unsigned long Lower = 0;
	unsigned long Upper = NumPoolBucketSizes;
	unsigned long Middle;
	
	do
	{
		Middle = ( Upper + Lower ) >> 1;
		if( Size <= BucketSizes[Middle-1] )
		{
			Upper = Middle;
		}
		else
		{
			Lower = Middle;
		}
	}
	while( Upper - Lower > 1 );
	
	check( Size <= BucketSizes[Lower] );
	check( (Lower == 0 ) || ( Size > BucketSizes[Lower-1] ) );
	
	return (int32)Args.Storage * NumPoolBucketSizes + Lower;
}

uint32 FMetalBufferPoolPolicyData::GetPoolBucketSize(uint32 Bucket)
{
	check(Bucket < NumPoolBuckets);
	uint32 Index = Bucket % NumPoolBucketSizes;
	return BucketSizes[Index];
}

TMap<MTL::Buffer*, FMetalBufferPoolPolicyData::CreationArguments> FMetalBufferPoolPolicyData::CreationArgumentMap;

MTL::Buffer* FMetalBufferPoolPolicyData::CreateResource(FRHICommandListBase&, CreationArguments Args)
{
	check(Args.Device);	
	uint32 BufferSize = GetPoolBucketSize(GetPoolBucketIndex(Args));
	
	NS::UInteger CpuCacheMode = (NS::UInteger)Args.CpuCacheMode << MTL::ResourceCpuCacheModeShift;
	NS::UInteger StorageMode = (NS::UInteger)Args.Storage << MTL::ResourceStorageModeShift;
	MTL::ResourceOptions ResourceOptions = FMetalCommandQueue::GetCompatibleResourceOptions(MTL::ResourceOptions(CpuCacheMode | StorageMode | MTL::ResourceHazardTrackingModeUntracked));
	
	MTL::Buffer* NewBuf = Args.Device->GetDevice()->newBuffer(BufferSize, ResourceOptions);

	check(!CreationArgumentMap.Contains(NewBuf));
	CreationArgumentMap.Add(NewBuf, Args);
	
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
	MetalLLM::LogAllocBufferNative(NewBuf);
#endif
	INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, NewBuf->length());
	INC_MEMORY_STAT_BY(STAT_MetalPooledBufferUnusedMemory, NewBuf->length());
	return NewBuf;
}

FMetalBufferPoolPolicyData::CreationArguments FMetalBufferPoolPolicyData::GetCreationArguments(MTL::Buffer* Resource)
{
	FMetalBufferPoolPolicyData::CreationArguments& Args = CreationArgumentMap.FindChecked(Resource); 
	return Args;
}

void FMetalBufferPoolPolicyData::FreeResource(MTL::Buffer* Resource)
{
	DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Resource->length());
	DEC_MEMORY_STAT_BY(STAT_MetalPooledBufferUnusedMemory, Resource->length());
	
	CreationArgumentMap.Remove(Resource);
	
	MetalLLM::LogFreeBufferNative(Resource);
	if (GMetalResourcePurgeOnDelete)
	{
		Resource->setPurgeableState(MTL::PurgeableStateEmpty);
	}
	Resource->release();
}

FMetalTexturePool::FMetalTexturePool(FCriticalSection& InPoolMutex)
: PoolMutex(InPoolMutex)
{
}

FMetalTexturePool::~FMetalTexturePool()
{
}

MTLTexturePtr FMetalTexturePool::CreateTexture(FMetalDevice& Device, MTL::TextureDescriptor* Desc)
{
	FMetalTexturePool::Descriptor Descriptor;
	Descriptor.textureType = (NS::UInteger)Desc->textureType();
	Descriptor.pixelFormat = (NS::UInteger)Desc->pixelFormat();
	Descriptor.width = Desc->width();
	Descriptor.height = Desc->height();
	Descriptor.depth = Desc->depth();
	Descriptor.mipmapLevelCount = Desc->mipmapLevelCount();
	Descriptor.sampleCount = Desc->sampleCount();
	Descriptor.arrayLength = Desc->arrayLength();
	Descriptor.resourceOptions = Desc->resourceOptions();
	Descriptor.usage = Desc->usage();

	check(Descriptor.usage != MTL::TextureUsageUnknown);
	
	Descriptor.freedFrame = 0;

	FScopeLock Lock(&PoolMutex);
	MTLTexturePtr* Tex = Pool.Find(Descriptor);
    MTLTexturePtr Texture;
	if (Tex)
	{
		Texture = *Tex;
		Pool.Remove(Descriptor);
		if (GMetalResourcePurgeInPool)
		{
            Texture->setPurgeableState(MTL::PurgeableStateNonVolatile);
        }
	}
	else
	{
        Texture = NS::TransferPtr(Device.GetDevice()->newTexture(Desc));
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocTexture(Texture.get());
#endif
	}
	return Texture;
}

void FMetalTexturePool::ReleaseTexture(MTLTexturePtr Texture)
{
	FMetalTexturePool::Descriptor Descriptor;
	Descriptor.textureType = (NS::UInteger)Texture->textureType();
	Descriptor.pixelFormat = (NS::UInteger)Texture->pixelFormat();
	Descriptor.width = Texture->width();
	Descriptor.height = Texture->height();
	Descriptor.depth = Texture->depth();
	Descriptor.mipmapLevelCount = Texture->mipmapLevelCount();
	Descriptor.sampleCount = Texture->sampleCount();
	Descriptor.arrayLength = Texture->arrayLength();
	Descriptor.resourceOptions = ((NS::UInteger)Texture->storageMode() << MTL::ResourceStorageModeShift) | ((NS::UInteger)Texture->cpuCacheMode() << MTL::ResourceCpuCacheModeShift);
	Descriptor.usage = Texture->usage();
	Descriptor.freedFrame = GFrameNumberRenderThread;
	
    if (GMetalResourcePurgeInPool && Texture->setPurgeableState(MTL::PurgeableStateKeepCurrent) == MTL::PurgeableStateNonVolatile)
    {
        Texture->setPurgeableState(MTL::PurgeableStateVolatile);
    }
	
	FScopeLock Lock(&PoolMutex);
	
	if(Pool.Find(Descriptor))
	{
		MetalLLM::LogFreeTexture(Texture.get());
		if (GMetalResourcePurgeOnDelete)
		{
			Texture->setPurgeableState(MTL::PurgeableStateEmpty);
		}
	}
	else
	{
		Pool.Add(Descriptor, Texture);
	}
}
	
void FMetalTexturePool::Drain(bool const bForce)
{
	for (auto It = Pool.CreateIterator(); It; ++It)
	{
		if (bForce || (GFrameNumberRenderThread - It->Key.freedFrame) >= CullAfterNumFrames)
		{
			MetalLLM::LogFreeTexture(It->Value.get());
			if (GMetalResourcePurgeOnDelete)
			{
				It->Value->setPurgeableState(MTL::PurgeableStateEmpty);
			}
			It.RemoveCurrent();
		}
		else
		{
			if (GMetalResourcePurgeInPool && (GFrameNumberRenderThread - It->Key.freedFrame) >= PurgeAfterNumFrames)
			{
				It->Value->setPurgeableState(MTL::PurgeableStateEmpty);
			}
		}
	}
}

FMetalResourceHeap::FMetalResourceHeap(FMetalDevice& MetalDevice)
: Device(MetalDevice)
, Queue(nullptr)
, TexturePool(Mutex)
, TargetPool(Mutex)
{
}

FMetalResourceHeap::~FMetalResourceHeap()
{
	Compact(true);
}

void FMetalResourceHeap::Init(FMetalCommandQueue& InQueue)
{
	Queue = &InQueue;
}

uint32 FMetalResourceHeap::GetMagazineIndex(uint32 Size)
{
	unsigned long Lower = 0;
	unsigned long Upper = NumMagazineSizes;
	unsigned long Middle;
	
	do
	{
		Middle = ( Upper + Lower ) >> 1;
		if( Size <= MagazineSizes[Middle-1] )
		{
			Upper = Middle;
		}
		else
		{
			Lower = Middle;
		}
	}
	while( Upper - Lower > 1 );
	
	check( Size <= MagazineSizes[Lower] );
	check( (Lower == 0 ) || ( Size > MagazineSizes[Lower-1] ) );
	
	return Lower;
}

uint32 FMetalResourceHeap::GetHeapIndex(uint32 Size)
{
	unsigned long Lower = 0;
	unsigned long Upper = NumHeapSizes;
	unsigned long Middle;
	
	do
	{
		Middle = ( Upper + Lower ) >> 1;
		if( Size <= HeapSizes[Middle-1] )
		{
			Upper = Middle;
		}
		else
		{
			Lower = Middle;
		}
	}
	while( Upper - Lower > 1 );
	
	check( Size <= HeapSizes[Lower] );
	check( (Lower == 0 ) || ( Size > HeapSizes[Lower-1] ) );
	
	return Lower;
}

FMetalResourceHeap::TextureHeapSize FMetalResourceHeap::TextureSizeToIndex(uint32 Size)
{
	unsigned long Lower = 0;
	unsigned long Upper = NumTextureHeapSizes;
	unsigned long Middle;
	
	do
	{
		Middle = ( Upper + Lower ) >> 1;
		if( Size <= (HeapTextureHeapSizes[Middle-1] / MinTexturesPerHeap) )
		{
			Upper = Middle;
		}
		else
		{
			Lower = Middle;
		}
	}
	while( Upper - Lower > 1 );
	
	check( Size <= (HeapTextureHeapSizes[Lower] / MinTexturesPerHeap) );
	check( (Lower == 0 ) || ( Size > (HeapTextureHeapSizes[Lower-1] / MinTexturesPerHeap) ) );
	
	return (TextureHeapSize)Lower;
}

MTLHeapPtr FMetalResourceHeap::GetTextureHeap(MTL::TextureDescriptor* Desc, MTL::SizeAndAlign Size)
{
    MTLHeapPtr Result;
    
	static bool bTextureHeaps = FParse::Param(FCommandLine::Get(),TEXT("metaltextureheaps"));
	if (Device.SupportsFeature(EMetalFeaturesHeaps) && bTextureHeaps && Size.size <= HeapTextureHeapSizes[MaxTextureSize])
	{
		FMetalResourceHeap::TextureHeapSize HeapIndex = TextureSizeToIndex(Size.size);

		EMetalHeapTextureUsage UsageIndex = EMetalHeapTextureUsageNum;
		MTL::StorageMode StorageMode = Desc->storageMode();
        MTL::CPUCacheMode CPUMode = Desc->cpuCacheMode();
		if ((Desc->usage() & MTL::TextureUsageRenderTarget) && StorageMode == MTL::StorageModePrivate && CPUMode == MTL::CPUCacheModeDefaultCache)
		{
			UsageIndex = PLATFORM_MAC ? EMetalHeapTextureUsageNum : EMetalHeapTextureUsageRenderTarget;
		}
		else if (StorageMode == MTL::StorageModePrivate && CPUMode == MTL::CPUCacheModeWriteCombined)
		{
			UsageIndex = EMetalHeapTextureUsageResource;
		}
		
		if (UsageIndex < EMetalHeapTextureUsageNum)
		{
			for (MTLHeapPtr & Heap : TextureHeaps[UsageIndex][HeapIndex])
			{
				if (Heap->maxAvailableSize(Size.align) >= Size.size)
				{
					Result = Heap;
					break;
				}
			}
			if (!Result)
			{
                MTL::HeapDescriptor* HeapDesc = MTL::HeapDescriptor::alloc()->init();
                check(HeapDesc);
                
				HeapDesc->setSize(HeapTextureHeapSizes[HeapIndex]);
				HeapDesc->setStorageMode(Desc->storageMode());
				HeapDesc->setCpuCacheMode(Desc->cpuCacheMode());
				Result = NS::TransferPtr(Device.GetDevice()->newHeap(HeapDesc));
                HeapDesc->release();
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
				MetalLLM::LogAllocHeap(Result.get());
#endif
				TextureHeaps[UsageIndex][HeapIndex].Add(Result);
			}
			check(Result);
		}
	}
	return Result;
}

TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator FMetalResourceHeap::SplitBlock(TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>& List, TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator BlockIt, const uint64 Offset, const uint32 Size)
{
	MemoryBlock& Block = *BlockIt;
	check(Size < Block.Size && Block.Resource == nil);
	check(Offset >= Block.Offset);

	if (Offset > Block.Offset)
	{
		uint64 PreBlockSize = Offset - Block.Offset;

		MemoryBlock PreBlock;
		PreBlock.Heap = Block.Heap;
		PreBlock.Offset = Block.Offset;
		PreBlock.Size = PreBlockSize;
		PreBlock.Resource = nil;
		PreBlock.Options = MTL::ResourceOptions(0);

		// Move the block at the real offset
		Block.Offset += PreBlockSize;
		Block.Size   -= PreBlockSize;

		List.InsertNode(PreBlock, BlockIt.GetNode());
	}
	check(Block.Size >= Size);

	// If we have space left after the allocation split the leftover space into its own free block
	TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator MidBlockIt = BlockIt;
	if (Block.Size > Size)
	{
		MemoryBlock PostBlock;
		PostBlock.Heap = Block.Heap;
		PostBlock.Offset = Block.Offset + Size;
		PostBlock.Size = Block.Size - Size;
		PostBlock.Resource = nil;
		PostBlock.Options = MTL::ResourceOptions(0);

		// Shrink the current block to size
		Block.Size = Size;

		// Insert the new block *after* the existing one
		List.InsertNode(PostBlock, BlockIt.GetNode());
		BlockIt++;
	}
	check(Block.Size == Size);

	// Return the block with the required size first
	return MidBlockIt;
}

TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator FMetalResourceHeap::MergeBlocks(TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>& List,
																							  TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator BlockItA,
																							  TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator BlockItB)
{	
	MemoryBlock& BlockA = *BlockItA;
	MemoryBlock& BlockB = *BlockItB;

	// Extend block A to cover the range of block B as well
	BlockA.Size += BlockB.Size;

	// Remove B from the list
	List.RemoveNode(BlockItB.GetNode());

	return BlockItA;
}

#if PLATFORM_MAC
static constexpr uint32 METALHEAP_DEFAULT_BLOCKSIZE = 1024 << 19; // 512mb
#else
// Use a lower block size on iOS to reduce memory pressure.
static constexpr uint32 METALHEAP_DEFAULT_BLOCKSIZE = 1024 << 17; // 128mb
#endif

void FMetalResourceHeap::FreeBlock(const uint32 ResourceAllocationHandle)
{
	FScopeLock ScopeLock(&FreeListCS);

	auto BlockIt = InUseResources[ResourceAllocationHandle];
	MemoryBlock& AllocatedBlock = *BlockIt;
	check(AllocatedBlock.Resource != nullptr);
	// AllocatedBlock.Resource->release(); // Will be released once the resource dtor is called (otherwise we may end up double-releasing).
	AllocatedBlock.Resource = nullptr;
	{
		FScopeLock InUseFreeListScopeLock(&InUseResourcesCS);
		InUseResourcesFreeList.Enqueue(ResourceAllocationHandle);
	}

	auto& FreeList = FreeLists[AllocatedBlock.Options];
	auto& UsedList = UsedLists[AllocatedBlock.Options];

	// Find where this block should be placed in the list
	TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator NextBlockIt(FreeList->GetHead());
	while (NextBlockIt
	   && ((*NextBlockIt).Heap != AllocatedBlock.Heap || (*NextBlockIt).Offset < AllocatedBlock.Offset + AllocatedBlock.Size))
	{
		NextBlockIt++;
	}
	
	// Put the block into the free list	
	UsedList->RemoveNode(BlockIt.GetNode(), false);
	FreeList->InsertNode(BlockIt.GetNode(), NextBlockIt ? NextBlockIt.GetNode() : nullptr);
	
	INC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, (*BlockIt).Size);

	// If we have a next block then attempt to merge blocks
	if(NextBlockIt)
	{
		BlockIt = NextBlockIt;
		if (BlockIt.GetNode() == nullptr)
		{
			BlockIt = TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator(FreeList->GetHead());
		}
		else
		{
			BlockIt--;
		}
		check((*BlockIt).Offset == AllocatedBlock.Offset); // Temp sanity check; to be removed.
		
		// Figure out the earliest block we can merge the newly freed block with
		{
			auto PrevBlockIt = BlockIt;
			PrevBlockIt--;
			while (BlockIt
				   && PrevBlockIt
				   && (*BlockIt).Heap == (*PrevBlockIt).Heap
				   && (*BlockIt).Offset == (*PrevBlockIt).Offset + (*PrevBlockIt).Size)
			{
				BlockIt = PrevBlockIt;
				PrevBlockIt--;
			}
		}
		
		// Merge all adjacent allocations to minimise fragmentation
		{
			auto NextIt = BlockIt;
			NextIt++;
			
			while (NextIt)
			{
				MemoryBlock& BlockA = *BlockIt;
				MemoryBlock& BlockB = *NextIt;
				
				if (BlockA.Heap == BlockB.Heap
					&& (BlockA.Offset + BlockA.Size) == BlockB.Offset)
				{
					BlockIt = MergeBlocks(*FreeList, BlockIt, NextIt);
					
					// NOTE: We don't increment 'blockIt' the next block in the list might also be mergeable into the newly merged block
					// We merged A and B, nextIt is invalidated so we need to re-initialise it from blockIt++
					NextIt = BlockIt;
					NextIt++;
				}
				else
				{
					break;
				}
			}
		}
	}
	
	// If heap is completely deallocated then remove it
	MemoryBlock& CurrentBlock = *BlockIt;
	
	if(CurrentBlock.Size == CurrentBlock.Heap->size())
	{
		FMetalDynamicRHI::Get().DeferredDelete([HeapPtr = CurrentBlock.Heap](){});
		CurrentBlock.Heap = nullptr;
		FreeList->RemoveNode(BlockIt.GetNode(), false);
	}
}

TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator FMetalResourceHeap::FindOrAllocateBlock(uint32 Size, uint32 Alignment, MTL::ResourceOptions Options)
{
	FScopeLock ScopeLock(&FreeListCS);

	TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>** FreeListIt = FreeLists.Find(Options);
	if (!FreeListIt)
	{
		FreeLists.Add(Options, new TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>());
		UsedLists.Add(Options, new TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>());
	}

	auto& FreeList = FreeLists[Options];
	auto& UsedList = UsedLists[Options];

	// Look for the first existing block with enough space
	if (!FreeList->IsEmpty())
	{
		for (TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator It(FreeList->GetHead()); It; ++It)
		{
			FMetalResourceHeap::MemoryBlock& Block = *It;
			check(Block.Resource == nil);

			uint64 AlignedOffset = Align(Block.Offset, Alignment);
			uint64 WastedSpace = AlignedOffset - Block.Offset;

			if (Block.Size >= (Size + WastedSpace))
			{
				// If the block doesn't fit perfectly we need to split
				if (Block.Size > Size)
				{
					auto MidIt = SplitBlock(*FreeList, It, AlignedOffset, Size);
					It = MidIt;
				}

				// Transfert node from free to used list
				FreeList->RemoveNode(It.GetNode(), false);
				UsedList->AddHead(It.GetNode());

				DEC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, (*It).Size);
				
				return It;
			}
		}
	}
	
	{
		uint64 NewBlockSize = FMath::Max(Size, METALHEAP_DEFAULT_BLOCKSIZE);

		MTL::StorageMode StorageMode = (MTL::StorageMode)(((NS::UInteger)Options & MTL::ResourceStorageModeMask) >> MTL::ResourceStorageModeShift);
		MTL::CPUCacheMode CpuMode = (MTL::CPUCacheMode)(((NS::UInteger)Options & MTL::ResourceCpuCacheModeMask) >> MTL::ResourceCpuCacheModeShift);

		MTL::HeapDescriptor* HeapDesc = MTL::HeapDescriptor::alloc()->init();
		check(HeapDesc);
		HeapDesc->setSize(NewBlockSize);
		HeapDesc->setStorageMode(StorageMode);
		HeapDesc->setCpuCacheMode(CpuMode);
		HeapDesc->setType(MTL::HeapTypePlacement);
		HeapDesc->setHazardTrackingMode(MTL::HazardTrackingModeTracked);

		MTLHeapPtr BlockHeap = NS::TransferPtr(Device.GetDevice()->newHeap(HeapDesc));
		HeapDesc->release();

		MemoryBlock NewBlock;
		NewBlock.Heap = BlockHeap;
		NewBlock.Offset = 0;
		NewBlock.Size = BlockHeap->size();
		NewBlock.Resource = nullptr;
		NewBlock.Options = Options;

		FreeList->AddTail(NewBlock);
		
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocHeap(BlockHeap.get());
#endif
		INC_MEMORY_STAT_BY(STAT_MetalHeapBufferMemory, NewBlockSize);
		INC_MEMORY_STAT_BY(STAT_MetalHeapBufferUnusedMemory, NewBlockSize);

		// Try again, but this time a fitting block will be immediately available at the front of the list
		return FindOrAllocateBlock(Size, Alignment, Options);
	}
}

FMetalBufferPtr FMetalResourceHeap::CreateBuffer(uint32 Size, uint32 Alignment, EBufferUsageFlags Flags, MTL::ResourceOptions Options, bool bForceUnique)
{
	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	if (bSupportsHeaps)
	{
		MTL::SizeAndAlign BufferSizeAndAlign = Device.GetDevice()->heapBufferSizeAndAlign(Size, Options);
		Alignment = FMath::Max((uint32)BufferSizeAndAlign.align, Alignment);

		uint32 BlockSize = Align(Size, Alignment);
		auto BlockIt = FindOrAllocateBlock(BlockSize, Alignment, Options);
		MemoryBlock& Block = *BlockIt;

		check(Block.Resource == nil && Block.Size == BlockSize);
		Block.Options = Options;

		FScopeLock ScopeLock(&InUseResourcesCS);
		
		uint32 HeapAllocationHandle = UINT32_MAX;
		{
			if (!InUseResourcesFreeList.IsEmpty())
			{
				InUseResourcesFreeList.Dequeue(HeapAllocationHandle);
				InUseResources[HeapAllocationHandle] = BlockIt;
			}
			else
			{
				HeapAllocationHandle = InUseResources.Add(BlockIt);
			}
		}

		FMetalBufferPtr Buffer = FMetalBufferPtr(new FMetalBuffer(Block.Heap->newBuffer(Size, Options, Block.Offset), NS::Range(0, Size), this));
		check(Buffer->GetMTLBuffer() && Buffer->GetMTLBuffer());
		AllocationHandlesLUT.Add(Buffer->GetMTLBuffer(), HeapAllocationHandle);

		Block.Resource = Buffer->GetMTLBuffer();

		return Buffer;
	}
	
	static bool bSupportsBufferSubAllocation = Device.SupportsFeature(EMetalFeaturesBufferSubAllocation);
	bForceUnique |= (!bSupportsBufferSubAllocation && !bSupportsHeaps);
	
	uint32 Usage = EnumHasAnyFlags(Flags, BUF_Static) ? UsageStatic : UsageDynamic;
	
	FMetalBufferPtr Buffer = nullptr;
	uint32 BlockSize = Align(Size, Alignment);
	MTL::StorageMode StorageMode = (MTL::StorageMode)(((NS::UInteger)Options & MTL::ResourceStorageModeMask) >> MTL::ResourceStorageModeShift);
	MTL::CPUCacheMode CpuMode = (MTL::CPUCacheMode)(((NS::UInteger)Options & MTL::ResourceCpuCacheModeMask) >> MTL::ResourceCpuCacheModeShift);
	
	// Write combined should be on a case by case basis
	check(CpuMode == MTL::CPUCacheModeDefaultCache);
	
	check(GMetalMaxBucketSize <= 32 * 1024 * 1024);
	if (BlockSize <= GMetalMaxBucketSize)
	{
		switch (StorageMode)
		{
	#if PLATFORM_MAC
			case MTL::StorageModeManaged:
			{
				// TextureBuffers must be 1024 aligned.
				check(Alignment == 256 || Alignment == 1024);
				FScopeLock Lock(&Mutex);

				// Disabled Managed sub-allocation as it seems inexplicably slow on the GPU				
				 if (!bForceUnique && BlockSize <= HeapSizes[NumHeapSizes - 1])
				 {
				 	FMetalSubBufferLinear* Found = nullptr;
				 	for (FMetalSubBufferLinear* Heap : ManagedSubHeaps)
				 	{
				 		if (Heap->CanAllocateSize(BlockSize))
				 		{
				 			Found = Heap;
				 			break;
				 		}
				 	}
				 	if (!Found)
				 	{
				 		Found = new FMetalSubBufferLinear(Device, HeapAllocSizes[NumHeapSizes - 1], BufferOffsetAlignment, MTL::ResourceOptions((NS::UInteger)Options & (MTL::ResourceStorageModeMask|MTL::ResourceHazardTrackingModeMask)), Mutex);
				 		ManagedSubHeaps.Add(Found);
				 	}
				 	check(Found);
					
				 	return Found->NewBuffer(BlockSize);
				 }
				 else
				 {
					MTL::Buffer* MTLBuffer = ManagedBuffers.CreatePooledResource(FRHICommandListExecutor::GetImmediateCommandList(), FMetalPooledBufferArgs(&Device, BlockSize, Flags, StorageMode, CpuMode));
                    Buffer = FMetalBufferPtr(new FMetalBuffer(MTLBuffer, NS::Range(0, MTLBuffer->length()), this));
					 
					if (GMetalResourcePurgeInPool)
					{
						Buffer->GetMTLBuffer()->setPurgeableState(MTL::PurgeableStateNonVolatile);
					}
					DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Buffer->GetLength());
					DEC_MEMORY_STAT_BY(STAT_MetalPooledBufferUnusedMemory, Buffer->GetLength());
					INC_MEMORY_STAT_BY(STAT_MetalPooledBufferMemory, Buffer->GetLength());
				}
				break;
			}
	#endif
			case MTL::StorageModePrivate:
			case MTL::StorageModeShared:
			{
				AllocTypes Storage = StorageMode != MTL::StorageModePrivate ? AllocShared : AllocPrivate;
				check(Alignment == 16 || Alignment == 64 || Alignment == 256 || Alignment == 1024);

				static bool bSupportsPrivateBufferSubAllocation = Device.SupportsFeature(EMetalFeaturesPrivateBufferSubAllocation);
				if (!bForceUnique && BlockSize <= MagazineSizes[NumMagazineSizes - 1] && (Storage == AllocShared || bSupportsPrivateBufferSubAllocation))
				{
					FScopeLock Lock(&Mutex);
					
					uint32 i = GetMagazineIndex(BlockSize);
					TArray<FMetalSubBufferMagazine*>& Heaps = SmallBuffers[Usage][Storage][i];
					
					FMetalSubBufferMagazine* Found = nullptr;
					for (FMetalSubBufferMagazine* Heap : Heaps)
					{
						if (Heap->CanAllocateSize(BlockSize))
						{
							Found = Heap;
							break;
						}
					}
					
					if (!Found)
					{
						Found = new FMetalSubBufferMagazine(Device, MagazineAllocSizes[i], MagazineSizes[i], MTL::ResourceOptions((NS::UInteger)Options & (MTL::ResourceStorageModeMask|MTL::ResourceHazardTrackingModeMask)));
						SmallBuffers[Usage][Storage][i].Add(Found);
					}
					check(Found);
					
					Buffer = Found->NewBuffer();
					check(Buffer);
				}
				else if (!bForceUnique && BlockSize <= HeapSizes[NumHeapSizes - 1] && (Storage == AllocShared || bSupportsPrivateBufferSubAllocation))
				{
					FScopeLock Lock(&Mutex);
					
					uint32 i = GetHeapIndex(BlockSize);
					TArray<FMetalSubBufferHeap*>& Heaps = BufferHeaps[Usage][Storage][i];
					
					FMetalSubBufferHeap* Found = nullptr;
					for (FMetalSubBufferHeap* Heap : Heaps)
					{
						if (Heap->CanAllocateSize(BlockSize))
						{
							Found = Heap;
							break;
						}
					}
					
					if (!Found)
					{
						uint32 MinAlign = PLATFORM_MAC ? 1024 : 64;
						Found = new FMetalSubBufferHeap(Device, HeapAllocSizes[i], MinAlign, MTL::ResourceOptions((NS::UInteger)Options & (MTL::ResourceStorageModeMask|MTL::ResourceHazardTrackingModeMask)), Mutex);
						BufferHeaps[Usage][Storage][i].Add(Found);
					}
					check(Found);
					
					Buffer = Found->NewBuffer(BlockSize);
					check(Buffer);
				}
				else
				{
					FScopeLock Lock(&Mutex);
					MTL::Buffer* MTLBuffer = Buffers[Storage].CreatePooledResource(FRHICommandListExecutor::GetImmediateCommandList(), FMetalPooledBufferArgs(&Device, BlockSize, Flags, StorageMode, CpuMode));
					Buffer = FMetalBufferPtr(new FMetalBuffer(MTLBuffer, NS::Range(0, MTLBuffer->length()), this));
					
					if (GMetalResourcePurgeInPool)
					{
                   		Buffer->GetMTLBuffer()->setPurgeableState(MTL::PurgeableStateNonVolatile);
					}
					DEC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Buffer->GetLength());
					DEC_MEMORY_STAT_BY(STAT_MetalPooledBufferUnusedMemory, Buffer->GetLength());
					INC_MEMORY_STAT_BY(STAT_MetalPooledBufferMemory, Buffer->GetLength());
				}
				break;
			}
			default:
			{
				check(false);
				break;
			}
		}
	}
	else
	{
        Buffer = FMetalBufferPtr(new FMetalBuffer(Device.GetDevice()->newBuffer(BlockSize, Options), FMetalBuffer::FreePolicy::Owner));
	}
	
    METAL_DEBUG_OPTION(Device.ValidateIsInactiveBuffer(Buffer->GetMTLBuffer(), Buffer->GetRange()));
	METAL_FATAL_ASSERT(Buffer, TEXT("Failed to create buffer of size %u and resource options %u"), Size, (uint32)Options);
	return Buffer;
}

void FMetalResourceHeap::ReleaseBuffer(FMetalBuffer* Buffer)
{
    MTL::Buffer* MTLBuffer = Buffer->GetMTLBuffer();
	MTL::StorageMode StorageMode = MTLBuffer->storageMode();
	
	FScopeLock ScopeLock(&InUseResourcesCS);
	
	auto It = AllocationHandlesLUT.Find(MTLBuffer);
	if (It && *It != UINT32_MAX)
	{
		FreeBlock(*It);
		AllocationHandlesLUT.Remove(MTLBuffer);
		
		static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
		if (GMetalResourcePurgeOnDelete && !bSupportsHeaps)
		{
			MTLBuffer->setPurgeableState(MTL::PurgeableStateEmpty);
		}
		MTLBuffer->release();
	}
	else
	{
		FScopeLock Lock(&Mutex);
		
		INC_MEMORY_STAT_BY(STAT_MetalBufferUnusedMemory, Buffer->GetLength());
		INC_MEMORY_STAT_BY(STAT_MetalPooledBufferUnusedMemory, Buffer->GetLength());
		DEC_MEMORY_STAT_BY(STAT_MetalPooledBufferMemory, Buffer->GetLength());
		
		if (GMetalResourcePurgeInPool)
		{
			MTLBuffer->setPurgeableState(MTL::PurgeableStateVolatile);
		}
        
		switch (StorageMode)
		{
	#if PLATFORM_MAC
			case MTL::StorageModeManaged:
			{
				ManagedBuffers.ReleasePooledResource(Buffer->GetMTLBuffer());
				break;
			}
	#endif
			case MTL::StorageModePrivate:
			case MTL::StorageModeShared:
			{
				Buffers[(NS::UInteger)StorageMode].ReleasePooledResource(Buffer->GetMTLBuffer());
				break;
			}
			default:
			{
				check(false);
				break;
			}
		}
	}
}

MTLTexturePtr FMetalResourceHeap::CreateTexture(MTL::TextureDescriptor* Desc, FMetalSurface* Surface)
{
	LLM_SCOPE_METAL(ELLMTagMetal::Textures);
	LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::Textures);

	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	if (bSupportsHeaps)
	{
		MTL::SizeAndAlign SizeAndAlign = Device.GetDevice()->heapTextureSizeAndAlign(Desc);
		uint32 Size = SizeAndAlign.size;
		uint32 Alignment = SizeAndAlign.align;
		uint32 BlockSize = Align(Size, Alignment);
		auto BlockIt = FindOrAllocateBlock(BlockSize, Alignment, Desc->resourceOptions());
		MemoryBlock& Block = *BlockIt;
	
		check(Block.Resource == nil && Block.Size == BlockSize);
		Block.Options = Desc->resourceOptions();

		FScopeLock ScopeLock(&InUseResourcesCS);
		
		uint32 HeapAllocationHandle = UINT32_MAX;
		{
			if (!InUseResourcesFreeList.IsEmpty())
			{
				InUseResourcesFreeList.Dequeue(HeapAllocationHandle);
				InUseResources[HeapAllocationHandle] = BlockIt;
			}
			else
			{
				HeapAllocationHandle = InUseResources.Add(BlockIt);
			}
		}
		check(HeapAllocationHandle != UINT32_MAX);

		MTLTexturePtr Texture = NS::TransferPtr(Block.Heap->newTexture(Desc,Block.Offset));
		AllocationHandlesLUT.Add(Texture.get(), HeapAllocationHandle);

		Block.Resource = Texture.get();

		return Texture;
	}
		
	MTL::SizeAndAlign Res = Device.GetDevice()->heapTextureSizeAndAlign(Desc);
    MTLHeapPtr Heap = GetTextureHeap(Desc, Res);
	if (Heap)
	{
		MTLTexturePtr Texture = NS::TransferPtr(Heap->newTexture(Desc));
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
		MetalLLM::LogAllocTexture(Texture.get());
#endif
		return Texture;
	}
	else if (Desc->usage() & MTL::TextureUsageRenderTarget)
	{
		return TargetPool.CreateTexture(Device, Desc);
	}
	else
	{
		return TexturePool.CreateTexture(Device, Desc);
	}
}

void FMetalResourceHeap::ReleaseTexture(FMetalSurface* Surface, MTLTexturePtr Texture)
{
	FScopeLock ScopeLock(&InUseResourcesCS);
	
	if (Texture && !Texture->buffer() && !Texture->parentTexture())
	{
		auto It = AllocationHandlesLUT.Find(Texture.get());
		if (It && *It != UINT32_MAX)
		{
			FreeBlock(*It);
			AllocationHandlesLUT.Remove(Texture.get());
		}
		else if (Texture->usage() & MTL::TextureUsageRenderTarget)
		{
			TargetPool.ReleaseTexture(Texture);
		}
		else
		{
			TexturePool.ReleaseTexture(Texture);
		}
	}
}

#if METAL_RHI_RAYTRACING
FMetalAccelerationStructure* FMetalResourceHeap::CreateAccelerationStructure(uint32 Size)
{
	LLM_SCOPE_METAL(ELLMTagMetal::AccelerationStructures);
	LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::AccelerationStructures);

	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	check(bSupportsHeaps);
	
	MTLAccelerationStructurePtr AccelerationStructure;
	
	{
		FScopeLock ScopeLock(&InUseResourcesCS);
		
		MTL::SizeAndAlign SizeAndAlign = Device.GetDevice()->heapAccelerationStructureSizeAndAlign(Size);
		uint32 ASStructSize = SizeAndAlign.size;
		uint32 ASStructAlignment = SizeAndAlign.align;
		
		uint32_t ResourceOptions = MTL::ResourceStorageModePrivate | MTL::ResourceHazardTrackingModeTracked;
		
		auto BlockIt = FindOrAllocateBlock(ASStructSize, ASStructAlignment, ResourceOptions);
		MemoryBlock& Block = *BlockIt;
		
		check(Block.Resource == nil && Block.Size == ASStructSize);
		
		uint32 HeapAllocationHandle = UINT32_MAX;
		{
			if (!InUseResourcesFreeList.IsEmpty())
			{
				InUseResourcesFreeList.Dequeue(HeapAllocationHandle);
				InUseResources[HeapAllocationHandle] = BlockIt;
			}
			else
			{
				HeapAllocationHandle = InUseResources.Add(BlockIt);
			}
		}
		check(HeapAllocationHandle != UINT32_MAX);
		
		AccelerationStructure = NS::TransferPtr(Block.Heap->newAccelerationStructure(ASStructSize, Block.Offset));
		
		AllocationHandlesLUT.Add(AccelerationStructure.get(), HeapAllocationHandle);
		
		Block.Resource = AccelerationStructure.get();
		Block.Options = ResourceOptions;
	}
	
	FMetalAccelerationStructure* AS = new FMetalAccelerationStructure(AccelerationStructure, Size);
	
	return AS;
}

void FMetalResourceHeap::ReleaseAccelerationStructure(FMetalAccelerationStructure* AccelerationStructure)
{
	if (AccelerationStructure)
	{
		FScopeLock ScopeLock(&InUseResourcesCS);
		
		MTLAccelerationStructurePtr AS = AccelerationStructure->GetAccelerationStructure();
		if (AS)
		{
			auto It = AllocationHandlesLUT.Find(AS.get());
			if (It && *It != UINT32_MAX)
			{
				FreeBlock(*It);
				AllocationHandlesLUT.Remove(AS.get());
			}
		}
		
		delete AccelerationStructure;
		AccelerationStructure = nullptr;
	}
}
#endif

void FMetalResourceHeap::Compact(bool const bForce)
{
	FScopeLock Lock(&Mutex);
	for (uint32 u = 0; u < NumUsageTypes; u++)
	{
		for (uint32 t = 0; t < NumAllocTypes; t++)
		{
			for (uint32 i = 0; i < NumMagazineSizes; i++)
			{
				for (auto It = SmallBuffers[u][t][i].CreateIterator(); It; ++It)
				{
					FMetalSubBufferMagazine* Data = *It;
					if (Data->NumCurrentAllocations() == 0 || bForce)
					{
						It.RemoveCurrent();
						FMetalDynamicRHI::Get().DeferredDelete([Data](){
							delete Data;
						});
					}
				}
			}

			uint32 BytesCompacted = 0;
			uint32 const BytesToCompact = GMetalHeapBufferBytesToCompact;

			for (uint32 i = 0; i < NumHeapSizes; i++)
			{
				for (auto It = BufferHeaps[u][t][i].CreateIterator(); It; ++It)
				{
					FMetalSubBufferHeap* Data = *It;
					if (Data->NumCurrentAllocations() == 0 || bForce)
					{
						It.RemoveCurrent();
						FMetalDynamicRHI::Get().DeferredDelete([Data](){
							delete Data;
						});
					}
				}
			}
		}
	}

	for(uint32 AllocTypeIndex = 0;AllocTypeIndex < NumAllocTypes;++AllocTypeIndex)
	{
		Buffers[AllocTypeIndex].DrainPool(bForce);
	}
	
#if PLATFORM_MAC
	ManagedBuffers.DrainPool(bForce);
	for (auto It = ManagedSubHeaps.CreateIterator(); It; ++It)
	{
		FMetalSubBufferLinear* Data = *It;
		if (Data->GetUsedSize() == 0 || bForce)
		{
			It.RemoveCurrent();
			delete Data;
		}
	}
#endif
	TexturePool.Drain(bForce);
	TargetPool.Drain(bForce);
}
	
uint32 FMetalBufferPoolPolicyData::BucketSizes[FMetalBufferPoolPolicyData::NumPoolBucketSizes] = {
	256,
	512,
	1024,
	2048,
	4096,
	8192,
	16384,
	32768,
	65536,
	131072,
	262144,
	524288,
	1048576,
	2097152,
	4194304,
	8388608,
	12582912,
	16777216,
	25165824,
	33554432,
};

FMetalBufferPool::~FMetalBufferPool()
{
}

uint32 FMetalResourceHeap::MagazineSizes[FMetalResourceHeap::NumMagazineSizes] = {
	16,
	32,
	64,
	128,
	256,
	512,
	1024,
	2048,
	4096,
	8192,
};

uint32 FMetalResourceHeap::HeapSizes[FMetalResourceHeap::NumHeapSizes] = {
	1048576,
	2097152,
};

uint32 FMetalResourceHeap::MagazineAllocSizes[FMetalResourceHeap::NumMagazineSizes] = {
	4096,
	4096,
	4096,
	8192,
	8192,
	8192,
	16384,
	16384,
	16384,
	32768,
};

uint32 FMetalResourceHeap::HeapAllocSizes[FMetalResourceHeap::NumHeapSizes] = {
	2097152,
	4194304,
};

uint32 FMetalResourceHeap::HeapTextureHeapSizes[FMetalResourceHeap::NumTextureHeapSizes] = {
	4194304,
	8388608,
	16777216,
	33554432,
	67108864,
	134217728,
	268435456
};
