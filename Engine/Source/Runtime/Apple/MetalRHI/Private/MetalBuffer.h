// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "MetalRHIPrivate.h"
#include "MetalResources.h"
#include "Containers/LockFreeList.h"
#include "ResourcePool.h"

class FMetalCommandQueue;
class FMetalDevice;
class FMetalBuffer;
typedef TSharedPtr<FMetalBuffer, ESPMode::ThreadSafe> FMetalBufferPtr;

struct FMetalPooledBufferArgs
{
    FMetalPooledBufferArgs() : Device(nullptr), Size(0), Flags(BUF_None), Storage(MTL::StorageModeShared), CpuCacheMode(MTL::CPUCacheModeDefaultCache) {}
	
    FMetalPooledBufferArgs(FMetalDevice* InDevice, uint32 InSize, EBufferUsageFlags InFlags, MTL::StorageMode InStorage, MTL::CPUCacheMode InCpuCacheMode = MTL::CPUCacheModeDefaultCache)
	: Device(InDevice)
	, Size(InSize)
    , Flags(InFlags)
	, Storage(InStorage)
	, CpuCacheMode(InCpuCacheMode)
	{
	}
	
	FMetalDevice* Device;
	uint32 Size;
	EBufferUsageFlags Flags;
	MTL::StorageMode Storage;
	MTL::CPUCacheMode CpuCacheMode;
};

class IMetalBufferAllocator
{
public:
	virtual void ReleaseBuffer(FMetalBuffer* Buffer) = 0;
};

class FMetalSubBufferHeap : public IMetalBufferAllocator
{
    friend class FMetalResourceHeap;
    
public:
	FMetalSubBufferHeap(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger Alignment, MTL::ResourceOptions, FCriticalSection& PoolMutex);
	~FMetalSubBufferHeap();
	
	NS::String*       GetLabel() const;
    MTL::Device*      GetDevice() const;
    MTL::StorageMode  GetStorageMode() const;
    MTL::CPUCacheMode GetCpuCacheMode() const;
    NS::UInteger      GetSize() const;
    NS::UInteger      GetUsedSize() const;
	NS::UInteger	  MaxAvailableSize() const;
	int64             NumCurrentAllocations() const;
    bool              CanAllocateSize(NS::UInteger Size) const;

    void SetLabel(const NS::String* label);
	
    FMetalBufferPtr NewBuffer(NS::UInteger length);
    MTL::PurgeableState SetPurgeableState(MTL::PurgeableState state);
	void FreeRange(NS::Range const& Range);
	
	virtual void ReleaseBuffer(FMetalBuffer* Buffer) override;

    void SetOwner(NS::Range const& Range, FMetalRHIBuffer* Owner, bool bIsSwap);

private:
    struct Allocation
    {
        Allocation() : Range(0,0) {}
        
        NS::Range Range;
		MTL::Buffer* Resource;
        FMetalRHIBuffer* Owner;
    };
    
	FMetalDevice& Device;
	FCriticalSection& PoolMutex;
	int64 volatile OutstandingAllocs;
	NS::UInteger MinAlign;
	NS::UInteger UsedSize;
	MTL::Buffer* ParentBuffer;
	MTLHeapPtr ParentHeap;
	TArray<NS::Range> FreeRanges;
    TArray<Allocation> AllocRanges;
};

class FMetalSubBufferLinear : public IMetalBufferAllocator
{
public:
	FMetalSubBufferLinear(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger Alignment, MTL::ResourceOptions, FCriticalSection& PoolMutex);
	~FMetalSubBufferLinear();
	
	NS::String*         GetLabel() const;
	MTL::Device*        GetDevice() const;
	MTL::StorageMode    GetStorageMode() const;
	MTL::CPUCacheMode   GetCpuCacheMode() const;
	NS::UInteger        GetSize() const;
	NS::UInteger        GetUsedSize() const;
	bool	            CanAllocateSize(NS::UInteger Size) const;

	void SetLabel(const NS::String* label);
	
	FMetalBufferPtr NewBuffer(NS::UInteger length);
	MTL::PurgeableState SetPurgeableState(MTL::PurgeableState state);
	void FreeRange(NS::Range const& Range);
	
	virtual void ReleaseBuffer(FMetalBuffer* Buffer) override;
	
private:
	FMetalDevice& 		Device;
	FCriticalSection& 	PoolMutex;
	NS::UInteger 		MinAlign;
	NS::UInteger 		WriteHead;
	NS::UInteger 		UsedSize;
	NS::UInteger 		FreedSize;
	MTL::Buffer* 		ParentBuffer;
};

class FMetalSubBufferMagazine : public IMetalBufferAllocator
{
public:
	FMetalSubBufferMagazine(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger ChunkSize, MTL::ResourceOptions);
	~FMetalSubBufferMagazine();
	
	NS::String*   GetLabel() const;
    MTL::Device*       GetDevice() const;
    MTL::StorageMode  GetStorageMode() const;
    MTL::CPUCacheMode GetCpuCacheMode() const;
    NS::UInteger     GetSize() const;
    NS::UInteger     GetUsedSize() const;
	NS::UInteger	 GetFreeSize() const;
	int64     NumCurrentAllocations() const;
    bool     CanAllocateSize(NS::UInteger Size) const;

    void SetLabel(const NS::String* label);
	void FreeRange(NS::Range const& Range);
	
	virtual void ReleaseBuffer(FMetalBuffer* Buffer) override;

    FMetalBufferPtr NewBuffer();
    MTL::PurgeableState SetPurgeableState(MTL::PurgeableState state);

private:
	FMetalDevice& 	Device;
	NS::UInteger 	MinAlign;
    NS::UInteger 	BlockSize;
	int64 volatile 	OutstandingAllocs;
	int64 volatile 	UsedSize;
	MTL::Buffer* 	ParentBuffer;
	MTLHeapPtr 		ParentHeap;
	TArray<int8> 	Blocks;
};

struct FMetalRingBufferRef
{
	FMetalRingBufferRef(FMetalDevice& InDevice, FMetalBufferPtr Buf);
	~FMetalRingBufferRef();
	
	void SetLastRead(uint64 Read) { FPlatformAtomics::InterlockedExchange((int64*)&LastRead, Read); }
    
    FMetalBufferPtr GetBuffer()
    {
        return Buffer;
    }
    
	MTL::Buffer* GetMTLBuffer()
    {
        return Buffer ? Buffer->GetMTLBuffer() : nullptr;
    }
    
	FMetalDevice& Device;
	FMetalBufferPtr Buffer = nullptr;
	uint64 LastRead;
};

class FMetalResourceHeap;
class FMetalCommandBuffer;

class FMetalSubBufferRing : public IMetalBufferAllocator
{
public:
	FMetalSubBufferRing(FMetalDevice& MetalDevice, NS::UInteger Size, NS::UInteger Alignment, MTL::ResourceOptions Options);
	~FMetalSubBufferRing();
	
	MTL::Device*        GetDevice() const;
	MTL::StorageMode    GetStorageMode() const;
	MTL::CPUCacheMode   GetCpuCacheMode() const;
    NS::UInteger        GetSize() const;
	
	FMetalBufferPtr NewBuffer(NS::UInteger Size, uint32 Alignment);
	virtual void ReleaseBuffer(FMetalBuffer* Buffer) override 
	{};
	
	/** Tries to shrink the ring-buffer back toward its initial size, but not smaller. */
	void Shrink();
	
	/** Commits a completion handler to the cmd-buffer to release the processed range */
	void Commit(FMetalCommandBuffer* CmdBuffer);
	
private:
	/** Submits all outstanding writes to the GPU, coalescing the updates into a single contiguous range. */
	void Submit();
	
	FMetalDevice& Device;
	
	NS::UInteger FrameSize[10];
	NS::UInteger LastFrameChange;
	NS::UInteger InitialSize;
	NS::UInteger MinAlign;
	NS::UInteger CommitHead;
	NS::UInteger SubmitHead;
	NS::UInteger WriteHead;
	NS::UInteger BufferSize;
	MTL::ResourceOptions Options;
	MTL::StorageMode Storage;
	TSharedPtr<FMetalRingBufferRef, ESPMode::ThreadSafe> RingBufferRef;
	TArray<NS::Range> AllocatedRanges;
};

class FMetalBufferPoolPolicyData
{
	enum BucketSizes
	{
		// These sizes are required for ring-buffers and esp. Managed Memory which is a Mac-only feature
		BucketSize256,
		BucketSize512,
		BucketSize1k,
		BucketSize2k,
		BucketSize4k,
		BucketSize8k,
		BucketSize16k,
		BucketSize32k,
		BucketSize64k,
		BucketSize128k,
		BucketSize256k,
		BucketSize512k,
		BucketSize1Mb,
		BucketSize2Mb,
		BucketSize4Mb,
		// These sizes are the ones typically used by buffer allocations
		BucketSize8Mb,
		BucketSize12Mb,
		BucketSize16Mb,
		BucketSize24Mb,
		BucketSize32Mb,
		NumBucketSizes
	};
public:
	/** Buffers are created with a simple byte size */
	typedef FMetalPooledBufferArgs CreationArguments;
	enum
	{
		NumResourceStorageModes = 4, /* Corresponds to MTLStorageMode types: managed, shared, private, memoryless */
		NumSafeFrames = 1, /** Number of frames to leave buffers before reclaiming/reusing */
		NumPoolBucketSizes = NumBucketSizes, /** Number of pool bucket sizes */
		NumPoolBuckets = NumPoolBucketSizes * NumResourceStorageModes, /** Number of pool bucket sizes - all entries must use consistent ResourceOptions, so the total number of pool buckets is the number of pool bucket sizes x the number of resource storage modes */
		NumToDrainPerFrame = 65536, /** Max. number of resources to cull in a single frame */
		CullAfterFramesNum = 30 /** Resources are culled if unused for more frames than this */
	};
	
	/** Get the pool bucket index from the size
	 * @param Size the number of bytes for the resource
	 * @returns The bucket index.
	 */
	uint32 GetPoolBucketIndex(CreationArguments Args);
	
	/** Get the pool bucket size from the index
	 * @param Bucket the bucket index
	 * @returns The bucket size.
	 */
	uint32 GetPoolBucketSize(uint32 Bucket);
	
	/** Creates the resource
	 * @param Args The buffer size in bytes.
	 * @returns A suitably sized buffer or NULL on failure.
	 */
	MTL::Buffer* CreateResource(FRHICommandListBase& RHICmdList, CreationArguments Args);
	
	/** Gets the arguments used to create resource
	 * @param Resource The buffer to get data for.
	 * @returns The arguments used to create the buffer.
	 */
	CreationArguments GetCreationArguments(MTL::Buffer* Resource);
	
	/** Frees the resource
	 * @param Resource The buffer to prepare for release from the pool permanently.
	 */
	void FreeResource(MTL::Buffer* Resource);
	
private:
	/** The bucket sizes */
	static uint32 BucketSizes[NumPoolBucketSizes];
	static TMap<MTL::Buffer*, CreationArguments> CreationArgumentMap; 
};

/** A pool for metal buffers with consistent usage, bucketed for efficiency. */
class FMetalBufferPool : public TResourcePool<MTL::Buffer*, FMetalBufferPoolPolicyData, FMetalBufferPoolPolicyData::CreationArguments>
{
public:
	/** Destructor */
	virtual ~FMetalBufferPool();
};

class FMetalTexturePool
{
	enum
	{
        PurgeAfterNumFrames = 2, /* Textures must be reused fairly rapidly but after this number of frames we reclaim the memory, even though the object persists */
		CullAfterNumFrames = 3, /* Textures must be reused fairly rapidly or we bin them as they are much larger than buffers */
	};
public:
	struct Descriptor
	{
		friend uint32 GetTypeHash(Descriptor const& Other)
		{
			uint32 Hash = GetTypeHash((uint64)Other.textureType);
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.pixelFormat));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.usage));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.width));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.height));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.depth));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.mipmapLevelCount));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.sampleCount));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.arrayLength));
			Hash = HashCombine(Hash, GetTypeHash((uint64)Other.resourceOptions));
			return Hash;
		}
		
		bool operator<(Descriptor const& Other) const
		{
			if (this != &Other)
			{
				return (textureType < Other.textureType ||
						pixelFormat < Other.pixelFormat ||
						width < Other.width ||
						height < Other.height ||
						depth < Other.depth ||
						mipmapLevelCount < Other.mipmapLevelCount ||
						sampleCount < Other.sampleCount ||
						arrayLength < Other.arrayLength ||
						resourceOptions < Other.resourceOptions ||
						usage < Other.usage);
			}
			return false;
		}
		
		bool operator==(Descriptor const& Other) const
		{
			if (this != &Other)
			{
				return (textureType == Other.textureType &&
				pixelFormat == Other.pixelFormat &&
				width == Other.width &&
				height == Other.height &&
				depth == Other.depth &&
				mipmapLevelCount == Other.mipmapLevelCount &&
				sampleCount == Other.sampleCount &&
				arrayLength == Other.arrayLength &&
				resourceOptions == Other.resourceOptions &&
				usage == Other.usage);
			}
			return true;
		}
		
		NS::UInteger textureType;
        NS::UInteger pixelFormat;
        NS::UInteger width;
        NS::UInteger height;
        NS::UInteger depth;
        NS::UInteger mipmapLevelCount;
        NS::UInteger sampleCount;
		NS::UInteger arrayLength;
		NS::UInteger resourceOptions;
		NS::UInteger usage;
		NS::UInteger freedFrame;
	};
	
	FMetalTexturePool(FCriticalSection& PoolMutex);
	~FMetalTexturePool();
	
	MTLTexturePtr CreateTexture(FMetalDevice& Device, MTL::TextureDescriptor* Desc);
	void ReleaseTexture(MTLTexturePtr Texture);
	
	void Drain(bool const bForce);

private:
	FCriticalSection& PoolMutex;
	TMap<Descriptor, MTLTexturePtr> Pool;
};

typedef NS::SharedPtr<MTL::Heap> MTLHeapPtr;

class FMetalResourceHeap : public IMetalBufferAllocator
{
	enum MagazineSize
	{
		Size16,
		Size32,
		Size64,
		Size128,
		Size256,
		Size512,
		Size1024,
		Size2048,
		Size4096,
		Size8192,
		NumMagazineSizes
	};
	
	enum HeapSize
	{
		Size1Mb,
		Size2Mb,
		NumHeapSizes
	};

	enum TextureHeapSize
	{
		Size4Mb,
		Size8Mb,
		Size16Mb,
		Size32Mb,
		Size64Mb,
		Size128Mb,
		Size256Mb,
		NumTextureHeapSizes,
		MinTexturesPerHeap = 4,
		MaxTextureSize = Size64Mb,
	};
	
	enum AllocTypes
	{
		AllocShared,
		AllocPrivate,
		NumAllocTypes = 2
	};
	
	enum EMetalHeapTextureUsage
	{
		/** Regular texture resource */
		EMetalHeapTextureUsageResource = 0,
		/** Render target or UAV that can be aliased */
		EMetalHeapTextureUsageRenderTarget = 1,
		/** Number of texture usage types */
		EMetalHeapTextureUsageNum = 2
	};
    
    enum UsageTypes
    {
        UsageStatic,
        UsageDynamic,
        NumUsageTypes = 2
    };
    
public:
	FMetalResourceHeap(FMetalDevice& MetalDevice);
	~FMetalResourceHeap();
	
	void Init(FMetalCommandQueue& Queue);
	
	FMetalBufferPtr CreateBuffer(uint32 Size, uint32 Alignment, EBufferUsageFlags Flags, MTL::ResourceOptions Options, bool bForceUnique = false);
	MTLTexturePtr CreateTexture(MTL::TextureDescriptor* Desc, FMetalSurface* Surface);
	
	virtual void ReleaseBuffer(FMetalBuffer* Buffer) override;
	void ReleaseTexture(FMetalSurface* Surface, MTLTexturePtr Texture);
	
#if METAL_RHI_RAYTRACING
	FMetalAccelerationStructure* CreateAccelerationStructure(uint32 Size);
	void ReleaseAccelerationStructure(FMetalAccelerationStructure* AccelerationStructure);
#endif
	
	void Compact(bool const bForce);
	
private:
	uint32 GetMagazineIndex(uint32 Size);
	uint32 GetHeapIndex(uint32 Size);
	TextureHeapSize TextureSizeToIndex(uint32 Size);
	
	MTLHeapPtr GetTextureHeap(MTL::TextureDescriptor* Desc, MTL::SizeAndAlign Size);
	
private:
	static uint32 MagazineSizes[NumMagazineSizes];
	static uint32 HeapSizes[NumHeapSizes];
	static uint32 MagazineAllocSizes[NumMagazineSizes];
	static uint32 HeapAllocSizes[NumHeapSizes];
	static uint32 HeapTextureHeapSizes[NumTextureHeapSizes];

	FMetalDevice& 		Device;
	FCriticalSection 	Mutex;
	FMetalCommandQueue* Queue;
	
	/** Small allocations (<= 4KB) are made from magazine allocators that use sub-ranges of a buffer */
    TArray<FMetalSubBufferMagazine*> SmallBuffers[NumUsageTypes][NumAllocTypes][NumMagazineSizes];

	/** Typical allocations (4KB - 4MB) are made from heap allocators that use sub-ranges of a buffer */
	/** There are two alignment categories for heaps - 16b for Vertes/Index data and 256b for constant data (macOS-only) */
    TArray<FMetalSubBufferHeap*> BufferHeaps[NumUsageTypes][NumAllocTypes][NumHeapSizes];
	
	/** Larger buffers (up-to 32MB) that are subject to bucketing & pooling rather than sub-allocation */
	FMetalBufferPool Buffers[NumAllocTypes];
#if PLATFORM_MAC // All managed buffers are bucketed & pooled rather than sub-allocated to avoid memory consistency complexities
	FMetalBufferPool ManagedBuffers;
	TArray<FMetalSubBufferLinear*> ManagedSubHeaps;
#endif
	/** Anything else is just allocated directly from the device! */
	
	/** We can reuse texture allocations as well, to minimize their performance impact */
	FMetalTexturePool TexturePool;
	FMetalTexturePool TargetPool;
	
	TArray<MTLHeapPtr> TextureHeaps[EMetalHeapTextureUsageNum][NumTextureHeapSizes];
	
	struct MemoryBlock
	{
		MTLHeapPtr                      Heap;
		uint64           		        Offset;
		uint64           		        Size;
		MTL::Resource*	                Resource;
		MTL::ResourceOptions 	        Options;
	};
	
	using FMetalListIterator = TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>::TIterator;
	
	TMap<MTL::ResourceOptions, TDoubleLinkedList<MemoryBlock>*> FreeLists;
	TMap<MTL::ResourceOptions, TDoubleLinkedList<MemoryBlock>*> UsedLists;
	
	FCriticalSection                FreeListCS;
	
	// TODO: AAPL: Figure out how to guarantee index uniqueness without using a set (as iterators cant be hashed)
	FCriticalSection             InUseResourcesCS;
	TArray<FMetalListIterator>   InUseResources;
	TQueue<uint32>               InUseResourcesFreeList;
	TMap<MTL::Resource*, uint32> AllocationHandlesLUT;
	
	FMetalListIterator MergeBlocks(TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>& List, FMetalListIterator BlockItA, FMetalListIterator BlockItB);
	void FreeBlock(const uint32 ResourceAllocationHandle);
	FMetalListIterator FindOrAllocateBlock(uint32 Size, uint32 Alignment, MTL::ResourceOptions Options);
	FMetalListIterator SplitBlock(TDoubleLinkedList<FMetalResourceHeap::MemoryBlock>& List, FMetalListIterator BlockIt, const uint64 Offset, const uint32 Size);
};
