// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalCaptureManager.h"
#include "MetalRHIPrivate.h"
#include "MetalCommandQueue.h"

bool GMetalSupportsCaptureManager = false;

FMetalCaptureManager::FMetalCaptureManager(MTL::Device* InDevice, FMetalCommandQueue& InQueue)
: Device(InDevice)
, Queue(InQueue)
{
    GMetalSupportsCaptureManager = true;

    MTL::CaptureManager* Manager = MTL::CaptureManager::sharedCaptureManager();
    Manager->setDefaultCaptureScope(Manager->newCaptureScope(Device));
    Manager->defaultCaptureScope()->setLabel(NS::String::string("1 Frame", NS::UTF8StringEncoding));

    FMetalCaptureScope DefaultScope;
    DefaultScope.MTLScope = Manager->defaultCaptureScope();
    DefaultScope.Type = EMetalCaptureTypePresent;
    DefaultScope.StepCount = 1;
    DefaultScope.LastTrigger = 0;
    ActiveScopes.Add(DefaultScope);
    DefaultScope.MTLScope->beginScope();

    uint32 PresentStepCounts[] = {2, 5, 10, 15, 30, 60, 90, 120};
    for (uint32 i = 0; i < (sizeof(PresentStepCounts) / sizeof(uint32)); i++)
    {
        FMetalCaptureScope Scope;
        Scope.MTLScope = Manager->newCaptureScope(Device);
        FString Label = FString::Printf(TEXT("%u Frames"), PresentStepCounts[i]);
        Scope.MTLScope->setLabel(FStringToNSString(Label));
        Scope.Type = EMetalCaptureTypePresent;
        Scope.StepCount = PresentStepCounts[i];
        Scope.LastTrigger = 0;
        ActiveScopes.Add(Scope);
        Scope.MTLScope->beginScope();
    }
}

FMetalCaptureManager::~FMetalCaptureManager()
{
}

void FMetalCaptureManager::PresentFrame(uint32 FrameNumber)
{
	if (GMetalSupportsCaptureManager)
	{
		for (FMetalCaptureScope& Scope : ActiveScopes)
		{
			uint32 Diff = 0;
			if (FrameNumber > Scope.LastTrigger)
			{
				Diff = FrameNumber - Scope.LastTrigger;
			}
			else
			{
				Diff = (UINT32_MAX - Scope.LastTrigger) + FrameNumber;
            }

			if (Diff >= Scope.StepCount)
			{
				Scope.MTLScope->endScope();
				Scope.MTLScope->beginScope();
				Scope.LastTrigger = FrameNumber;
			}
		}
	}
	else
	{
		Queue.InsertDebugCaptureBoundary();
	}
}

void FMetalCaptureManager::BeginCapture(void)
{
	if (GMetalSupportsCaptureManager)
	{
		MTL::CaptureManager::sharedCaptureManager()->startCapture(Device);
	}
}

void FMetalCaptureManager::EndCapture(void)
{
	if (GMetalSupportsCaptureManager)
	{
		MTL::CaptureManager::sharedCaptureManager()->stopCapture();
	}
}

