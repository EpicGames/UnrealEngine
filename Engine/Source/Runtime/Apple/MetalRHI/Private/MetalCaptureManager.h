// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"

class FMetalCommandQueue;

class FMetalCaptureManager
{
public:
	FMetalCaptureManager(MTL::Device* Device, FMetalCommandQueue& Queue);
	~FMetalCaptureManager();
	
	// Called by the MetalRHI code to trigger the provided capture scopes visible in Xcode.
	void PresentFrame(uint32 FrameNumber);
	
	// Programmatic captures without an Xcode capture scope.
	// Use them to instrument the code manually to debug issues.
	void BeginCapture(void);
	void EndCapture(void);
	
private:
	MTL::Device* Device;
	FMetalCommandQueue& Queue;
	bool bSupportsCaptureManager;
	
private:
	enum EMetalCaptureType
	{
		EMetalCaptureTypeUnknown,
		EMetalCaptureTypeFrame, // (BeginFrame-EndFrame) * StepCount
		EMetalCaptureTypePresent, // (Present-Present) * StepCount
		EMetalCaptureTypeViewport, // (Present-Present) * Viewports * StepCount
	};

	struct FMetalCaptureScope
	{
		EMetalCaptureType Type;
		uint32 StepCount;
		uint32 LastTrigger;
		MTL::CaptureScope* MTLScope;
	};
	
	TArray<FMetalCaptureScope> ActiveScopes;
};
