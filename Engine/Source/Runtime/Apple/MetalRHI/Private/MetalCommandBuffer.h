// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalThirdParty.h"
#include "MetalResources.h"
#include "MetalShaderResources.h"
#include "RHIBreadcrumbs.h"
#include "MetalProfiler.h"
#include "MetalCommandQueue.h"

class FMetalCommandBuffer
{
public:
    FMetalCommandBuffer(MTL::CommandBuffer* InCommandBuffer, FMetalCommandQueue& Queue) 
	: EventStream(Queue.GetProfilerQueue())
    {
        CommandBuffer = InCommandBuffer;
		CommandBuffer->retain();
    }
	
	~FMetalCommandBuffer()
	{
		CommandBuffer->release();
	}
    
    FORCEINLINE MTL::CommandBuffer*& GetMTLCmdBuffer() { return CommandBuffer; }
	
#if WITH_RHI_BREADCRUMBS
	void BeginBreadcrumb(FMetalBreadcrumbInstance* Instance)
	{
		FMetalBreadcrumbTrackerObject TrackerObject;
		TrackerObject.Type = EMetalBreadcrumbTrackerType::Begin;
		TrackerObject.CmdBuffer = this;
		TrackerObject.Instance = Instance;
		BreadcrumbTrackerStream.Add(TrackerObject);
	}

	void EndBreadcrumb(FMetalBreadcrumbInstance* Instance)
	{
		FMetalBreadcrumbTrackerObject TrackerObject;
		TrackerObject.Type = EMetalBreadcrumbTrackerType::End;
		TrackerObject.CmdBuffer = this;
		TrackerObject.Instance = Instance;
		BreadcrumbTrackerStream.Add(TrackerObject);
	}
#endif
    
	void AddCounterSample(FMetalCounterSamplePtr CounterSample)
	{
		if(!CounterSample)
		{
			return;
		}
		
	#if WITH_RHI_BREADCRUMBS
		FMetalBreadcrumbTrackerObject TrackerObject;
		TrackerObject.Type = EMetalBreadcrumbTrackerType::Encode;
		TrackerObject.CmdBuffer = this;
		TrackerObject.CounterSample = CounterSample;
		BreadcrumbTrackerStream.Add(TrackerObject);
	#endif
		
		CounterSamples.Add(CounterSample);
	}
	
	void SetBeginWorkTimestamp(uint64_t* Timestamp)
	{
		BeginWorkTimestamp = Timestamp;
	}
	
	void SetEndWorkTimestamp(uint64_t* Timestamp)
	{
		EndWorkTimestamp = Timestamp;
	}

	template <typename TEventType, typename... TArgs>
	TEventType& EmplaceProfilerEvent(TArgs&&... Args)
	{
		TEventType& Data = EventStream.Emplace<TEventType>(Forward<TArgs>(Args)...);

		if constexpr (std::is_same_v<UE::RHI::GPUProfiler::FEvent::FBeginWork, TEventType>)
		{
			// Store BeginEvents in a separate array as the CPUTimestamp field needs updating at submit time.
			BeginEvents.Add(&Data);
		}
		
		// Not sure what to do here, ask Luke!
		return Data;
	}
	
	void FlushProfilerEvents(UE::RHI::GPUProfiler::FEventStream& Destination, uint64 CPUTimestamp)
	{
		for (UE::RHI::GPUProfiler::FEvent::FBeginWork* BeginEvent : BeginEvents)
		{
			BeginEvent->CPUTimestamp = CPUTimestamp;
		}
		BeginEvents.Reset();
		Destination.Append(MoveTemp(EventStream));
	}
	
	TArray<FMetalRHIRenderQuery*> TimestampQueries;
	TArray<FMetalRHIRenderQuery*> OcclusionQueries;
	TArray<FMetalCounterSamplePtr> CounterSamples;

	UE::RHI::GPUProfiler::FEventStream EventStream;
	TArray<UE::RHI::GPUProfiler::FEvent::FBeginWork*, TInlineAllocator<8>> BeginEvents;
	
#if WITH_RHI_BREADCRUMBS
	TArray<FMetalBreadcrumbTrackerObject> BreadcrumbTrackerStream;
#endif
	
	uint64_t* BeginWorkTimestamp;
	uint64_t* EndWorkTimestamp;
	
private:
    MTL::CommandBuffer* CommandBuffer;
};
