// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCommandEncoder.cpp: Metal command encoder wrapper.
=============================================================================*/

#include "MetalCommandEncoder.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalCommandBuffer.h"
#include "MetalRHIPrivate.h"
#include "MetalShaderTypes.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalProfiler.h"
#include "MetalShaderResources.h"
#include "MetalStateCache.h"
#include "MetalRHIContext.h"

const uint32 EncoderRingBufferSize = 1024 * 1024;

#if METAL_DEBUG_OPTIONS
extern int32 GMetalBufferScribble;
#endif

static TCHAR const* const GMetalCommandDataTypeName[] = {
	TEXT("DrawPrimitive"),
	TEXT("DrawPrimitiveIndexed"),
	TEXT("DrawPrimitivePatch"),
	TEXT("DrawPrimitiveIndirect"),
	TEXT("DrawPrimitiveIndexedIndirect"),
	TEXT("Dispatch"),
	TEXT("DispatchIndirect"),
};

FString FMetalCommandData::ToString() const
{
	FString Result;
	if ((uint32)CommandType < (uint32)FMetalCommandData::Type::Num)
	{
		Result = GMetalCommandDataTypeName[(uint32)CommandType];
		switch(CommandType)
		{
			case FMetalCommandData::Type::DrawPrimitive:
				Result += FString::Printf(TEXT(" BaseInstance: %u InstanceCount: %u VertexCount: %u VertexStart: %u"), Draw.baseInstance, Draw.instanceCount, Draw.vertexCount, Draw.vertexStart);
				break;
			case FMetalCommandData::Type::DrawPrimitiveIndexed:
				Result += FString::Printf(TEXT(" BaseInstance: %u BaseVertex: %u IndexCount: %u IndexStart: %u InstanceCount: %u"), DrawIndexed.baseInstance, DrawIndexed.baseVertex, DrawIndexed.indexCount, DrawIndexed.indexStart, DrawIndexed.instanceCount);
				break;
			case FMetalCommandData::Type::DrawPrimitivePatch:
				Result += FString::Printf(TEXT(" BaseInstance: %u InstanceCount: %u PatchCount: %u PatchStart: %u"), DrawPatch.baseInstance, DrawPatch.instanceCount, DrawPatch.patchCount, DrawPatch.patchStart);
				break;
			case FMetalCommandData::Type::Dispatch:
				Result += FString::Printf(TEXT(" X: %u Y: %u Z: %u"), (uint32)Dispatch.threadgroupsPerGrid[0], (uint32)Dispatch.threadgroupsPerGrid[1], (uint32)Dispatch.threadgroupsPerGrid[2]);
				break;
			case FMetalCommandData::Type::DispatchIndirect:
				Result += FString::Printf(TEXT(" Buffer: %p Offset: %u"), (void*)DispatchIndirect.ArgumentBuffer, (uint32)DispatchIndirect.ArgumentOffset);
				break;
			case FMetalCommandData::Type::DrawPrimitiveIndirect:
			case FMetalCommandData::Type::DrawPrimitiveIndexedIndirect:
			case FMetalCommandData::Type::Num:
			default:
				break;
		}
	}
	return Result;
};

#pragma mark - Public C++ Boilerplate -

FMetalCommandEncoder::FMetalCommandEncoder(FMetalDevice& MetalDevice, FMetalCommandQueue& CommandQueue)
	: Device(MetalDevice)
	, CommandQueue(CommandQueue)
	, RingBuffer(Device, EncoderRingBufferSize, BufferOffsetAlignment, FMetalCommandQueue::GetCompatibleResourceOptions((MTL::ResourceOptions)(MTL::ResourceHazardTrackingModeUntracked | BUFFER_RESOURCE_STORAGE_MANAGED)))
	, RenderPassDesc(nullptr)
	, EncoderFence(nullptr)
	, EncoderNum(0)
	, CmdBufIndex(0)
{
	for (uint32 Frequency = 0; Frequency < uint32(MTL::FunctionTypeObject)+1; Frequency++)
	{
		FMemory::Memzero(ShaderBuffers[Frequency].ReferencedResources);
		FMemory::Memzero(ShaderBuffers[Frequency].Bytes);
		FMemory::Memzero(ShaderBuffers[Frequency].Offsets);
		FMemory::Memzero(ShaderBuffers[Frequency].Lengths);
		FMemory::Memzero(ShaderBuffers[Frequency].Usage);
		ShaderBuffers[Frequency].SideTable = new FMetalBufferData;
        ShaderBuffers[Frequency].SideTable->Data = (uint8*)(&ShaderBuffers[Frequency].Lengths[0]);
        ShaderBuffers[Frequency].SideTable->Len = sizeof(ShaderBuffers[Frequency].Lengths);
        
		ShaderBuffers[Frequency].Bound = 0;
	}
	
	for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
	{
		ColorStoreActions[i] = MTL::StoreActionUnknown;
	}
	DepthStoreAction = MTL::StoreActionUnknown;
	StencilStoreAction = MTL::StoreActionUnknown;
}

void FMetalCommandEncoder::Release(void)
{
	check(!IsRenderCommandEncoderActive());
	check(!IsComputeCommandEncoderActive());
	check(!IsBlitCommandEncoderActive());
#if METAL_RHI_RAYTRACING
	check(!IsAccelerationStructureCommandEncoderActive());
#endif // METAL_RHI_RAYTRACING
	
	RenderPassDesc = nullptr;

    for(NS::String* Str : DebugGroups)
    {
        Str->release();
    }
    DebugGroups.Empty();
    
	for (uint32 Frequency = 0; Frequency < uint32(MTL::FunctionTypeObject)+1; Frequency++)
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].Buffers[i] = nullptr;
		}
		FMemory::Memzero(ShaderBuffers[Frequency].Bytes);
		FMemory::Memzero(ShaderBuffers[Frequency].ReferencedResources);
		FMemory::Memzero(ShaderBuffers[Frequency].Offsets);
		FMemory::Memzero(ShaderBuffers[Frequency].Lengths);
		FMemory::Memzero(ShaderBuffers[Frequency].Usage);
        ShaderBuffers[Frequency].SideTable->Data = nullptr;
        delete ShaderBuffers[Frequency].SideTable;
		ShaderBuffers[Frequency].SideTable = nullptr;
		ShaderBuffers[Frequency].Bound = 0;
	}
}

void FMetalCommandEncoder::Reset(void)
{
    check(IsRenderCommandEncoderActive() == false
          && IsComputeCommandEncoderActive() == false
          && IsBlitCommandEncoderActive() == false
#if METAL_RHI_RAYTRACING
		  && IsAccelerationStructureCommandEncoderActive() == false
#endif // METAL_RHI_RAYTRACING
		  );
	
	RenderPassDesc = nullptr;
	
	{
		for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
		{
			ColorStoreActions[i] = MTL::StoreActionUnknown;
		}
		DepthStoreAction = MTL::StoreActionUnknown;
		StencilStoreAction = MTL::StoreActionUnknown;
	}
	
	for (uint32 Frequency = 0; Frequency < uint32(MTL::FunctionTypeObject)+1; Frequency++)
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].Buffers[i] = nullptr;
		}
#if METAL_RHI_RAYTRACING
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].AccelerationStructure[i] = nullptr;
		}
#endif // METAL_RHI_RAYTRACING
    	FMemory::Memzero(ShaderBuffers[Frequency].Bytes);
		FMemory::Memzero(ShaderBuffers[Frequency].ReferencedResources);
		FMemory::Memzero(ShaderBuffers[Frequency].Offsets);
		FMemory::Memzero(ShaderBuffers[Frequency].Lengths);
		FMemory::Memzero(ShaderBuffers[Frequency].Usage);
		ShaderBuffers[Frequency].Bound = 0;
	}
	
    for(NS::String* Str : DebugGroups)
    {
        Str->release();
    }
    DebugGroups.Empty();
}

void FMetalCommandEncoder::ResetLive(void)
{
	for (uint32 Frequency = 0; Frequency < uint32(MTL::FunctionTypeObject)+1; Frequency++)
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].Buffers[i] = nullptr;
		}
		FMemory::Memzero(ShaderBuffers[Frequency].ReferencedResources);
		FMemory::Memzero(ShaderBuffers[Frequency].Bytes);
		FMemory::Memzero(ShaderBuffers[Frequency].Offsets);
		FMemory::Memzero(ShaderBuffers[Frequency].Lengths);
		ShaderBuffers[Frequency].Bound = 0;
	}
	
	if (IsRenderCommandEncoderActive())
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			RenderCommandEncoder->setVertexBuffer(nullptr, 0, i);
			RenderCommandEncoder->setFragmentBuffer(nullptr, 0, i);
		}
		
		for (uint32 i = 0; i < ML_MaxTextures; i++)
		{
			RenderCommandEncoder->setVertexTexture(nullptr, i);
			RenderCommandEncoder->setFragmentTexture(nullptr, i);
		}
	}
	else if (IsComputeCommandEncoderActive())
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ComputeCommandEncoder->setBuffer(nullptr, 0, i);
		}
		
		for (uint32 i = 0; i < ML_MaxTextures; i++)
		{
			ComputeCommandEncoder->setTexture(nullptr, i);
		}
	}
}

#pragma mark - Public Command Buffer Mutators -

void FMetalCommandEncoder::StartCommandBuffer(void)
{
	check(!CommandBuffer || EncoderNum == 0);
	check(IsRenderCommandEncoderActive() == false
          && IsComputeCommandEncoderActive() == false
          && IsBlitCommandEncoderActive() == false
#if METAL_RHI_RAYTRACING
		  && IsAccelerationStructureCommandEncoderActive() == false
#endif // METAL_RHI_RAYTRACING
		  );

	if (!CommandBuffer)
	{
		CmdBufIndex++;
		CommandBuffer = CommandQueue.CreateCommandBuffer();
		
		if (DebugGroups.Num())
		{
			CommandBuffer->GetMTLCmdBuffer()->setLabel(DebugGroups.Last());
		}
	}
}
	
void FMetalCommandEncoder::EndCommandBuffer(FMetalRHICommandContext* Context)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	check(CommandBuffer);
	check(IsRenderCommandEncoderActive() == false
		  && IsComputeCommandEncoderActive() == false
		  && IsBlitCommandEncoderActive() == false
#if METAL_RHI_RAYTRACING
		  && IsAccelerationStructureCommandEncoderActive() == false
#endif // METAL_RHI_RAYTRACING
		  );
	
	if(CommandBuffer->GetMTLCmdBuffer()->label() == nullptr && DebugGroups.Num() > 0)
	{
		CommandBuffer->GetMTLCmdBuffer()->setLabel(DebugGroups.Last());
	}
	
	RingBuffer.Commit(CommandBuffer);
	
#if METAL_DEBUG_OPTIONS
	if(Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)
	{
		for (FMetalBufferPtr Buffer : ActiveBuffers)
		{
			Device.AddActiveBuffer(Buffer->GetMTLBuffer(), Buffer->GetRange());
		}
		
		TSet<FMetalBufferPtr> NewActiveBuffers = MoveTemp(ActiveBuffers);
		
		Context->GetContextSyncPoint()->OnCompletionCallback([&InDevice = Device, NewActiveBuffers]()
		{
			for (FMetalBufferPtr Buffer : NewActiveBuffers)
			{
				InDevice.RemoveActiveBuffer(Buffer->GetMTLBuffer(), Buffer->GetRange());
			}
		});
	}
#endif
	
	CommandBuffer = nullptr;
	EncoderNum = 0;
}

#pragma mark - Public Command Encoder Accessors -

#if METAL_RHI_RAYTRACING
bool FMetalCommandEncoder::IsAccelerationStructureCommandEncoderActive(void) const
{
	return AccelerationStructureCommandEncoder.get() != nullptr;
}
#endif // METAL_RHI_RAYTRACING

bool FMetalCommandEncoder::IsRenderPassDescriptorValid(void) const
{
	return (RenderPassDesc != nullptr);
}

const MTL::RenderPassDescriptor* FMetalCommandEncoder::GetRenderPassDescriptor(void) const
{
	return RenderPassDesc;
}

MTL::RenderCommandEncoder* FMetalCommandEncoder::GetRenderCommandEncoder(void)
{
	check(IsRenderCommandEncoderActive() && RenderCommandEncoder);
	return RenderCommandEncoder.get();
}

MTL::ComputeCommandEncoder* FMetalCommandEncoder::GetComputeCommandEncoder(void)
{
	check(IsComputeCommandEncoderActive());
	return ComputeCommandEncoder.get();
}

MTL::BlitCommandEncoder* FMetalCommandEncoder::GetBlitCommandEncoder(void)
{
	check(IsBlitCommandEncoderActive());
	return BlitCommandEncoder.get();
}

#if METAL_RHI_RAYTRACING
MTL::AccelerationStructureCommandEncoder* FMetalCommandEncoder::GetAccelerationStructureCommandEncoder(void)
{
	check(IsAccelerationStructureCommandEncoderActive());
	return AccelerationStructureCommandEncoder.get();
}
#endif // METAL_RHI_RAYTRACING

TRefCountPtr<FMetalFence> const& FMetalCommandEncoder::GetEncoderFence(void) const
{
	return EncoderFence;
}
	
#pragma mark - Public Command Encoder Mutators -

void FMetalCommandEncoder::BeginRenderCommandEncoding(FMetalCounterSampler* Sampler)
{
	check(RenderPassDesc);
	check(CommandBuffer);
	check(!IsAnyCommandEncoderActive());
	
	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if (bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation))
	{
		CommandEncoderFence.FenceResources = MoveTemp(TransitionedResources);

		// Update fence state if current pass and prologue pass render to the same render targets
		MTL::RenderPassColorAttachmentDescriptorArray* ColorAttachments = RenderPassDesc->colorAttachments();
		for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
		{
            MTL::RenderPassColorAttachmentDescriptor* ColorDesc = ColorAttachments->object(i);
			if (ColorDesc->texture())
			{
				FenceResource(ColorDesc->texture(), MTL::FunctionTypeFragment, true);
			}
		}
		if (RenderPassDesc->depthAttachment()->texture())
		{
			FenceResource(RenderPassDesc->depthAttachment()->texture(), MTL::FunctionTypeFragment, true);
		}
		if (RenderPassDesc->stencilAttachment()->texture() && RenderPassDesc->stencilAttachment()->texture() != RenderPassDesc->depthAttachment()->texture())
		{
			FenceResource(RenderPassDesc->stencilAttachment()->texture(), MTL::FunctionTypeFragment, true);
		}
	}

    // Clear Residency Cache (TODO: Move this to a separate function)
    ResourceUsage.Empty();

	if(Sampler)
	{
		CommandBuffer->AddCounterSample(Sampler->SetupStageCounters(RenderPassDesc));
	}
	
    RenderCommandEncoder = NS::RetainPtr(CommandBuffer->GetMTLCmdBuffer()->renderCommandEncoder(RenderPassDesc));
	EncoderNum++;

	check(!EncoderFence);
    NS::String* Label = nullptr;
	
	if(GetEmitDrawEvents())
	{
		Label = FStringToNSString(FString::Printf(TEXT("RenderEncoder: %s"), DebugGroups.Num() > 0 ? *NSStringToFString(DebugGroups.Last()) : TEXT("InitialPass")));
		RenderCommandEncoder->setLabel(Label);
		
        for (NS::String* Group : DebugGroups)
        {
            RenderCommandEncoder->pushDebugGroup(Group);
        }
	}

	EncoderFence = CommandQueue.CreateFence(Label);
}

void FMetalCommandEncoder::BeginRenderCommandEncoding(MTL::RenderPassDescriptor* InRenderPassDesc, MTLParallelRenderCommandEncoderPtr ParallelEncoder)
{
	ResourceUsage.Empty();

	RenderPassDesc = InRenderPassDesc;
	RenderCommandEncoder = NS::RetainPtr(ParallelEncoder->renderCommandEncoder());
	bIsParallelEncoding = true;
	
	EncoderNum++;

	check(!EncoderFence);
	NS::String* Label = nullptr;
	
	if(GetEmitDrawEvents())
	{
		Label = FStringToNSString(FString::Printf(TEXT("RenderEncoder: %s"), DebugGroups.Num() > 0 ? *NSStringToFString(DebugGroups.Last()) : TEXT("InitialPass")));
		RenderCommandEncoder->setLabel(Label);
		
		for (NS::String* Group : DebugGroups)
		{
			RenderCommandEncoder->pushDebugGroup(Group);
		}
	}

	//EncoderFence = CommandQueue.CreateFence(Label);
}

MTLParallelRenderCommandEncoderPtr FMetalCommandEncoder::BeginParallelRenderCommandEncoding(FMetalCounterSampler* Sampler)
{
	check(RenderPassDesc);
	check(CommandBuffer);
	check(!IsAnyCommandEncoderActive());
	
	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if (bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation))
	{
		CommandEncoderFence.FenceResources = MoveTemp(TransitionedResources);

		// Update fence state if current pass and prologue pass render to the same render targets
		MTL::RenderPassColorAttachmentDescriptorArray* ColorAttachments = RenderPassDesc->colorAttachments();
		for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
		{
			MTL::RenderPassColorAttachmentDescriptor* ColorDesc = ColorAttachments->object(i);
			if (ColorDesc->texture())
			{
				FenceResource(ColorDesc->texture(), MTL::FunctionTypeFragment, true);
			}
		}
		if (RenderPassDesc->depthAttachment()->texture())
		{
			FenceResource(RenderPassDesc->depthAttachment()->texture(), MTL::FunctionTypeFragment, true);
		}
		if (RenderPassDesc->stencilAttachment()->texture() && RenderPassDesc->stencilAttachment()->texture() != RenderPassDesc->depthAttachment()->texture())
		{
			FenceResource(RenderPassDesc->stencilAttachment()->texture(), MTL::FunctionTypeFragment, true);
		}
	}

	// Clear Residency Cache (TODO: Move this to a separate function)
	ResourceUsage.Empty();

	if(Sampler)
	{
		CommandBuffer->AddCounterSample(Sampler->SetupStageCounters(RenderPassDesc));
	}
	
	ParallelRenderCommandEncoder = NS::RetainPtr(CommandBuffer->GetMTLCmdBuffer()->parallelRenderCommandEncoder(RenderPassDesc));
	EncoderNum++;

	check(!EncoderFence);
	NS::String* Label = nullptr;
	
	if(GetEmitDrawEvents())
	{
		Label = FStringToNSString(FString::Printf(TEXT("RenderEncoder: %s"), DebugGroups.Num() > 0 ? *NSStringToFString(DebugGroups.Last()) : TEXT("InitialPass")));
		RenderCommandEncoder->setLabel(Label);
		
		for (NS::String* Group : DebugGroups)
		{
			RenderCommandEncoder->pushDebugGroup(Group);
		}
	}

	EncoderFence = CommandQueue.CreateFence(Label);
	
	return ParallelRenderCommandEncoder;
}

void FMetalCommandEncoder::BeginComputeCommandEncoding(MTL::DispatchType DispatchType, FMetalCounterSampler* Sampler)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	check(CommandBuffer);

	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if (bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation))
	{
		CommandEncoderFence.FenceResources = MoveTemp(TransitionedResources);
	}
	
	// Clear Residency Cache (TODO: Move this to a separate function)
    ResourceUsage.Empty();

	MTL::ComputePassDescriptor* ComputePassDesc = MTL::ComputePassDescriptor::alloc()->init();
	ComputePassDesc->setDispatchType(DispatchType);
	
	if(Sampler)
	{
		CommandBuffer->AddCounterSample(Sampler->SetupStageCounters(ComputePassDesc));
	}
	
	ComputeCommandEncoder = NS::RetainPtr(CommandBuffer->GetMTLCmdBuffer()->computeCommandEncoder(ComputePassDesc));
    
	ComputePassDesc->release();
	
	EncoderNum++;
	
	check(!EncoderFence);
	NS::String* Label = nullptr;
	
	if(GetEmitDrawEvents())
	{
        Label = FStringToNSString(FString::Printf(TEXT("ComputeEncoder: %s"), DebugGroups.Num() > 0 ? *NSStringToFString(DebugGroups.Last()) : TEXT("InitialPass")));
        ComputeCommandEncoder->setLabel(Label);
		
        for (NS::String* Group : DebugGroups)
        {
            ComputeCommandEncoder->pushDebugGroup(Group);
        }
	}
	
	EncoderFence = CommandQueue.CreateFence(Label);
}

void FMetalCommandEncoder::BeginBlitCommandEncoding(FMetalCounterSampler* Sampler)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	check(CommandBuffer);
	check(!IsAnyCommandEncoderActive());
	
	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if (bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation))
	{
		CommandEncoderFence.FenceResources = MoveTemp(TransitionedResources);
	}
	
	MTL::BlitPassDescriptor* BlitPassDesc = MTL::BlitPassDescriptor::alloc()->init();
	
	if(Sampler)
	{
		CommandBuffer->AddCounterSample(Sampler->SetupStageCounters(BlitPassDesc));
	}
	
    BlitCommandEncoder = NS::RetainPtr(CommandBuffer->GetMTLCmdBuffer()->blitCommandEncoder(BlitPassDesc));
	
	BlitPassDesc->release();
	
	EncoderNum++;
	
	check(!EncoderFence);
	NS::String* Label = nullptr;
	
	if(GetEmitDrawEvents())
	{
        Label = FStringToNSString(FString::Printf(TEXT("BlitEncoder: %s"), DebugGroups.Num() > 0 ? *NSStringToFString(DebugGroups.Last()) : TEXT("InitialPass")));
		BlitCommandEncoder->setLabel(Label);
		
        for (NS::String* Group : DebugGroups)
        {
            BlitCommandEncoder->pushDebugGroup(Group);
        }
	}
	
	EncoderFence = CommandQueue.CreateFence(Label);
}

#if METAL_RHI_RAYTRACING
void FMetalCommandEncoder::BeginAccelerationStructureCommandEncoding(void)
{
	check(CommandBuffer);
	check(!IsAnyCommandEncoderActive());
	
	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if (bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation))
	{
		CommandEncoderFence.FenceResources = MoveTemp(TransitionedResources);
	}

	EncoderNum++;

	check(!EncoderFence);
	NS::String* Label = nullptr;

	AccelerationStructureCommandEncoder = NS::RetainPtr(CommandBuffer->GetMTLCmdBuffer()->accelerationStructureCommandEncoder());
	
	if(GetEmitDrawEvents())
	{
		Label = FStringToNSString(FString::Printf(TEXT("AccelerationStructureEncoder: %s"), DebugGroups.Num() > 0 ? *NSStringToFString(DebugGroups.Last()) : TEXT("InitialPass")));
		AccelerationStructureCommandEncoder->setLabel(Label);
		
		for (NS::String* Group : DebugGroups)
		{
			AccelerationStructureCommandEncoder->pushDebugGroup(Group);
		}
	}

	EncoderFence = CommandQueue.CreateFence(Label);
}
#endif // METAL_RHI_RAYTRACING

TRefCountPtr<FMetalFence> FMetalCommandEncoder::EndEncoding()
{
	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	TRefCountPtr<FMetalFence> Fence = nullptr;
    MTL_SCOPED_AUTORELEASE_POOL;

    if(IsRenderCommandEncoderActive())
    {
        if (RenderCommandEncoder)
        {
            check(!bSupportsFences || EncoderFence);
            check(RenderPassDesc);
			
			if(!bIsParallelEncoding)
			{
				MTL::RenderPassColorAttachmentDescriptorArray* ColorAttachments = RenderPassDesc->colorAttachments();
				for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
				{
					MTL::RenderPassColorAttachmentDescriptor* ColorAttachment = ColorAttachments->object(i);
					if (ColorAttachment->texture())
					{
						if (ColorAttachment->storeAction() == MTL::StoreActionUnknown)
						{
							MTL::StoreAction Action = ColorStoreActions[i];
							check(Action != MTL::StoreActionUnknown);
							RenderCommandEncoder->setColorStoreAction((MTL::StoreAction)Action, i);
						}
						// Recorded in case the epilogue pass renders to the same render targets
						TransitionResources(ColorAttachment->texture());
					}
				}
				
				if (RenderPassDesc->depthAttachment()->texture())
				{
					if (RenderPassDesc->depthAttachment()->storeAction() == MTL::StoreActionUnknown)
					{
						MTL::StoreAction Action = DepthStoreAction;
						check(Action != MTL::StoreActionUnknown);
						RenderCommandEncoder->setDepthStoreAction((MTL::StoreAction)Action);
					}
					// Recorded in case the epilogue pass renders to the same render targets
					TransitionResources(RenderPassDesc->depthAttachment()->texture());
				}
				if (RenderPassDesc->stencilAttachment()->texture())
				{
					if (RenderPassDesc->stencilAttachment()->storeAction() == MTL::StoreActionUnknown)
					{
						MTL::StoreAction Action = StencilStoreAction;
						check(Action != MTL::StoreActionUnknown);
						RenderCommandEncoder->setStencilStoreAction((MTL::StoreAction)Action);
					}
					// Recorded in case the epilogue pass renders to the same render targets
					if (RenderPassDesc->stencilAttachment()->texture() != RenderPassDesc->depthAttachment()->texture())
					{
						TransitionResources(RenderPassDesc->stencilAttachment()->texture());
					}
				}
			}
			else
			{
				check(!CommandBuffer);
			}

            // Wait the prologue fence
            {
                EMetalFenceWaitStage::Type FenceWaitStage = CommandEncoderFence.BarrierScope.GetFenceWaitStage();
                FMetalFence* PrologueFence = CommandEncoderFence.Fence;
                if (PrologueFence)
                {
                    MTL::RenderStages FenceStage = FenceWaitStage == EMetalFenceWaitStage::BeforeVertex ? MTL::RenderStageVertex : MTL::RenderStageFragment;
                    MTL::Fence* MTLFence = PrologueFence->Get();
                    RenderCommandEncoder->waitForFence(MTLFence, FenceStage);
                    PrologueFence->Wait();
                }
                CommandEncoderFence.Reset();
            }

            Fence = EncoderFence;
            UpdateFence(EncoderFence);
            
            RenderCommandEncoder->endEncoding();
            RenderCommandEncoder.reset();
            EncoderFence = nullptr;
			bIsParallelEncoding = false;
        }
    }
	else if(IsParallelRenderCommandEncoderActive())
	{
		if (ParallelRenderCommandEncoder)
		{
			check(!bSupportsFences || EncoderFence);
			check(RenderPassDesc);
				
			MTL::RenderPassColorAttachmentDescriptorArray* ColorAttachments = RenderPassDesc->colorAttachments();
			for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
			{
				MTL::RenderPassColorAttachmentDescriptor* ColorAttachment = ColorAttachments->object(i);
				if (ColorAttachment->texture())
				{
					if (ColorAttachment->storeAction() == MTL::StoreActionUnknown)
					{
						MTL::StoreAction Action = ColorStoreActions[i];
						check(Action != MTL::StoreActionUnknown);
						ParallelRenderCommandEncoder->setColorStoreAction((MTL::StoreAction)Action, i);
					}
					// Recorded in case the epilogue pass renders to the same render targets
					TransitionResources(ColorAttachment->texture());
				}
			}
			
			if (RenderPassDesc->depthAttachment()->texture())
			{
				if (RenderPassDesc->depthAttachment()->storeAction() == MTL::StoreActionUnknown)
				{
					MTL::StoreAction Action = DepthStoreAction;
					check(Action != MTL::StoreActionUnknown);
					ParallelRenderCommandEncoder->setDepthStoreAction((MTL::StoreAction)Action);
				}
				// Recorded in case the epilogue pass renders to the same render targets
				TransitionResources(RenderPassDesc->depthAttachment()->texture());
			}
			if (RenderPassDesc->stencilAttachment()->texture())
			{
				if (RenderPassDesc->stencilAttachment()->storeAction() == MTL::StoreActionUnknown)
				{
					MTL::StoreAction Action = StencilStoreAction;
					check(Action != MTL::StoreActionUnknown);
					ParallelRenderCommandEncoder->setStencilStoreAction((MTL::StoreAction)Action);
				}
				// Recorded in case the epilogue pass renders to the same render targets
				if (RenderPassDesc->stencilAttachment()->texture() != RenderPassDesc->depthAttachment()->texture())
				{
					TransitionResources(RenderPassDesc->stencilAttachment()->texture());
				}
			}

			{
				EMetalFenceWaitStage::Type FenceWaitStage = CommandEncoderFence.BarrierScope.GetFenceWaitStage();
				check(!CommandEncoderFence.Fence);
				CommandEncoderFence.Reset();
			}

			Fence = EncoderFence;
			UpdateFence(EncoderFence);
			
			ParallelRenderCommandEncoder->endEncoding();
			ParallelRenderCommandEncoder.reset();
			EncoderFence = nullptr;
		}
	}
    else if(IsComputeCommandEncoderActive())
    {
        check(!bSupportsFences || EncoderFence);

        // Wait the prologue fence
        {
            EMetalFenceWaitStage::Type FenceWaitStage = CommandEncoderFence.BarrierScope.GetFenceWaitStage();
            FMetalFence* PrologueFence = CommandEncoderFence.Fence;
            if (PrologueFence)
            {
                MTL::Fence* MTLFence = PrologueFence->Get();
                ComputeCommandEncoder->waitForFence(MTLFence);
                PrologueFence->Wait();
            }
            CommandEncoderFence.Reset();
        }

        Fence = EncoderFence;
        UpdateFence(EncoderFence);
        
        ComputeCommandEncoder->endEncoding();
        ComputeCommandEncoder.reset();
        EncoderFence = nullptr;
    }
    else if(IsBlitCommandEncoderActive())
    {
        // check(!bSupportsFences || EncoderFence);
        // Wait the prologue fence
        {
            EMetalFenceWaitStage::Type FenceWaitStage = CommandEncoderFence.BarrierScope.GetFenceWaitStage();
            FMetalFence* PrologueFence = CommandEncoderFence.Fence;
            if (PrologueFence)
            {
                MTL::Fence* MTLFence = PrologueFence->Get();
                BlitCommandEncoder->waitForFence(MTLFence);
                PrologueFence->Wait();
            }
            CommandEncoderFence.Reset();
        }

        Fence = EncoderFence;
        UpdateFence(EncoderFence);
        
        BlitCommandEncoder->endEncoding();
        BlitCommandEncoder.reset();
        EncoderFence = nullptr;
    }
#if METAL_RHI_RAYTRACING
    else if(IsAccelerationStructureCommandEncoderActive())
    {
        UpdateFence(EncoderFence);

        AccelerationStructureCommandEncoder->endEncoding();
        AccelerationStructureCommandEncoder.reset();
        EncoderFence = nullptr;
    }
#endif // METAL_RHI_RAYTRACING
	
	for (uint32 Frequency = 0; Frequency < uint32(MTL::FunctionTypeObject)+1; Frequency++)
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].Buffers[i] = nullptr;
		}
#if METAL_RHI_RAYTRACING
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].AccelerationStructure[i] = nullptr;
		}
#endif // METAL_RHI_RAYTRACING
		FMemory::Memzero(ShaderBuffers[Frequency].Bytes);
		FMemory::Memzero(ShaderBuffers[Frequency].Offsets);
		FMemory::Memzero(ShaderBuffers[Frequency].Lengths);
		FMemory::Memzero(ShaderBuffers[Frequency].Usage);
		ShaderBuffers[Frequency].Bound = 0;
	}
	
    return Fence;
}

void FMetalCommandEncoder::UpdateFence(FMetalFence* Fence)
{
	check(IsAnyCommandEncoderActive());
	
	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if ((bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)) && Fence)
	{
		MTL::Fence* MTLFence = Fence->Get();
		if (RenderCommandEncoder)
		{
			RenderCommandEncoder->updateFence(MTLFence, MTL::RenderStageFragment);
			Fence->Write();
		}
		else if (ComputeCommandEncoder)
		{
			ComputeCommandEncoder->updateFence(MTLFence);
			Fence->Write();
		}
		else if (BlitCommandEncoder)
		{
			BlitCommandEncoder->updateFence(MTLFence);
			Fence->Write();
		}
	}
}

void FMetalCommandEncoder::WaitForFence(FMetalFence* Fence)
{
	check(IsAnyCommandEncoderActive());

	static bool bSupportsFences = Device.SupportsFeature(EMetalFeaturesFences);
	if ((bSupportsFences METAL_DEBUG_OPTION(|| Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)) && Fence)
	{
		// Will be waited at encoder end recording
		CommandEncoderFence.Fence = Fence;
	}
}

void FMetalCommandEncoder::SignalEvent(MTLEventPtr Event, uint32_t SignalCount)
{
	check(CommandBuffer);
	
	if(IsAnyCommandEncoderActive())
	{
		EndEncoding();
	}
	CommandBuffer->GetMTLCmdBuffer()->encodeSignalEvent(Event.get(), SignalCount);
}

void FMetalCommandEncoder::WaitForEvent(MTLEventPtr Event, uint32_t SignalCount)
{
	check(!IsAnyCommandEncoderActive());
	check(CommandBuffer);
	
	CommandBuffer->GetMTLCmdBuffer()->encodeWait(Event.get(), SignalCount);
}

void FMetalCommandEncoder::SignalSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount)
{
	check(CommandBuffer);
	
	if(IsAnyCommandEncoderActive())
	{
		EndEncoding();
	}
	CommandBuffer->GetMTLCmdBuffer()->encodeSignalEvent(Event.get(), SignalCount);
}

void FMetalCommandEncoder::WaitForSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount)
{
	check(!IsAnyCommandEncoderActive());
	check(CommandBuffer);
	
	CommandBuffer->GetMTLCmdBuffer()->encodeWait(Event.get(), SignalCount);
}

#pragma mark - Public Debug Support -

void FMetalCommandEncoder::InsertDebugSignpost(NS::String* String)
{
	if (String)
	{
		if (RenderCommandEncoder)
		{
			RenderCommandEncoder->insertDebugSignpost(String);
		}
		else if (ComputeCommandEncoder)
		{
			ComputeCommandEncoder->insertDebugSignpost(String);
		}
		else if (BlitCommandEncoder)
		{
			BlitCommandEncoder->insertDebugSignpost(String);
		}
#if METAL_RHI_RAYTRACING
		else if (AccelerationStructureCommandEncoder)
		{
			AccelerationStructureCommandEncoder->insertDebugSignpost(String);
		}
#endif // METAL_RHI_RAYTRACING
	}
}

void FMetalCommandEncoder::PushDebugGroup(NS::String* String)
{
	if (String)
	{
        String->retain();
		DebugGroups.Add(String);
        
		if (RenderCommandEncoder)
		{
			RenderCommandEncoder->pushDebugGroup(String);
		}
		else if (ComputeCommandEncoder)
		{
			ComputeCommandEncoder->pushDebugGroup(String);
		}
		else if (BlitCommandEncoder)
		{
			BlitCommandEncoder->pushDebugGroup(String);
		}
#if METAL_RHI_RAYTRACING
		else if (AccelerationStructureCommandEncoder)
		{
			AccelerationStructureCommandEncoder->pushDebugGroup(String);
		}
#endif
	}
}

void FMetalCommandEncoder::PopDebugGroup(void)
{
	if (DebugGroups.Num() > 0)
	{
        DebugGroups.Pop()->release();
		if (RenderCommandEncoder)
		{
			RenderCommandEncoder->popDebugGroup();
		}
		else if (ComputeCommandEncoder)
		{
			ComputeCommandEncoder->popDebugGroup();
		}
		else if (BlitCommandEncoder)
		{
			BlitCommandEncoder->popDebugGroup();
		}
#if METAL_RHI_RAYTRACING
		else if (AccelerationStructureCommandEncoder)
		{
			AccelerationStructureCommandEncoder->popDebugGroup();
		}
#endif
	}
}

#pragma mark - Public Render State Mutators -

void FMetalCommandEncoder::SetRenderPassDescriptor(MTL::RenderPassDescriptor* RenderPass)
{
	check(RenderPass);
	
	if(RenderPass != RenderPassDesc)
	{
		RenderPassDesc = RenderPass;
		{
			for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
			{
				ColorStoreActions[i] = MTL::StoreActionUnknown;
			}
			DepthStoreAction = MTL::StoreActionUnknown;
			StencilStoreAction = MTL::StoreActionUnknown;
		}
	}
	check(RenderPassDesc);
	
	for (uint32 Frequency = 0; Frequency < uint32(MTL::FunctionTypeObject)+1; Frequency++)
	{
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].Buffers[i] = nullptr;
		}
#if METAL_RHI_RAYTRACING
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			ShaderBuffers[Frequency].AccelerationStructure[i] = nullptr;
		}
#endif // METAL_RHI_RAYTRACING
		FMemory::Memzero(ShaderBuffers[Frequency].Bytes);
		FMemory::Memzero(ShaderBuffers[Frequency].Offsets);
		FMemory::Memzero(ShaderBuffers[Frequency].Lengths);
		FMemory::Memzero(ShaderBuffers[Frequency].Usage);
		ShaderBuffers[Frequency].Bound = 0;
	}
}

void FMetalCommandEncoder::SetRenderPassStoreActions(MTL::StoreAction const* const ColorStore, MTL::StoreAction const DepthStore, MTL::StoreAction const StencilStore)
{
	check(RenderPassDesc);
	{
		for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
		{
			ColorStoreActions[i] = ColorStore[i];
		}
		DepthStoreAction = DepthStore;
		StencilStoreAction = StencilStore;
	}
}

void FMetalCommandEncoder::SetRenderPipelineState(FMetalShaderPipeline* PipelineState)
{
	check (RenderCommandEncoder);
	{
		RenderCommandEncoder->setRenderPipelineState(PipelineState->RenderPipelineState.get());
	}
}

void FMetalCommandEncoder::SetViewport(MTL::Viewport const Viewport[], uint32 NumActive)
{
	check(RenderCommandEncoder);
	check(NumActive >= 1 && NumActive < ML_MaxViewports);
	if (NumActive == 1)
	{
		RenderCommandEncoder->setViewport(Viewport[0]);
	}
#if PLATFORM_MAC
	else
	{
		check(Device.SupportsFeature(EMetalFeaturesMultipleViewports));
		RenderCommandEncoder->setViewports(Viewport, NumActive);
	}
#endif
}

void FMetalCommandEncoder::SetFrontFacingWinding(MTL::Winding const InFrontFacingWinding)
{
    check (RenderCommandEncoder);
	{
		RenderCommandEncoder->setFrontFacingWinding(InFrontFacingWinding);
	}
}

void FMetalCommandEncoder::SetCullMode(MTL::CullMode const InCullMode)
{
    check (RenderCommandEncoder);
	{
		RenderCommandEncoder->setCullMode(InCullMode);
	}
}

void FMetalCommandEncoder::SetDepthBias(float const InDepthBias, float const InSlopeScale, float const InClamp)
{
    check (RenderCommandEncoder);
	{
		RenderCommandEncoder->setDepthBias(InDepthBias, InSlopeScale, InClamp);
	}
}

void FMetalCommandEncoder::SetScissorRect(MTL::ScissorRect const Rect[], uint32 NumActive)
{
    check(RenderCommandEncoder);
	check(NumActive >= 1 && NumActive < ML_MaxViewports);
	if (NumActive == 1)
	{
		RenderCommandEncoder->setScissorRect(Rect[0]);
	}
#if PLATFORM_MAC
	else
	{
		check(Device.SupportsFeature(EMetalFeaturesMultipleViewports));
		RenderCommandEncoder->setScissorRects(Rect, NumActive);
	}
#endif
}

void FMetalCommandEncoder::SetTriangleFillMode(MTL::TriangleFillMode const InFillMode)
{
    check(RenderCommandEncoder);
	{
		RenderCommandEncoder->setTriangleFillMode(InFillMode);
	}
}

void FMetalCommandEncoder::SetDepthClipMode(MTL::DepthClipMode const InDepthClipMode)
{
	check(RenderCommandEncoder);
	{
		RenderCommandEncoder->setDepthClipMode(InDepthClipMode);
	}
}

void FMetalCommandEncoder::SetBlendColor(float const Red, float const Green, float const Blue, float const Alpha)
{
	check(RenderCommandEncoder);
	{
		RenderCommandEncoder->setBlendColor(Red, Green, Blue, Alpha);
	}
}

void FMetalCommandEncoder::SetDepthStencilState(MTL::DepthStencilState* InDepthStencilState)
{
    check (RenderCommandEncoder);
	{
		RenderCommandEncoder->setDepthStencilState(InDepthStencilState);
	}
}

void FMetalCommandEncoder::SetStencilReferenceValue(uint32 const ReferenceValue)
{
    check (RenderCommandEncoder);
	{
		RenderCommandEncoder->setStencilReferenceValue(ReferenceValue);
	}
}

void FMetalCommandEncoder::SetVisibilityResultMode(MTL::VisibilityResultMode const Mode, NS::UInteger const Offset)
{
    check (RenderCommandEncoder);
	{
		check(Mode == MTL::VisibilityResultModeDisabled || RenderPassDesc->visibilityResultBuffer());
		RenderCommandEncoder->setVisibilityResultMode(Mode, Offset);
	}
}
	
#pragma mark - Public Shader Resource Mutators -
#if METAL_RHI_RAYTRACING
void FMetalCommandEncoder::SetShaderAccelerationStructure(MTL::FunctionType const FunctionType, MTLAccelerationStructurePtr AccelerationStructure, NS::UInteger const index)
{
	if (AccelerationStructure)
	{
		ShaderBuffers[uint32(FunctionType)].Bound |= (1 << index);
	}
	else
	{
		ShaderBuffers[uint32(FunctionType)].Bound &= ~(1 << index);
	}

	ShaderBuffers[uint32(FunctionType)].AccelerationStructure[index] = AccelerationStructure;
	ShaderBuffers[uint32(FunctionType)].Buffers[index] = nullptr;
	ShaderBuffers[uint32(FunctionType)].Bytes[index] = nullptr;
	ShaderBuffers[uint32(FunctionType)].Offsets[index] = 0;
	ShaderBuffers[uint32(FunctionType)].Usage[index] = MTL::ResourceUsage(0);

	SetShaderBufferInternal(FunctionType, index);
}
#endif // METAL_RHI_RAYTRACING

void FMetalCommandEncoder::SetShaderBuffer(MTL::FunctionType const FunctionType, FMetalBufferPtr Buffer, NS::UInteger const Offset, NS::UInteger const Length, NS::UInteger index, MTL::ResourceUsage const Usage, EPixelFormat const Format, NS::UInteger const ElementRowPitch, TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources)
{
	FenceResource(Buffer->GetMTLBuffer(), FunctionType);
	check(index < ML_MaxBuffers);
    
    if(Buffer && (ShaderBuffers[uint32(FunctionType)].Bound & (1 << index)) &&
	   ShaderBuffers[uint32(FunctionType)].Buffers[index] == Buffer)
    {
		if(ShaderBuffers[uint32(FunctionType)].Offsets[index] != Offset ||
		   ShaderBuffers[uint32(FunctionType)].Usage[index] != Usage)
		{
			SetShaderBufferOffset(FunctionType, Offset, Length, index);
			ShaderBuffers[uint32(FunctionType)].Usage[index] = Usage;
			ShaderBuffers[uint32(FunctionType)].ReferencedResources[index] = ReferencedResources;
			ShaderBuffers[uint32(FunctionType)].SetBufferMetaData(index, Length, GMetalBufferFormats[Format].DataFormat, ElementRowPitch);
		}
	}
    else
    {
		if(Buffer)
		{
			ShaderBuffers[uint32(FunctionType)].Bound |= (1 << index);
		}
		else
		{
			ShaderBuffers[uint32(FunctionType)].Bound &= ~(1 << index);
		}
		ShaderBuffers[uint32(FunctionType)].Buffers[index] = Buffer;
		ShaderBuffers[uint32(FunctionType)].Bytes[index] = nullptr;
#if METAL_RHI_RAYTRACING
		ShaderBuffers[uint32(FunctionType)].AccelerationStructure[index] = nullptr;
#endif // METAL_RHI_RAYTRACING
		ShaderBuffers[uint32(FunctionType)].ReferencedResources[index] = ReferencedResources;
		ShaderBuffers[uint32(FunctionType)].Offsets[index] = Offset;
		ShaderBuffers[uint32(FunctionType)].Usage[index] = Usage;
		ShaderBuffers[uint32(FunctionType)].SetBufferMetaData(index, Length, GMetalBufferFormats[Format].DataFormat, ElementRowPitch);
		
		SetShaderBufferInternal(FunctionType, index);
    }
}

void FMetalCommandEncoder::SetShaderData(MTL::FunctionType const FunctionType, FMetalBufferData* Data, NS::UInteger const Offset, NS::UInteger const Index, EPixelFormat const Format, NS::UInteger const ElementRowPitch)
{
	check(Index < ML_MaxBuffers);
	
	if(Data)
	{
		ShaderBuffers[uint32(FunctionType)].Bound |= (1 << Index);
	}
	else
	{
		ShaderBuffers[uint32(FunctionType)].Bound &= ~(1 << Index);
	}
	
	ShaderBuffers[uint32(FunctionType)].Buffers[Index] = nullptr;
#if METAL_RHI_RAYTRACING
	ShaderBuffers[uint32(FunctionType)].AccelerationStructure[Index] = nullptr;
#endif // METAL_RHI_RAYTRACINGs
	ShaderBuffers[uint32(FunctionType)].ReferencedResources[Index].Empty();
	ShaderBuffers[uint32(FunctionType)].Bytes[Index] = Data;
	ShaderBuffers[uint32(FunctionType)].Offsets[Index] = Offset;
	ShaderBuffers[uint32(FunctionType)].Usage[Index] = MTL::ResourceUsageRead;
	ShaderBuffers[uint32(FunctionType)].SetBufferMetaData(Index, Data ? (Data->Len - Offset) : 0, GMetalBufferFormats[Format].DataFormat, ElementRowPitch);
	SetShaderBufferInternal(FunctionType, Index);
}

void FMetalCommandEncoder::SetShaderBytes(MTL::FunctionType const FunctionType, uint8 const* Bytes, NS::UInteger const Length, NS::UInteger const Index)
{
	check(Index < ML_MaxBuffers);
	
	if(Bytes && Length)
	{
		ShaderBuffers[uint32(FunctionType)].Bound |= (1 << Index);

		switch (FunctionType)
		{
			case MTL::FunctionTypeVertex:
				check(RenderCommandEncoder);
				RenderCommandEncoder->setVertexBytes(Bytes, Length, Index);
				break;
			case MTL::FunctionTypeFragment:
				check(RenderCommandEncoder);
				RenderCommandEncoder->setFragmentBytes(Bytes, Length, Index);
				break;
			case MTL::FunctionTypeKernel:
				check(ComputeCommandEncoder);
				ComputeCommandEncoder->setBytes(Bytes, Length, Index);
				break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
			case MTL::FunctionTypeMesh:
				check(RenderCommandEncoder);
				RenderCommandEncoder->setMeshBytes(Bytes, Length, Index);
				break;
			case MTL::FunctionTypeObject:
				check(RenderCommandEncoder);
				RenderCommandEncoder->setObjectBytes(Bytes, Length, Index);
				break;
#endif
			default:
				check(false);
				break;
		}
		
		ShaderBuffers[uint32(FunctionType)].Buffers[Index] = nullptr;

#if METAL_RHI_RAYTRACING
		ShaderBuffers[uint32(FunctionType)].AccelerationStructure[Index] = nullptr;
#endif // METAL_RHI_RAYTRACING
		ShaderBuffers[uint32(FunctionType)].ReferencedResources[Index].Empty();
		ShaderBuffers[uint32(FunctionType)].Bytes[Index] = nullptr;
		ShaderBuffers[uint32(FunctionType)].Offsets[Index] = 0;
		ShaderBuffers[uint32(FunctionType)].Usage[Index] = MTL::ResourceUsageRead;
		ShaderBuffers[uint32(FunctionType)].SetBufferMetaData(Index, Length, GMetalBufferFormats[PF_Unknown].DataFormat, 0);
	}
	else
	{
		ShaderBuffers[uint32(FunctionType)].Bound &= ~(1 << Index);
		
#if METAL_RHI_RAYTRACING
		ShaderBuffers[uint32(FunctionType)].AccelerationStructure[Index] = nullptr;
#endif // METAL_RHI_RAYTRACING
		ShaderBuffers[uint32(FunctionType)].ReferencedResources[Index].Empty();
		ShaderBuffers[uint32(FunctionType)].Buffers[Index] = nullptr;
		ShaderBuffers[uint32(FunctionType)].Bytes[Index] = nullptr;
		ShaderBuffers[uint32(FunctionType)].Offsets[Index] = 0;
		ShaderBuffers[uint32(FunctionType)].Usage[Index] = MTL::ResourceUsage(0);
		ShaderBuffers[uint32(FunctionType)].SetBufferMetaData(Index, 0, GMetalBufferFormats[PF_Unknown].DataFormat, 0);
	}
	
	SetShaderBufferInternal(FunctionType, Index);
}

void FMetalCommandEncoder::SetShaderBufferOffset(MTL::FunctionType FunctionType, NS::UInteger const Offset, NS::UInteger const Length, NS::UInteger const index)
{
	check(index < ML_MaxBuffers);
    checkf(ShaderBuffers[uint32(FunctionType)].Buffers[index] && (ShaderBuffers[uint32(FunctionType)].Bound & (1 << index)), TEXT("Buffer must already be bound"));
	ShaderBuffers[uint32(FunctionType)].Offsets[index] = Offset;
	ShaderBuffers[uint32(FunctionType)].SetBufferMetaData(index, Length, GMetalBufferFormats[PF_Unknown].DataFormat, 0);
	
	switch (FunctionType)
	{
		case MTL::FunctionTypeVertex:
			check (RenderCommandEncoder);
			RenderCommandEncoder->setVertexBufferOffset(Offset + ShaderBuffers[uint32(FunctionType)].Buffers[index]->GetOffset(), index);
			break;
		case MTL::FunctionTypeFragment:
			check(RenderCommandEncoder);
			RenderCommandEncoder->setFragmentBufferOffset(Offset + ShaderBuffers[uint32(FunctionType)].Buffers[index]->GetOffset(), index);
			break;
		case MTL::FunctionTypeKernel:
			check (ComputeCommandEncoder);
			ComputeCommandEncoder->setBufferOffset(Offset + ShaderBuffers[uint32(FunctionType)].Buffers[index]->GetOffset(), index);
			break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
        case MTL::FunctionTypeObject:
			check(RenderCommandEncoder);
			RenderCommandEncoder->setObjectBufferOffset(Offset + ShaderBuffers[uint32(FunctionType)].Buffers[index]->GetOffset(), index);
            break;
        case MTL::FunctionTypeMesh:
			check(RenderCommandEncoder);
			RenderCommandEncoder->setMeshBufferOffset(Offset + ShaderBuffers[uint32(FunctionType)].Buffers[index]->GetOffset(), index);
			break;
#endif
		default:
			check(false);
			break;
	}
}

void FMetalCommandEncoder::SetShaderTexture(MTL::FunctionType FunctionType, MTL::Texture* Texture, NS::UInteger index, MTL::ResourceUsage Usage)
{
	FenceResource(Texture, FunctionType);
	check(index < ML_MaxTextures);
	switch (FunctionType)
	{
		case MTL::FunctionTypeVertex:
			check (RenderCommandEncoder);
			RenderCommandEncoder->setVertexTexture(Texture, index);
			break;
		case MTL::FunctionTypeFragment:
			check(RenderCommandEncoder);
			RenderCommandEncoder->setFragmentTexture(Texture, index);
			break;
		case MTL::FunctionTypeKernel:
			check (ComputeCommandEncoder);
			ComputeCommandEncoder->setTexture(Texture, index);
			break;
		default:
			check(false);
			break;
	}
	
	if (Texture)
	{
		uint8 Swizzle[4] = {0,0,0,0};
		assert(sizeof(Swizzle) == sizeof(uint32));
		if (Texture->pixelFormat() == MTL::PixelFormatX32_Stencil8 || Texture->pixelFormat() == MTL::PixelFormatX24_Stencil8)
		{
			Swizzle[0] = Swizzle[1] = Swizzle[2] = Swizzle[3] = 1;
		}
		
		ShaderBuffers[uint32(FunctionType)].SetTextureSwizzle(index, Swizzle);
	}
}

void FMetalCommandEncoder::SetShaderSamplerState(MTL::FunctionType FunctionType, MTL::SamplerState* Sampler, NS::UInteger index)
{
	check(index < ML_MaxSamplers);
	switch (FunctionType)
	{
		case MTL::FunctionTypeVertex:
       		check (RenderCommandEncoder);
			RenderCommandEncoder->setVertexSamplerState(Sampler, index);
			break;
		case MTL::FunctionTypeFragment:
			check (RenderCommandEncoder);
			RenderCommandEncoder->setFragmentSamplerState(Sampler, index);
			break;
		case MTL::FunctionTypeKernel:
			check (ComputeCommandEncoder);
			ComputeCommandEncoder->setSamplerState(Sampler, index);
			break;
		default:
			check(false);
			break;
	}
}

void FMetalCommandEncoder::SetShaderSideTable(MTL::FunctionType const FunctionType, NS::UInteger const Index)
{
	if (Index < ML_MaxBuffers)
	{
		SetShaderData(FunctionType, ShaderBuffers[uint32(FunctionType)].SideTable, 0, Index);
	}
}

void FMetalCommandEncoder::UseIndirectArgumentResource(MTL::Texture* Texture, MTL::ResourceUsage const Usage)
{
	FenceResource(Texture, MTL::FunctionTypeVertex);
	UseResource(Texture, Usage);
}

void FMetalCommandEncoder::UseIndirectArgumentResource(FMetalBufferPtr Buffer, MTL::ResourceUsage const Usage)
{
    MTL::Buffer* MTLBuffer = Buffer->GetMTLBuffer();
	FenceResource(MTLBuffer, MTL::FunctionTypeVertex);
	UseResource(MTLBuffer, Usage);
}

void FMetalCommandEncoder::TransitionResources(MTL::Resource* Resource)
{
	TransitionedResources.Add(Resource);
}

#pragma mark - Public Compute State Mutators -

void FMetalCommandEncoder::SetComputePipelineState(FMetalShaderPipelinePtr State)
{
	check (ComputeCommandEncoder);
	{
		ComputeCommandEncoder->setComputePipelineState(State->ComputePipelineState.get());
	}
}

void FMetalCommandEncoder::SetComputePipelineState(MTL::ComputePipelineState* State)
{
	check (ComputeCommandEncoder);
	check (State);
	{
		ComputeCommandEncoder->setComputePipelineState(State);
	}
}

#pragma mark - Public Ring-Buffer Accessor -
	
FMetalSubBufferRing& FMetalCommandEncoder::GetRingBuffer(void)
{
	return RingBuffer;
}

#pragma mark - Public Resource query Access -

#pragma mark - Private Functions -

void FMetalCommandEncoder::FenceResource(MTL::Texture* Resource, const MTL::FunctionType Function, bool bIsRenderTarget/* = false*/)
{
	MTL::Resource* Res = Resource;
	MTL::Texture* Parent = Resource->parentTexture();
	MTL::Buffer* Buffer = Resource->buffer();
	if (Parent)
	{
		Res = Parent;
	}
	else if (Buffer)
	{
		Res = Buffer;
	}

	FMetalBarrierScope& BarrierScope = CommandEncoderFence.BarrierScope;
	if (CommandEncoderFence.FenceResources.Contains(Res))
	{
		switch (Function)
		{
		case MTL::FunctionTypeKernel:
			BarrierScope.TexturesWaitStage = EMetalFenceWaitStage::BeforeVertex;
			break;
		case MTL::FunctionTypeVertex:
			BarrierScope.TexturesWaitStage = EMetalFenceWaitStage::BeforeVertex;
			break;
		case MTL::FunctionTypeFragment:
			if (bIsRenderTarget)
			{
				BarrierScope.RenderTargetsWaitStage = EMetalFenceWaitStage::BeforeFragment;
			}
			else if (BarrierScope.TexturesWaitStage == EMetalFenceWaitStage::None)
			{
				BarrierScope.TexturesWaitStage = EMetalFenceWaitStage::BeforeFragment;
			}
			break;
		default:
			checkNoEntry();
			break;
		};
	}
}

void FMetalCommandEncoder::FenceResource(MTL::Buffer* Resource, const MTL::FunctionType Function)
{
	MTL::Resource* Res = Resource;
	FMetalBarrierScope& BarrierScope = CommandEncoderFence.BarrierScope;
	if (CommandEncoderFence.FenceResources.Contains(Res))
	{
		switch (Function)
		{
		case MTL::FunctionTypeKernel:
			BarrierScope.BuffersWaitStage = EMetalFenceWaitStage::BeforeVertex;
			break;
		case MTL::FunctionTypeVertex:
			BarrierScope.BuffersWaitStage = EMetalFenceWaitStage::BeforeVertex;
			break;
		case MTL::FunctionTypeFragment:
			if (BarrierScope.BuffersWaitStage == EMetalFenceWaitStage::None)
			{
				BarrierScope.BuffersWaitStage = EMetalFenceWaitStage::BeforeFragment;
			}
			break;
		default:
			checkNoEntry();
			break;
		};
	}
}

void FMetalCommandEncoder::UseHeaps(TArray<MTL::Heap*> const& Heaps, const MTL::FunctionType Function)
{
	if (RenderCommandEncoder)
	{
		MTL::RenderStages RenderStage = (MTL::RenderStages)0;
		switch (Function)
		{
		case MTL::FunctionTypeVertex:
			RenderStage |= MTLRenderStageVertex;
			break;
		case MTL::FunctionTypeFragment:
			RenderStage |= MTLRenderStageFragment;
			break;
		#if PLATFORM_SUPPORTS_MESH_SHADERS
		case MTL::FunctionTypeMesh:
			RenderStage |= MTLRenderStageMesh;
			break;
		case MTL::FunctionTypeObject:
			RenderStage |= MTLRenderStageObject;
			break;
		#endif
		default:
			checkNoEntry();
			break;
		}

		RenderCommandEncoder->useHeaps(Heaps.GetData(), Heaps.Num(), RenderStage);
	}
	else if (ComputeCommandEncoder)
	{
		ComputeCommandEncoder->useHeaps(Heaps.GetData(), Heaps.Num());
	}
}

void FMetalCommandEncoder::UseResource(MTL::Resource* Resource, MTL::ResourceUsage const Usage)
{
	// TODO: Rework residency caching (current one is broken)
    auto IsAlreadyResident = ResourceUsage.Find(Resource);
    if (!IsAlreadyResident)
    {
        ResourceUsage.Add(Resource, Usage);
    }
    else if (Usage != *IsAlreadyResident)
    {
        ResourceUsage[Resource] = Usage;
    }
    else
    {
        return;
    }

    if (RenderCommandEncoder)
    {
		RenderCommandEncoder->useResource(Resource, Usage);
    }
    else if (ComputeCommandEncoder)
    {
		ComputeCommandEncoder->useResource(Resource, Usage);
    }
}

void FMetalCommandEncoder::UseResources(TArray<MTL::Resource*> const& Resources, MTL::ResourceUsage const Usage, MTL::RenderStages RenderStages)
{
	if (RenderCommandEncoder)
	{
		checkSlow(RenderStages != 0);
		RenderCommandEncoder->useResources(Resources.GetData(), Resources.Num(), Usage, RenderStages);
	}
	else if (ComputeCommandEncoder)
	{
		ComputeCommandEncoder->useResources(Resources.GetData(), Resources.Num(), Usage);
	}
}

void FMetalCommandEncoder::SetShaderBufferInternal(MTL::FunctionType Function, uint32 Index)
{
	FMetalBufferBindings& Binding = ShaderBuffers[uint32(Function)];

    NS::UInteger Offset = Binding.Offsets[Index];
	
	bool bBufferHasBytes = Binding.Bytes[Index] != nullptr;
	
	TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources = ShaderBuffers[uint32(Function)].ReferencedResources[Index];
	switch (Function)
	{
	case MTL::FunctionTypeKernel:
		for (TTuple<MTL::Resource*, MTL::ResourceUsage>& ReferencedResource : ReferencedResources)
		{
			ComputeCommandEncoder->useResource(
				ReferencedResource.Key,
				ReferencedResource.Value
			);
		}
		break;
	case MTL::FunctionTypeVertex:
		for (TTuple<MTL::Resource*, MTL::ResourceUsage>& ReferencedResource : ReferencedResources)
		{
			RenderCommandEncoder->useResource(
				ReferencedResource.Key,
				ReferencedResource.Value,
				MTL::RenderStageVertex
			);
		}
		break;
	case MTL::FunctionTypeFragment:
		for (TTuple<MTL::Resource*, MTL::ResourceUsage>& ReferencedResource : ReferencedResources)
		{
			RenderCommandEncoder->useResource(
				ReferencedResource.Key,
				ReferencedResource.Value,
				MTL::RenderStageFragment
			);
		}
		break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
    case MTL::FunctionTypeObject:
        for (TTuple<MTL::Resource*, MTL::ResourceUsage>& ReferencedResource : ReferencedResources)
		{
			RenderCommandEncoder->useResource(
				ReferencedResource.Key,
				ReferencedResource.Value,
				MTL::RenderStageObject
			);
		}
        break;
    case MTL::FunctionTypeMesh:
       for (TTuple<MTL::Resource*, MTL::ResourceUsage>& ReferencedResource : ReferencedResources)
		{
			RenderCommandEncoder->useResource(
				ReferencedResource.Key,
				ReferencedResource.Value,
				MTL::RenderStageMesh
			);
		}
        break;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
	default:
		checkNoEntry();
		break;
	};

	FMetalBufferPtr Buffer = Binding.Buffers[Index];
#if METAL_RHI_RAYTRACING
	MTLAccelerationStructurePtr AS = ShaderBuffers[uint32(Function)].AccelerationStructure[Index];
	if (AS)
	{
		switch (Function)
		{
		case MTL::FunctionTypeKernel:
			ShaderBuffers[uint32(Function)].Bound |= (1 << Index);
			check(ComputeCommandEncoder);
			ComputeCommandEncoder->useResource(AS.get(), MTL::ResourceUsageRead);
			ComputeCommandEncoder->setAccelerationStructure(AS.get(), Index);
			break;
		default:
			checkNoEntry();
			break;
		}
	}
	else
#endif // METAL_RHI_RAYTRACING
	if (Buffer)
	{
#if METAL_DEBUG_OPTIONS
		if(Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)
		{
			ActiveBuffers.Add(Buffer);
		}
#endif
        MTL::Buffer* MTLBuffer = Buffer->GetMTLBuffer();
        FenceResource(MTLBuffer, Function);
		switch (Function)
		{
			case MTL::FunctionTypeVertex:
				Binding.Bound |= (1 << Index);
				check(RenderCommandEncoder);
				RenderCommandEncoder->setVertexBuffer(MTLBuffer, Offset + Buffer->GetOffset(), Index);
				break;

			case MTL::FunctionTypeFragment:
				Binding.Bound |= (1 << Index);
				check(RenderCommandEncoder);
				RenderCommandEncoder->setFragmentBuffer(MTLBuffer, Offset + Buffer->GetOffset(), Index);
				break;

			case MTL::FunctionTypeKernel:
				Binding.Bound |= (1 << Index);
				check(ComputeCommandEncoder);
				ComputeCommandEncoder->setBuffer(MTLBuffer, Offset + Buffer->GetOffset(), Index);
				break;

 #if PLATFORM_SUPPORTS_MESH_SHADERS
            case MTL::FunctionTypeObject:
				Binding.Bound |= (1 << Index);
				check(RenderCommandEncoder);
				RenderCommandEncoder->setObjectBuffer(MTLBuffer, Offset + Buffer->GetOffset(), Index);
				break;

            case MTL::FunctionTypeMesh:
				Binding.Bound |= (1 << Index);
				check(RenderCommandEncoder);
				RenderCommandEncoder->setMeshBuffer(MTLBuffer, Offset + Buffer->GetOffset(), Index);
				break;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS

			default:
				check(false);
				break;
		}
	}
	else if (bBufferHasBytes)
	{
		uint8 const* Bytes = (((uint8 const*)Binding.Bytes[Index]->Data) + Binding.Offsets[Index]);
		uint32 Len = Binding.Bytes[Index]->Len - Binding.Offsets[Index];
		
		switch (Function)
		{
            case MTL::FunctionTypeVertex:
				Binding.Bound |= (1 << Index);
				check(RenderCommandEncoder);
				RenderCommandEncoder->setVertexBytes(Bytes, Len, Index);
				break;

			case MTL::FunctionTypeFragment:
				Binding.Bound |= (1 << Index);
				check(RenderCommandEncoder);
				RenderCommandEncoder->setFragmentBytes(Bytes, Len, Index);
				break;

			case MTL::FunctionTypeKernel:
				Binding.Bound |= (1 << Index);
				check(ComputeCommandEncoder);
				ComputeCommandEncoder->setBytes(Bytes, Len, Index);
				break;

#if PLATFORM_SUPPORTS_MESH_SHADERS
            case MTL::FunctionTypeObject:
                Binding.Bound |= (1 << Index);
                check(RenderCommandEncoder);
				RenderCommandEncoder->setObjectBytes(Bytes, Len, Index);
				break;

            case MTL::FunctionTypeMesh:
                Binding.Bound |= (1 << Index);
                check(RenderCommandEncoder);
				RenderCommandEncoder->setMeshBytes(Bytes, Len, Index);
				break;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS

			default:
				check(false);
				break;
		}
	}
}
