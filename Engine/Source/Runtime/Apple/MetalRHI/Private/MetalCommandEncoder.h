// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"
#include "MetalBuffer.h"
#include "MetalFence.h"
#include "MetalPipeline.h"
#include "MetalProfiler.h"
#include "MetalCounterSampler.h"

class FMetalCommandQueue;
class FMetalGraphicsPipelineState;
class FMetalDevice;
class FMetalRHICommandContext;
class FMetalShaderPipeline;
typedef TSharedPtr<FMetalShaderPipeline, ESPMode::ThreadSafe> FMetalShaderPipelinePtr;

@class FMetalCommandBufferDebug;

struct FMetalCommandData
{
	enum class Type
	{
		DrawPrimitive,
		DrawPrimitiveIndexed,
		DrawPrimitivePatch,
		DrawPrimitiveIndirect,
		DrawPrimitiveIndexedIndirect,
		Dispatch,
		DispatchIndirect,
		Num,
	};
	struct DispatchIndirectArgs
	{
        MTL::Buffer* ArgumentBuffer;
		NS::UInteger ArgumentOffset;
	};
	Type CommandType;
	union
	{
		MTL::DrawPrimitivesIndirectArguments Draw;
        MTL::DrawIndexedPrimitivesIndirectArguments DrawIndexed;
        MTL::DrawPatchIndirectArguments DrawPatch;
		MTL::DispatchThreadgroupsIndirectArguments Dispatch;
		DispatchIndirectArgs DispatchIndirect;
	};
	FString ToString() const;
};

struct FMetalCommandDebug
{
    uint32 CmdBufIndex;
	uint32 Encoder;
	uint32 Index;
	FMetalGraphicsPipelineState* PSO;
	FMetalComputeShader* ComputeShader;
	FMetalCommandData Data;
};

namespace EMetalFenceWaitStage
{
	enum Type
	{
		None			= 0,
		BeforeVertex	= 1 << 0, // Wait fence before VS/CS is executed
		BeforeFragment	= 1 << 1, // Wait fence before PS is executed
	};
};

// An emulator of MTLBarrierScope that is supported since IOS 16.0
struct FMetalBarrierScope
{
	EMetalFenceWaitStage::Type BuffersWaitStage;
	EMetalFenceWaitStage::Type RenderTargetsWaitStage;
	EMetalFenceWaitStage::Type TexturesWaitStage;

	FMetalBarrierScope()
	{
		Reset();
	}

	void Reset()
	{
		BuffersWaitStage = EMetalFenceWaitStage::None;
		RenderTargetsWaitStage = EMetalFenceWaitStage::None;
		TexturesWaitStage = EMetalFenceWaitStage::None;
	}

	EMetalFenceWaitStage::Type GetFenceWaitStage()
	{
		if (BuffersWaitStage == EMetalFenceWaitStage::BeforeVertex
			|| TexturesWaitStage == EMetalFenceWaitStage::BeforeVertex)
		{
			return EMetalFenceWaitStage::BeforeVertex;
		}
		else
		{
			return EMetalFenceWaitStage::BeforeFragment;
		}
	}
};

struct FMetalCommandEncoderFence
{
	// Resources produced in the prologue pass
	TSet<MTL::Resource*> FenceResources;
	// Fence written by the prologue pass
	TRefCountPtr<FMetalFence> Fence;
	// The barrier state for the current pass
	FMetalBarrierScope BarrierScope;

	FMetalCommandEncoderFence() = default;

	FMetalCommandEncoderFence(FMetalCommandEncoderFence&& In)
	{
		FenceResources = MoveTemp(In.FenceResources);
		Fence = In.Fence;
	}

	FMetalCommandEncoderFence& operator=(FMetalCommandEncoderFence&& lhs)
	{
		if (this != &lhs)
		{
			FenceResources = MoveTemp(lhs.FenceResources);
			Fence = lhs.Fence;
		}
		return *this;
	}

	void Reset()
	{
		FenceResources.Empty();
		Fence = nullptr;
		BarrierScope.Reset();
	}
};

/**
 * FMetalCommandEncoder:
 *	Wraps the details of switching between different command encoders on the command-buffer, allowing for restoration of the render encoder if needed.
 * 	UnrealEngine expects the API to serialise commands in-order, but Metal expects applications to work with command-buffers directly so we need to implement 
 *	the RHI semantics by switching between encoder types. This class hides the ugly details. Eventually it might be possible to move some of the operations
 *	into pre- & post- command-buffers so that we avoid encoder switches but that will take changes to the RHI and high-level code too, so it won't happen soon.
 */
class FMetalCommandEncoder
{
public:
#pragma mark - Public C++ Boilerplate -

	/** Default constructor */
	FMetalCommandEncoder(FMetalDevice& Device, FMetalCommandQueue& CommandQueue);
	
	/** Release Function */
	void Release(void);
	
	/** Reset cached state for reuse */
	void Reset(void);
	
	/** Reset cached state for reuse while in rendering */
	void ResetLive(void);
	
#pragma mark - Public Command Buffer Mutators -

	/**
	 * Start encoding to CommandBuffer. It is an error to call this with any outstanding command encoders or current command buffer.
	 * Instead call EndEncoding & CommitCommandBuffer before calling this.
	 */
	void StartCommandBuffer(void);
	
	/**
	 * Will deprecate in 5.6, splits the command buffer into chunks, required for profiling until we switch to encoder counter buffers
	 */
	void EndCommandBuffer(FMetalRHICommandContext* Context);

#pragma mark - Public Command Buffer Accessors -
	
	/** @returns the current command buffer */
    FMetalCommandBuffer* GetCommandBuffer() { return CommandBuffer; }
	
	/** @returns the monotonically incremented command buffer index */
	uint32 GetCommandBufferIndex() const { return CmdBufIndex; }

#pragma mark - Public Command Encoder Accessors -
	
	/** @returns True if and only if there is an active parallel render command encoder, otherwise false. */
	inline bool IsParallelRenderCommandEncoderActive(void) const
	{
		return ParallelRenderCommandEncoder.get() != nullptr;
	}
	
	/** @returns True if and only if there is an active render command encoder, otherwise false. */
	inline bool IsRenderCommandEncoderActive(void) const
	{
		return RenderCommandEncoder.get() != nullptr;
	}

	/** @returns True if and only if there is an active compute command encoder, otherwise false. */
	inline bool IsComputeCommandEncoderActive(void) const
	{
		return ComputeCommandEncoder.get() != nullptr;
	}

	/** @returns True if and only if there is an active blit command encoder, otherwise false. */
	inline bool IsBlitCommandEncoderActive(void) const
	{
		return BlitCommandEncoder.get() != nullptr;
	}

	/** @returns True if any encoder is active, otherwise false. */
	inline bool IsAnyCommandEncoderActive(void) const
	{
		return IsRenderCommandEncoderActive() ||
				IsComputeCommandEncoderActive() ||
#if METAL_RHI_RAYTRACING
				IsAccelerationStructureCommandEncoderActive() ||
#endif
				IsBlitCommandEncoderActive() ||
				IsParallelRenderCommandEncoderActive();
	}
	
	inline bool IsParallelEncoding()
	{
		return bIsParallelEncoding;
	}
	
#if METAL_RHI_RAYTRACING
	/** @returns True if and only if there is an active acceleration structure command encoder, otherwise false. */
	bool IsAccelerationStructureCommandEncoderActive(void) const;
#endif // METAL_RHI_RAYTRACING

	/** @returns True if and only if there is valid render pass descriptor set on the encoder, otherwise false. */
	bool IsRenderPassDescriptorValid(void) const;
	
	/** @returns The current render pass descriptor. */
	const MTL::RenderPassDescriptor* GetRenderPassDescriptor(void) const;

	/** @returns The active render command encoder or nullptr if there isn't one. */
    MTL::RenderCommandEncoder* GetRenderCommandEncoder(void);
	
	/** @returns The active compute command encoder or nullptr if there isn't one. */
    MTL::ComputeCommandEncoder* GetComputeCommandEncoder(void);
	
	/** @returns The active blit command encoder or nullptr if there isn't one. */
    MTL::BlitCommandEncoder* GetBlitCommandEncoder(void);

#if METAL_RHI_RAYTRACING
	/** @returns The active acceleration structure command encoder or nullptr if there isn't one. */
	MTL::AccelerationStructureCommandEncoder* GetAccelerationStructureCommandEncoder(void);
#endif // METAL_RHI_RAYTRACING
	
	/** @returns The MTLFence for the current encoder or nullptr if there isn't one. */
	TRefCountPtr<FMetalFence> const& GetEncoderFence(void) const;
	
	/** @returns The number of encoded passes in the command buffer. */
	uint32 NumEncodedPasses(void) const { return EncoderNum; }
	
#pragma mark - Public Command Encoder Mutators -

	/**
	 * Begins encoding for parallel rendering commands into the current command buffer. No other encoder may be active
	 */
	MTLParallelRenderCommandEncoderPtr BeginParallelRenderCommandEncoding(FMetalCounterSampler* Sampler);
	
	/**
 	 * Begins encoding rendering commands into the current command buffer. No other encoder may be active
	 */
	void BeginRenderCommandEncoding(FMetalCounterSampler* Sampler);
	
	/**
	 * Begins encoding rendering commands into the parallel render encoder (no command buffer required)
	 */
	void BeginRenderCommandEncoding(MTL::RenderPassDescriptor* RenderPassDesc, MTLParallelRenderCommandEncoderPtr ParallelEncoder);
	
	/** Begins encoding compute commands into the current command buffer. No other encoder may be active. */
	void BeginComputeCommandEncoding(MTL::DispatchType Type = MTL::DispatchTypeSerial, FMetalCounterSampler* Sampler = nullptr);

#if METAL_RHI_RAYTRACING
	/** Begins encoding acceleration structure commands into the current command buffer. No other encoder may be active. */
	void BeginAccelerationStructureCommandEncoding(void);
#endif // METAL_RHI_RAYTRACING
	
	/** Begins encoding blit commands into the current command buffer. No other encoder may be active. */
	void BeginBlitCommandEncoding(FMetalCounterSampler* Sampler);
	
	/** Declare that all command generation from this encoder is complete, and detach from the MTLCommandBuffer if there is an encoder active or does nothing if there isn't. */
	TRefCountPtr<FMetalFence> EndEncoding();
	
	/** Update the event to capture all GPU work so far enqueued by this encoder. */
	void UpdateFence(FMetalFence* Fence);
	
	/** Prevent further GPU work until the event is reached. */
	void WaitForFence(FMetalFence* Fence);
	
	/** Update the event to capture all GPU work so far enqueued by this encoder. */
	void SignalEvent(MTLEventPtr Event, uint32_t SignalCount);
	
	/** Prevent further GPU work until the event is reached. */
	void WaitForEvent(MTLEventPtr Event, uint32_t SignalCount);
	
	/** Update the event to capture all GPU work so far enqueued by this encoder. */
	void SignalSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount);
	
	/** Prevent further GPU work until the event is reached. */
	void WaitForSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount);

#pragma mark - Public Debug Support -
	
	/*
	 * Inserts a debug string into the command buffer.  This does not change any API behavior, but can be useful when debugging.
	 * @param string The name of the signpost. 
	 */
	void InsertDebugSignpost(NS::String* String);
	
	/*
	 * Push a new named string onto a stack of string labels.
	 * @param string The name of the debug group. 
	 */
	void PushDebugGroup(NS::String* String);
	
	/* Pop the latest named string off of the stack. */
	void PopDebugGroup(void);

#pragma mark - Public Render State Mutators -

	/**
	 * Set the render pass descriptor - no encoder may be active when this function is called.
	 * @param RenderPass The render pass descriptor to set. May be nullptr.
	 */
	void SetRenderPassDescriptor(MTL::RenderPassDescriptor* RenderPass);
	
	/**
	 * Set the render pass store actions, call after SetRenderPassDescriptor but before EndEncoding.
	 * @param ColorStore The store actions for color targets.
	 * @param DepthStore The store actions for the depth buffer - use MTL::StoreActionUnknown if no depth-buffer bound.
	 * @param StencilStore The store actions for the stencil buffer - use MTL::StoreActionUnknown if no stencil-buffer bound.
	 */
	void SetRenderPassStoreActions(MTL::StoreAction const* const ColorStore, MTL::StoreAction const DepthStore, MTL::StoreAction const StencilStore);
	
	/*
	 * Sets the current render pipeline state object.
	 * @param PipelineState The pipeline state to set. Must not be null.
	 */
	void SetRenderPipelineState(FMetalShaderPipeline* const PipelineState);
	
	/*
	 * Set the viewport, which is used to transform vertexes from normalized device coordinates to window coordinates.  Fragments that lie outside of the viewport are clipped, and optionally clamped for fragments outside of znear/zfar.
	 * @param Viewport The array of viewport dimensions to use.
	 * @param NumActive The number of active viewport dimensions to use.
	 */
	void SetViewport(MTL::Viewport const Viewport[], uint32 NumActive);
	
	/*
	 * The winding order of front-facing primitives.
	 * @param FrontFacingWinding The front face winding.
	 */
	void SetFrontFacingWinding(MTL::Winding const FrontFacingWinding);
	
	/*
	 * Controls if primitives are culled when front facing, back facing, or not culled at all.
	 * @param CullMode The cull mode.
	 */
	void SetCullMode(MTL::CullMode const CullMode);
	
	/*
	 * Depth Bias.
	 * @param DepthBias The depth-bias value.
	 * @param SlopeScale The slope-scale to apply.
	 * @param Clamp The value to clamp to.
	 */
	void SetDepthBias(float const DepthBias, float const SlopeScale, float const Clamp);
	
	/*
	 * Specifies a rectangle for a fragment scissor test.  All fragments outside of this rectangle are discarded.
	 * @param Rect The array of scissor rect dimensions.
	 * @param NumActive The number of active scissor rect dimensions.
	 */
	void SetScissorRect(MTL::ScissorRect const Rect[], uint32 NumActive);
	
	/*
	 * Set how to rasterize triangle and triangle strip primitives.
	 * @param FillMode The fill mode.
	 */
	void SetTriangleFillMode(MTL::TriangleFillMode const FillMode);

	/*
	 * Set wether to clip or clamp triangles based on depth.
	 * @param FillMode The fill mode.
	 */
	void SetDepthClipMode(MTL::DepthClipMode DepthClipMode);
	
	/*
	 * Set the constant blend color used across all blending on all render targets
	 * @param Red The value for the red channel in 0-1.
	 * @param Green The value for the green channel in 0-1.
	 * @param Blue The value for the blue channel in 0-1.
	 * @param Alpha The value for the alpha channel in 0-1.
	 */
	void SetBlendColor(float const Red, float const Green, float const Blue, float const Alpha);
	
	/*
	 * Set the DepthStencil state object.
	 * @param DepthStencilState The depth-stencil state, must not be null.
	 */
	void SetDepthStencilState(MTL::DepthStencilState* DepthStencilState);
	
	/*
	 * Set the stencil reference value for both the back and front stencil buffers.
	 * @param ReferenceValue The stencil ref value to use.
	 */
	void SetStencilReferenceValue(uint32 const ReferenceValue);
	
	/*
	 * Monitor if samples pass the depth and stencil tests.
	 * @param Mode Controls if the counter is disabled or moniters passing samples.
	 * @param Offset The offset relative to the occlusion query buffer provided when the command encoder was created.  offset must be a multiple of 8.
	 */
	void SetVisibilityResultMode(MTL::VisibilityResultMode const Mode, NS::UInteger const Offset);
	
#pragma mark - Public Shader Resource Mutators -
#if METAL_RHI_RAYTRACING
	/*
	 * Set a global acceleration structure for the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param AccelerationStructure The acceleration structure to bind or nullptr to clear.
	 * @param Index The index to modify.
	 */
	void SetShaderAccelerationStructure(MTL::FunctionType const FunctionType, MTLAccelerationStructurePtr AccelerationStructure, NS::UInteger const Index);
#endif // METAL_RHI_RAYTRACING
	
	/*
	 * Set a global buffer for the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param Buffer The buffer to bind or nullptr to clear.
	 * @param Offset The offset in the buffer or 0 when Buffer is nullptr.
	 * @param Length The data length - caller is responsible for accounting for non-zero Offset.
	 * @param Index The index to modify.
	 * @param Usage The resource usage mask.
	 * @param Format The Pixel format to reinterpret the resource as.
	 * @param ReferencedResources Resources indirectly used by the bound buffer.
	 */
	void SetShaderBuffer(MTL::FunctionType const FunctionType, FMetalBufferPtr Buffer, NS::UInteger const Offset, NS::UInteger const Length, NS::UInteger const Index, MTL::ResourceUsage const Usage, EPixelFormat const Format = PF_Unknown, NS::UInteger const ElementRowPitch = 0, TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources = {});
	
	/*
	 * Set an FMetalBufferData to the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param Data The data to bind or nullptr to clear.
	 * @param Offset The offset in the buffer or 0 when Buffer is nullptr.
	 * @param Index The index to modify.
	 * @param Format The pixel format to reinterpret the resource as.
	 */
	void SetShaderData(MTL::FunctionType const FunctionType, FMetalBufferData* Data, NS::UInteger const Offset, NS::UInteger const Index, EPixelFormat const Format = PF_Unknown, NS::UInteger const ElementRowPitch = 0);
	
	/*
	 * Set bytes to the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param Bytes The data to bind or nullptr to clear.
	 * @param Length The length of the buffer or 0 when Bytes is nullptr.
	 * @param Index The index to modify.
	 */
	void SetShaderBytes(MTL::FunctionType const FunctionType, uint8 const* Bytes, NS::UInteger const Length, NS::UInteger const Index);
	
	/*
	 * Set a global texture for the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param Texture The texture to bind or nullptr to clear.
	 * @param Index The index to modify.
	 * @param Usage The resource usage mask.
	 */
	void SetShaderTexture(MTL::FunctionType const FunctionType, MTL::Texture* Texture, NS::UInteger const Index, MTL::ResourceUsage const Usage);
	
	/*
	 * Set a global sampler for the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param Sampler The sampler state to bind or nullptr to clear.
	 * @param Index The index to modify.
	 */
	void SetShaderSamplerState(MTL::FunctionType const FunctionType, MTL::SamplerState* Sampler, NS::UInteger const Index);
	
	/*
	 * Set the shader side-table data for FunctionType at Index.
	 * @param FunctionType The shader function to modify.
	 * @param Index The index to bind data to.
	 */
	void SetShaderSideTable(MTL::FunctionType const FunctionType, NS::UInteger const Index);
	
    /*
     * Inform the driver that an indirect argument resource will be used and what the usage mode will be.
     * @param Texture The texture that will be used as an indirect argument.
     * @param Usage The usage mode for the texture.
     */
	void UseIndirectArgumentResource(MTL::Texture* Texture, MTL::ResourceUsage const Usage);
    
    /*
     * Inform the driver that an indirect argument resource will be used and what the usage mode will be.
     * @param Buffer The buffer that will be used as an indirect argument.
     * @param Usage The usage mode for the texture.
     */
	void UseIndirectArgumentResource(FMetalBufferPtr Buffer, MTL::ResourceUsage const Usage);

	/*
	 * Transition resource so that we can barrier fragment->vertex stages.
	 * @param Resource The resource we are going to make readable having been written.
	 */
	void TransitionResources(MTL::Resource* Resource);
	
#pragma mark - Public Compute State Mutators -
	
	/*
	 * Set the compute pipeline state that will be used.
	 * @param State The state to set - must not be nullptr.
	 */
	void SetComputePipelineState(FMetalShaderPipelinePtr State);

	void SetComputePipelineState(MTL::ComputePipelineState* State);
#pragma mark - Public Ring-Buffer Accessor -
	
	/*
	 * Get the internal ring-buffer used for temporary allocations.
	 * @returns The temporary allocation buffer for this command-encoder.
	 */
	FMetalSubBufferRing& GetRingBuffer(void);
	
#pragma mark - Public Resource query Access -
	
private:
#pragma mark - Private Functions -
	/*
	 * Set the offset for the buffer bound on the specified shader frequency at the given bind point index.
	 * @param FunctionType The shader function to modify.
	 * @param Offset The offset in the buffer or 0 when Buffer is nullptr.
	 * @param Length The data length - caller is responsible for accounting for non-zero Offset.
	 * @param Index The index to modify.
	 */
	void SetShaderBufferOffset(MTL::FunctionType const FunctionType, NS::UInteger const Offset, NS::UInteger const Length, NS::UInteger const Index);
	
	void SetShaderBufferInternal(MTL::FunctionType Function, uint32 Index);
	
	void FenceResource(MTL::Texture* Resource, const MTL::FunctionType Function, bool bIsRenderTarget = false);
	void FenceResource(MTL::Buffer* Resource, const MTL::FunctionType Function);

public:
	void UseResource(MTL::Resource* Resource, MTL::ResourceUsage const Usage);
	void UseResources(TArray<MTL::Resource*> const& Resources, MTL::ResourceUsage const Usage, MTL::RenderStages RenderStages = 0);
	void UseHeaps(TArray<MTL::Heap*> const& Heaps, const MTL::FunctionType Function);
	
#pragma mark - Private Type Declarations -
private:
    /** A structure of arrays for the current buffer binding settings. */
    struct FMetalBufferBindings
    {
		/** Side-table wrapper object to allow us to use Set*Bytes. */
		FMetalBufferData* SideTable;
        /** The bound buffers or nullptr. */
		FMetalBufferPtr Buffers[ML_MaxBuffers];
		TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources[ML_MaxBuffers];
#if METAL_RHI_RAYTRACING
		/** The bound acceleration structure or nullptr. */
		MTLAccelerationStructurePtr AccelerationStructure[ML_MaxBuffers];
#endif // METAL_RHI_RAYTRACING
        /** The bound buffers or nullptr. */
        FMetalBufferData* Bytes[ML_MaxBuffers];
        /** The bound buffer offsets or 0. */
        NS::UInteger Offsets[ML_MaxBuffers];
		/** The usage mask for the bound resource or 0 */
		MTL::ResourceUsage Usage[ML_MaxBuffers];
		/** The bound buffer constants */
		struct FSizeConstants
		{
			union
			{
				uint32 	Length;
				uint32 	Swizzle;
			};
			uint32 Format;
			uint32 ElementRowPitch;
		};
		FSizeConstants Lengths[ML_MaxBuffers + ML_MaxTextures];
        /** A bitmask for which buffers were bound by the application where a bit value of 1 is bound and 0 is unbound. */
        uint32 Bound;
        
public:
        void SetBufferMetaData(NS::UInteger Index, NS::UInteger Length, NS::UInteger Format, NS::UInteger ElementRowPitch)
        {
			Lengths[Index].Length = Length;
			Lengths[Index].Format = Format;
			Lengths[Index].ElementRowPitch = ElementRowPitch;
        }
        void SetTextureSwizzle(NS::UInteger Index, uint8 (&Swizzle)[4])
        {
			FMemory::Memcpy(&Lengths[ML_MaxBuffers + Index].Swizzle, Swizzle, sizeof(Swizzle));
			Lengths[ML_MaxBuffers + Index].Format = 0;
			Lengths[ML_MaxBuffers + Index].ElementRowPitch = 0;
        }
	};
	
#pragma mark - Private Member Variables -
	FMetalDevice& Device;
	FMetalCommandQueue& CommandQueue;

	FMetalBufferBindings ShaderBuffers[int(MTL::FunctionTypeObject)+1];
	
    MTL::StoreAction ColorStoreActions[MaxSimultaneousRenderTargets];
    MTL::StoreAction DepthStoreAction;
    MTL::StoreAction StencilStoreAction;
	
	FMetalSubBufferRing RingBuffer;
	
    MTL::RenderPassDescriptor* RenderPassDesc = nullptr;
	
    FMetalCommandBuffer* CommandBuffer = nullptr;
	
	MTLParallelRenderCommandEncoderPtr ParallelRenderCommandEncoder;
    MTLRenderCommandEncoderPtr RenderCommandEncoder;
    MTLComputeCommandEncoderPtr ComputeCommandEncoder;
    MTLBlitCommandEncoderPtr BlitCommandEncoder;
#if METAL_RHI_RAYTRACING
    MTLAccelerationStructureCommandEncoderPtr AccelerationStructureCommandEncoder;
#endif // METAL_RHI_RAYTRACING
	
	TRefCountPtr<FMetalFence> EncoderFence;

	TArray<NS::String*> DebugGroups;
    
    TSet<FMetalBufferPtr> ActiveBuffers;
	
	TMap<MTL::Resource*, MTL::ResourceUsage> ResourceUsage;
	
	TSet<MTL::Resource*> TransitionedResources;
	FMetalCommandEncoderFence CommandEncoderFence;
	uint32 EncoderNum;
	uint32 CmdBufIndex;
	bool bIsParallelEncoding = false;
};
