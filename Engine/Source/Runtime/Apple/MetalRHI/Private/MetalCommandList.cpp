// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCommandList.cpp: Metal command buffer list wrapper.
=============================================================================*/

#include "MetalRHIPrivate.h"
#include "MetalShaderTypes.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalCommandQueue.h"
#include "MetalProfiler.h"
#include "MetalCommandBuffer.h"

#pragma mark - Public C++ Boilerplate -

#if PLATFORM_IOS
extern bool GIsSuspended;
#endif
	
#pragma mark - Public Command List Mutators -

static const TCHAR* StringFromCommandEncoderError(MTL::CommandEncoderErrorState ErrorState)
{
    switch (ErrorState)
    {
        case MTL::CommandEncoderErrorStateUnknown: return TEXT("Unknown");
        case MTL::CommandEncoderErrorStateAffected: return TEXT("Affected");
        case MTL::CommandEncoderErrorStateCompleted: return TEXT("Completed");
        case MTL::CommandEncoderErrorStateFaulted: return TEXT("Faulted");
        case MTL::CommandEncoderErrorStatePending: return TEXT("Pending");
    }
    return TEXT("Unknown");
}

extern CORE_API bool GIsGPUCrashed;
static void ReportMetalCommandBufferFailure(MTL::CommandBuffer* CompletedBuffer, TCHAR const* ErrorType, bool bDoCheck=true)
{
	GIsGPUCrashed = true;
	
	NS::String* Label = CompletedBuffer->label();
	int32 Code = CompletedBuffer->error()->code();
    NS::String* Domain = CompletedBuffer->error()->domain();
    NS::String* ErrorDesc = CompletedBuffer->error()->localizedDescription();
    NS::String* FailureDesc = CompletedBuffer->error()->localizedFailureReason();
    NS::String* RecoveryDesc = CompletedBuffer->error()->localizedRecoverySuggestion();
	
	FString LabelString = Label ? FString(Label->cString(NS::UTF8StringEncoding)) : FString("Unknown");
	FString DomainString = Domain ? FString(Domain->cString(NS::UTF8StringEncoding)) : FString("Unknown");
	FString ErrorString = ErrorDesc ? FString(ErrorDesc->cString(NS::UTF8StringEncoding)) : FString("Unknown");
	FString FailureString = FailureDesc ? FString(FailureDesc->cString(NS::UTF8StringEncoding)) : FString("Unknown");
	FString RecoveryString = RecoveryDesc ? FString(RecoveryDesc->cString(NS::UTF8StringEncoding)) : FString("Unknown");
	
	NS::String* Desc = CompletedBuffer->debugDescription();
	UE_LOGF(LogMetal, Warning, "Metal Command Buffer Failure: %ls, %ls", ErrorType, *FString(Desc->cString(NS::UTF8StringEncoding)));
	
#if PLATFORM_IOS
    if (bDoCheck && !GIsSuspended)
#endif
    {
        // Dump GPU fault information for the GPU encoders
        if (&MTLCommandBufferEncoderInfoErrorKey != nullptr)
        {
            NS::Dictionary* ErrorDict = CompletedBuffer->error()->userInfo();
            NS::Array* EncoderInfoArray = (NS::Array*)ErrorDict->object(MTL::CommandBufferEncoderInfoErrorKey);
            if (EncoderInfoArray)
            {
                UE_LOGF(LogMetal, Warning, "GPU Encoder Crash Info:");
                for(uint32 Idx = 0; Idx < EncoderInfoArray->count(); ++Idx)
                {
                    MTL::CommandBufferEncoderInfo* EncoderInfo = (MTL::CommandBufferEncoderInfo*)EncoderInfoArray->object(Idx);
                    UE_LOGF(LogMetal, Warning, "MTLCommandBufferEncoder - Label: %ls, State: %ls", *NSStringToFString(EncoderInfo->label()), StringFromCommandEncoderError(EncoderInfo->errorState()));
                    NS::Array* SignPosts = EncoderInfo->debugSignposts();
                    if (SignPosts->count() > 0)
                    {
                        UE_LOGF(LogMetal, Warning, "    Signposts:");
                        for (uint32_t SignPostIdx = 0; SignPostIdx < SignPosts->count(); ++SignPostIdx)
                        {
                            NS::String* Signpost = (NS::String*)SignPosts->object(SignPostIdx);
                            UE_LOGF(LogMetal, Warning, "    - %ls", *NSStringToFString(Signpost));
                        }
                    }
                }
            }
        }
        
#if PLATFORM_IOS
        UE_LOGF(LogMetal, Warning, "Command Buffer %ls Failed with %ls Error! Error Domain: %ls Code: %d Description %ls %ls %ls", *LabelString, ErrorType, *DomainString, Code, *ErrorString, *FailureString, *RecoveryString);
        FIOSPlatformMisc::GPUAssert();
#else
		UE_LOGF(LogMetal, Fatal, "Command Buffer %ls Failed with %ls Error! Error Domain: %ls Code: %d Description %ls %ls %ls", *LabelString, ErrorType, *DomainString, Code, *ErrorString, *FailureString, *RecoveryString);
#endif
    }
}

static __attribute__ ((optnone)) void MetalCommandBufferFailureInternal(MTL::CommandBuffer* CompletedBuffer)
{
	ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("Internal"));
}

static __attribute__ ((optnone)) void MetalCommandBufferFailureTimeout(MTL::CommandBuffer* CompletedBuffer)
{
    ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("Timeout"), PLATFORM_IOS);
}

static __attribute__ ((optnone)) void MetalCommandBufferFailurePageFault(MTL::CommandBuffer* CompletedBuffer)
{
	ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("PageFault"));
}

static __attribute__ ((optnone)) void MetalCommandBufferFailureAccessRevoked(MTL::CommandBuffer* CompletedBuffer)
{
	ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("AccessRevoked"));
}

static __attribute__ ((optnone)) void MetalCommandBufferFailureNotPermitted(MTL::CommandBuffer* CompletedBuffer)
{
	// when iOS goes into the background, it can get a delayed NotPermitted error, so we can't crash in this case, just allow it to not be submitted
	ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("NotPermitted"), !PLATFORM_IOS);
}

static __attribute__ ((optnone)) void MetalCommandBufferFailureOutOfMemory(MTL::CommandBuffer* CompletedBuffer)
{
	ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("OutOfMemory"));
}

static __attribute__ ((optnone)) void MetalCommandBufferFailureInvalidResource(MTL::CommandBuffer* CompletedBuffer)
{
	ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("InvalidResource"));
}

static void HandleMetalCommandBufferError(MTL::CommandBuffer* CompletedBuffer)
{
    MTL::CommandBufferError Code = (MTL::CommandBufferError)CompletedBuffer->error()->code();
	switch(Code)
	{
        case MTL::CommandBufferErrorInternal:
			MetalCommandBufferFailureInternal(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorTimeout:
			MetalCommandBufferFailureTimeout(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorPageFault:
			MetalCommandBufferFailurePageFault(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorAccessRevoked:
			MetalCommandBufferFailureAccessRevoked(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorNotPermitted:
			MetalCommandBufferFailureNotPermitted(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorOutOfMemory:
			MetalCommandBufferFailureOutOfMemory(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorInvalidResource:
			MetalCommandBufferFailureInvalidResource(CompletedBuffer);
			break;
        case MTL::CommandBufferErrorNone:
			// No error
			break;
		default:
			ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("Unknown"));
			break;
	}
}

void FMetalCommandQueue::HandleMetalCommandBufferFailure(MTL::CommandBuffer* CompletedBuffer)
{
	if (CompletedBuffer->error()->domain()->isEqualToString(NS::String::string("MTLCommandBufferErrorDomain", NS::UTF8StringEncoding)))
	{
		HandleMetalCommandBufferError(CompletedBuffer);
	}
	else
	{
		ReportMetalCommandBufferFailure(CompletedBuffer, TEXT("Unknown"));
	}
}
