// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCommandQueue.cpp: Metal command queue wrapper..
=============================================================================*/

#include "MetalCommandQueue.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalCommandBuffer.h"
#include "MetalDevice.h"
#include "MetalFence.h"
#include "MetalProfiler.h"
#include "MetalRHIPrivate.h"
#include "Misc/ConfigCacheIni.h"
#include "MetalDynamicRHI.h"

#if !UE_BUILD_SHIPPING
#import "MetalThirdParty.h"
#endif

#pragma mark - Private C++ Statics -
NS::UInteger FMetalCommandQueue::PermittedOptions = 0;

bool GMetalCommandBufferDebuggingEnabled = 0;

#pragma mark - Public C++ Boilerplate -

FMetalTiming::FMetalTiming(FMetalCommandQueue& Queue) 
	: Queue(Queue)
	, EventStream(Queue.GetProfilerQueue())
{
}

FMetalCommandQueue::FMetalCommandQueue(FMetalDevice& MetalDevice, uint32 const MaxNumCommandBuffers /* = 0 */)
	: Device(MetalDevice)
	, RuntimeDebuggingLevel(EMetalDebugLevelOff)
{
    int32 MetalShaderVersion = 0;
#if PLATFORM_MAC
	const TCHAR* const Settings = TEXT("/Script/MacTargetPlatform.MacTargetSettings");
#else
	const TCHAR* const Settings = TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings");
#endif
    GConfig->GetInt(Settings, TEXT("MetalLanguageVersion"), MetalShaderVersion, GEngineIni);
	
	MTL::LanguageVersion ShaderVersion = ValidateVersion(MetalShaderVersion);
	if(Device.IsShaderValidationEnabled() && ShaderVersion < MTL::LanguageVersion3_0)
	{
		UE_LOGF(LogMetal, Warning, "Shader validation is enabled, but ShaderVersion is less than 3.0, change MetalLanguageVersion to 3.0+ for Line/Trace info");
	}

	if(MaxNumCommandBuffers == 0)
	{
		CommandQueue = Device.GetDevice()->newCommandQueue();
	}
	else
	{
		CommandQueue = Device.GetDevice()->newCommandQueue(MaxNumCommandBuffers);
	}
	check(CommandQueue);
	
#if PLATFORM_IOS
#if !PLATFORM_TVOS
	if (Device.GetDevice()->supportsFeatureSet(MTL::FeatureSet_iOS_GPUFamily4_v1))
	{
		// The below implies tile shaders which are necessary to order the draw calls and generate a buffer that shows what PSOs/draws ran on each tile.
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
		GMetalCommandBufferDebuggingEnabled = UE::RHI::UseGPUCrashDebugging() || FParse::Param(FCommandLine::Get(), TEXT("metalgpudebug"));
#else
		GMetalCommandBufferDebuggingEnabled = true;
#endif
	}
#endif
#else // Assume that Mac & other platforms all support these from the start. They can diverge later.
    
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	GMetalCommandBufferDebuggingEnabled = UE::RHI::UseGPUCrashDebugging() || FParse::Param(FCommandLine::Get(),TEXT("metalgpudebug"));
#else
	GMetalCommandBufferDebuggingEnabled = true;
#endif
#endif
	
	PermittedOptions = 0;
	PermittedOptions |= MTL::ResourceCPUCacheModeDefaultCache;
	PermittedOptions |= MTL::ResourceCPUCacheModeWriteCombined;

	PermittedOptions |= MTL::ResourceStorageModeShared;
	PermittedOptions |= MTL::ResourceStorageModePrivate;
#if PLATFORM_MAC
	PermittedOptions |= MTL::ResourceStorageModeManaged;
#else
	PermittedOptions |= MTL::ResourceStorageModeMemoryless;
#endif
	PermittedOptions |= MTL::ResourceHazardTrackingModeTracked;
	
	SignalEvent.MetalEvent = Device.GetDevice()->newEvent();
}

FMetalCommandQueue::~FMetalCommandQueue(void)
{
	SignalEvent.MetalEvent->release();
}
	
#pragma mark - Public Command Buffer Mutators -

FMetalCommandBuffer* FMetalCommandQueue::CreateCommandBuffer(void)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
#if PLATFORM_MAC
	static bool bUnretainedRefs = FParse::Param(FCommandLine::Get(),TEXT("metalunretained"))
	|| !FParse::Param(FCommandLine::Get(),TEXT("metalretainrefs"));
#else
	static bool bUnretainedRefs = !FParse::Param(FCommandLine::Get(),TEXT("metalretainrefs"));
#endif
	
    MTL::CommandBufferDescriptor* CmdBufferDesc = MTL::CommandBufferDescriptor::alloc()->init();
    check(CmdBufferDesc);
    
    CmdBufferDesc->setRetainedReferences(!bUnretainedRefs);
    CmdBufferDesc->setErrorOptions(GMetalCommandBufferDebuggingEnabled ? MTL::CommandBufferErrorOptionEncoderExecutionStatus : MTL::CommandBufferErrorOptionNone);
    
	MTL::CommandBuffer* CmdBuffer = CommandQueue->commandBuffer(CmdBufferDesc);
    FMetalCommandBuffer* CommandBuffer = new FMetalCommandBuffer(CmdBuffer, *this);
    
	CmdBufferDesc->release();
	
	INC_DWORD_STAT(STAT_MetalCommandBufferCreatedPerFrame);
	return CommandBuffer;
}

void FMetalCommandQueue::CommitCommandBuffer(FMetalCommandBuffer* CommandBuffer)
{
	check(CommandBuffer);
	INC_DWORD_STAT(STAT_MetalCommandBufferCommittedPerFrame);
	
    CommandBuffer->GetMTLCmdBuffer()->commit();
    
	// Wait for completion when debugging command-buffers.
	if (RuntimeDebuggingLevel >= EMetalDebugLevelWaitForComplete)
	{
		CommandBuffer->GetMTLCmdBuffer()->waitUntilCompleted();
	}
}

FMetalFence* FMetalCommandQueue::CreateFence(NS::String* Label) const
{
	if (Device.SupportsFeature(EMetalFeaturesFences))
	{
		FMetalFence* InternalFence = FMetalFencePool::Get().AllocateFence();
		{
			MTL::Fence* InnerFence = InternalFence->Get();
			NS::String* String = nullptr;
			if (GetEmitDrawEvents())
			{
                NS::String* FenceString = FStringToNSString(FString::Printf(TEXT("%p"), InnerFence));
                String = FenceString->stringByAppendingString(Label);
			}

			if(InnerFence && String)
            {
                InnerFence->setLabel(String);
            }
		}
		return InternalFence;
	}
	else
	{
		return nullptr;
	}
}

#pragma mark - Public Command Queue Accessors -
	
FMetalDevice& FMetalCommandQueue::GetDevice(void)
{
	return Device;
}

MTL::ResourceOptions FMetalCommandQueue::GetCompatibleResourceOptions(MTL::ResourceOptions Options)
{
	NS::UInteger NewOptions = (Options & PermittedOptions);
#if PLATFORM_IOS // Swizzle Managed to Shared for iOS - we can do this as they are equivalent, unlike Shared -> Managed on Mac.
	if ((Options & (1 /*MTL::StorageModeManaged*/ << MTL::ResourceStorageModeShift)))
	{
#if WITH_IOS_SIMULATOR
		NewOptions |= MTL::ResourceStorageModePrivate;
#else
		NewOptions |= MTL::ResourceStorageModeShared;
#endif
	}
#endif
	return (MTL::ResourceOptions)NewOptions;
}

#pragma mark - Public Debug Support -

void FMetalCommandQueue::InsertDebugCaptureBoundary(void)
{
	CommandQueue->insertDebugCaptureBoundary();
}

UE::RHI::GPUProfiler::FQueue FMetalCommandQueue::GetProfilerQueue() const
{
	UE::RHI::GPUProfiler::FQueue Queue;
	Queue.GPU = 0;
	Queue.Index = 0;

	// TODO - Carl: Multiple queues
	Queue.Type = UE::RHI::GPUProfiler::FQueue::EType::Graphics;
	
	return Queue;
}
