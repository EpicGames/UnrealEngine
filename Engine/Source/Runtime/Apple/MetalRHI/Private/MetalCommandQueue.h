// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"
#include "MetalSubmission.h"
#include "RHI.h"
#include "RHIFeatureLevel.h"
#include "Containers/LockFreeList.h"

class FMetalCommandBuffer;
class FMetalFence;
class FMetalDevice;

class FMetalTiming
{
public:
	FMetalCommandQueue& Queue;

	// Timer calibration data
	uint64 GPUFrequency = 0, GPUTimestamp = 0;
	uint64 CPUFrequency = 0, CPUTimestamp = 0;

	UE::RHI::GPUProfiler::FEventStream EventStream;

	FMetalTiming(FMetalCommandQueue& Queue);
};

/**
 * FMetalCommandQueue:
 */
class FMetalCommandQueue
{
public:
#pragma mark - Public C++ Boilerplate -

	/**
	 * Constructor
	 * @param Device The Metal device to create on.
	 * @param MaxNumCommandBuffers The maximum number of incomplete command-buffers, defaults to 0 which implies the system default.
	 */
	FMetalCommandQueue(FMetalDevice& Device, uint32 const MaxNumCommandBuffers = 0);
	
	/** Destructor */
	~FMetalCommandQueue(void);
	
#pragma mark - Public Command Buffer Mutators -

	/**
	 * Start encoding to CommandBuffer. It is an error to call this with any outstanding command encoders or current command buffer.
	 * Instead call EndEncoding & CommitCommandBuffer before calling this.
	 * @param CommandBuffer The new command buffer to begin encoding to.
	 */
    FMetalCommandBuffer* CreateCommandBuffer(void);
	
	/**
	 * Commit the supplied command buffer immediately.
	 * @param CommandBuffer The command buffer to commit, must not be null
 	 */
	void CommitCommandBuffer(FMetalCommandBuffer* CommandBuffer);

	/** @returns Creates a new MTLFence or nullptr if this is unsupported */
	FMetalFence* CreateFence(NS::String* Label) const;
	
#pragma mark - Public Command Queue Accessors -
	
	/** @returns The command queue's native device. */
	FMetalDevice& GetDevice(void);
	
	/** @returns The command queue's native device. */
	MTL::CommandQueue* GetQueue(void) { return CommandQueue; }

	/** Converts a Metal v1.1+ resource option to something valid on the current version. */
	static MTL::ResourceOptions GetCompatibleResourceOptions(MTL::ResourceOptions Options);
	
	/**
	* @param InFeature A specific Metal feature to check for.
	* @returns True if RHISupportsSeparateMSAAAndResolveTextures will be true.  
	* Currently Mac only.
	*/
	static inline bool SupportsSeparateMSAAAndResolveTarget() { return (PLATFORM_MAC != 0 || GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5); }

	/** @returns True on UMA system; false otherwise.  */
	static inline bool IsUMASystem() { return IsRHIDeviceApple(); }

#pragma mark - Public Debug Support -

	/** Inserts a boundary that marks the end of a frame for the debug capture tool. */
	void InsertDebugCaptureBoundary(void);
	
	struct : public TQueue<FMetalPayload*, EQueueMode::Mpsc>
	{
		FMetalPayload* Peek()
		{
			if (FMetalPayload** Result = TQueue::Peek())
			{
				return *Result;
			}

			return nullptr;
		}		
	} PendingSubmission, PendingInterrupt;

	FMetalPayload* PayloadToSubmit  = nullptr;
	
	using FPayloadArray = TArray<FMetalPayload*>;

	// Batches the current payload's command lists, returning the latest fence value signaled for this queue.
	uint64 FinalizePayload(bool bRequiresSignal, FPayloadArray& PayloadsToHandDown);
	
	FMetalSignalEvent& GetSignalEvent()
	{
		return SignalEvent;
	}

	TArray<FMetalRHIRenderQuery*> TimestampQueries;
	TArray<FMetalRHIRenderQuery*> OcclusionQueries;
	TArray<FMetalCounterSamplePtr> CounterSamples;

	UE::RHI::GPUProfiler::FQueue GetProfilerQueue() const;
	FMetalTiming* Timing = nullptr;
	
	/**
	 * Command buffer failure reporting function.
	 * @param CompletedBuffer The buffer to check for failure.
	 */
	static void HandleMetalCommandBufferFailure(MTL::CommandBuffer* CompletedBuffer);

private:
#pragma mark - Private Member Variables -
	FMetalDevice& Device;
	MTL::CommandQueue* CommandQueue;
	int32 RuntimeDebuggingLevel;
	static NS::UInteger PermittedOptions;
	
	FMetalSignalEvent SignalEvent;
};
