// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCommands.cpp: Metal RHI commands implementation.
=============================================================================*/

#include "MetalRHIPrivate.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalShaderTypes.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalVertexDeclaration.h"
#include "MetalRHIContext.h"
#include "GlobalShader.h"
#include "OneColorShader.h"
#include "RHICommandList.h"
#include "RHIStaticStates.h"
#include "ShaderParameterUtils.h"
#include "SceneUtils.h"
#include "MetalProfiler.h"
#include "MetalCommandBuffer.h"
#include "StaticBoundShaderState.h"
#include "EngineGlobals.h"
#include "PipelineStateCache.h"
#include "RHICoreShader.h"
#include "RHIShaderParametersShared.h"
#include "RHIUtilities.h"
#include "MetalBindlessDescriptors.h"
#include "MetalResourceCollection.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIUniformBufferUtilities.h"

static const bool GUsesInvertedZ = true;

/** Vertex declaration for just one FVector4 position. */
class FVector4VertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float4, 0, sizeof(FVector4f)));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};
static TGlobalResource<FVector4VertexDeclaration> FVector4VertexDeclaration;

MTL::PrimitiveType TranslatePrimitiveType(uint32 PrimitiveType)
{
	switch (PrimitiveType)
	{
		case PT_TriangleList:	return MTL::PrimitiveTypeTriangle;
		case PT_TriangleStrip:	return MTL::PrimitiveTypeTriangleStrip;
		case PT_LineList:		return MTL::PrimitiveTypeLine;
		case PT_PointList:		return MTL::PrimitiveTypePoint;
		default:
			METAL_FATAL_ERROR(TEXT("Unsupported primitive type %d"), (int32)PrimitiveType);
			return MTL::PrimitiveTypeTriangle;
	}
}

static FORCEINLINE EMetalShaderStages GetShaderStage(EShaderFrequency ShaderFrequency)
{
	EMetalShaderStages Stage = EMetalShaderStages::Num;
	switch (ShaderFrequency)
	{
	case SF_Vertex:		Stage = EMetalShaderStages::Vertex; break;
	case SF_Pixel:		Stage = EMetalShaderStages::Pixel; break;
	case SF_Compute:	Stage = EMetalShaderStages::Compute; break;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	case SF_Geometry:   Stage = EMetalShaderStages::Geometry; break;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:               Stage = EMetalShaderStages::Mesh; break;
	case SF_Amplification:      Stage = EMetalShaderStages::Amplification; break;
#endif
	default:
		checkf(0, TEXT("FRHIShader Type %d is invalid or unsupported!"), (int32)ShaderFrequency);
		NOT_SUPPORTED("RHIShaderStage");
		break;
	}

	return Stage;
}

static FORCEINLINE EMetalShaderStages GetShaderStage(FRHIGraphicsShader* ShaderRHI)
{
	EMetalShaderStages Stage = EMetalShaderStages::Num;
	return GetShaderStage(ShaderRHI->GetFrequency());
}

void FMetalRHICommandContext::RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBufferRHI,uint32 Offset)
{
    MTL_SCOPED_AUTORELEASE_POOL;;
    
    FMetalRHIBuffer* VertexBuffer = ResourceCast(VertexBufferRHI);
    
    FMetalBufferPtr TheBuffer = nullptr;
    if (VertexBuffer)
    {
        TheBuffer = VertexBuffer->GetCurrentBuffer();
    }
    
    StateCache.SetVertexStream(StreamIndex, VertexBuffer ? TheBuffer : nullptr, nullptr, Offset, VertexBuffer ? VertexBuffer->GetSize() : 0);
}

static void SetUniformBufferInternal(FMetalStateCache& StateCache, FMetalShaderData* ShaderData, EMetalShaderStages Stage, uint32 BufferIndex, FRHIUniformBuffer* UBRHI)
{
    StateCache.BindUniformBuffer(Stage, BufferIndex, UBRHI);
        
    FMetalShaderBindings& Bindings = ShaderData->Bindings;
    if ((Bindings.ConstantBuffers) & (1 << BufferIndex))
    {
        FMetalUniformBuffer* UB = ResourceCast(UBRHI);
#if METAL_USE_METAL_SHADER_CONVERTER
        if (IsMetalBindlessEnabled())
        {
            StateCache.IRBindUniformBuffer(Stage, BufferIndex, UB);
        }
        else
#endif
        {
            StateCache.SetShaderBuffer(Stage, UB->BackingBuffer, nullptr,
									   0, UB->GetSize(),
									   BufferIndex, MTL::ResourceUsageRead);
        }
    }
}

inline FMetalShaderData* GetShaderData(FRHIShader* InShaderRHI, EMetalShaderStages Stage)
{
    switch (Stage)
    {
    case EMetalShaderStages::Vertex:        return ResourceCast(static_cast<FRHIVertexShader*>(InShaderRHI));
#if PLATFORM_SUPPORTS_MESH_SHADERS
    case EMetalShaderStages::Mesh:          return ResourceCast(static_cast<FRHIMeshShader*>(InShaderRHI));
    case EMetalShaderStages::Amplification: return ResourceCast(static_cast<FRHIAmplificationShader*>(InShaderRHI));
#endif
    case EMetalShaderStages::Pixel:         return ResourceCast(static_cast<FRHIPixelShader*>(InShaderRHI));
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    case EMetalShaderStages::Geometry:      return ResourceCast(static_cast<FRHIGeometryShader*>(InShaderRHI));
#endif
    case EMetalShaderStages::Compute:       return ResourceCast(static_cast<FRHIComputeShader*>(InShaderRHI));
            
    default:
        checkf(0, TEXT("FRHIShader Type %d is invalid or unsupported!"), (int32)InShaderRHI->GetFrequency());
        NOT_SUPPORTED("RHIShaderStage");
    }
    return nullptr;
}

static void BindUniformBuffer(FMetalStateCache& StateCache, FRHIShader* Shader, EMetalShaderStages Stage, uint32 BufferIndex, FRHIUniformBuffer* InBuffer)
{
    if (FMetalShaderData* ShaderData = GetShaderData(Shader, Stage))
    {
        SetUniformBufferInternal(StateCache, ShaderData, Stage, BufferIndex, InBuffer);
    }
}

static void ApplyStaticUniformBuffersOnContext(FMetalRHICommandContext& Context,
											   FMetalStateCache& StateCache,
											   FRHIShader* Shader,
											   FMetalShaderData* ShaderData)
{
    if (Shader)
    {
        MTL_SCOPED_AUTORELEASE_POOL;

        const EMetalShaderStages Stage = GetMetalShaderFrequency(Shader->GetFrequency());

        UE::RHICore::ApplyStaticUniformBuffers(
            Shader,
            Context.GetStaticUniformBuffers(),
            [&StateCache, ShaderData, Stage](int32 BufferIndex, FRHIUniformBuffer* Buffer)
            {
                SetUniformBufferInternal(StateCache, ShaderData, Stage, BufferIndex, ResourceCast(Buffer));
            }
        );
    }
}

template <typename TRHIShader>
static void ApplyStaticUniformBuffersOnContext(FMetalRHICommandContext& Context,
											   FMetalStateCache& StateCache,
											   TRefCountPtr<TRHIShader>& Shader)
{
    if (IsValidRef(Shader))
    {
        ApplyStaticUniformBuffersOnContext(Context, StateCache, Shader, static_cast<FMetalShaderData*>(Shader.GetReference()));
    }
}

void FMetalRHICommandContext::RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	
	check(!bWithinRenderPass);
	
	PushDescriptorUpdates();
    
    FMetalComputeShader* ComputeShader = ResourceCast(ComputePipelineState->GetComputeShader());
	
	// cache this for Dispatch
	// sets this compute shader pipeline as the current (this resets all state, so we need to set all resources after calling this)
	StateCache.SetComputeShader(ComputeShader);

    ApplyStaticUniformBuffersOnContext(*this, StateCache, ComputeShader, static_cast<FMetalShaderData*>(ComputeShader));
}

#if METAL_USE_METAL_SHADER_CONVERTER

#if PLATFORM_SUPPORTS_MESH_SHADERS

static void IRBindIndirectMeshDrawArguments(MTL::RenderCommandEncoder* Encoder, MTL::PrimitiveType PrimitiveType, FMetalBufferPtr TheBackingBuffer, const uint32 ArgumentOffset, FMetalStateCache& State)
{
	IRRuntimeDrawInfo DrawInfos = { 0 };
	DrawInfos.primitiveTopology = static_cast<uint8_t>(PrimitiveType);
	
	Encoder->useResource(TheBackingBuffer->GetMTLBuffer(), MTL::ResourceUsageRead);
	
	Encoder->setMeshBuffer(TheBackingBuffer->GetMTLBuffer(), TheBackingBuffer->GetOffset() + ArgumentOffset, kIRArgumentBufferDrawArgumentsBindPoint);
	Encoder->setMeshBytes(&DrawInfos, sizeof(IRRuntimeDrawInfo), kIRArgumentBufferUniformsBindPoint);
	
	Encoder->setObjectBuffer(TheBackingBuffer->GetMTLBuffer(), TheBackingBuffer->GetOffset() + ArgumentOffset, kIRArgumentBufferDrawArgumentsBindPoint);
	Encoder->setObjectBytes(&DrawInfos, sizeof(IRRuntimeDrawInfo), kIRArgumentBufferUniformsBindPoint);
	
	State.IRMapVertexBuffers(Encoder, true);
}

#endif

static IRRuntimeDrawInfo IRRuntimeCalculateDrawInfoForGSEmulation(IRRuntimePrimitiveType primitiveType, uint32 vertexSizeInBytes, uint32 maxInputPrimitivesPerMeshThreadgroup, uint32 instanceCount)
{
	const uint32_t PrimitiveVertexCount = IRRuntimePrimitiveTypeVertexCount(primitiveType);
	const uint32_t Alignment = PrimitiveVertexCount;

	const uint32_t TotalPayloadBytes = 16384;
	const uint32_t PayloadBytesForMetadata = 32;
	const uint32_t PayloadBytesForVertexData = TotalPayloadBytes - PayloadBytesForMetadata;

	const uint32_t MaxVertexCountLimitedByPayloadMemory = (((PayloadBytesForVertexData / vertexSizeInBytes)) / Alignment) * Alignment;

	const uint32_t MaxMeshThreadgroupsPerObjectThreadgroup = 1024;
	const uint32_t MaxPrimCountLimitedByAmplificationRate = MaxMeshThreadgroupsPerObjectThreadgroup * maxInputPrimitivesPerMeshThreadgroup;
	uint32_t MaxPrimsPerObjectThreadgroup = FMath::Min(MaxVertexCountLimitedByPayloadMemory / PrimitiveVertexCount, MaxPrimCountLimitedByAmplificationRate);

	const uint32_t MaxThreadsPerThreadgroup = 256;
	MaxPrimsPerObjectThreadgroup = FMath::Min(MaxPrimsPerObjectThreadgroup, MaxThreadsPerThreadgroup / PrimitiveVertexCount);

	IRRuntimeDrawInfo Infos = {0};
	Infos.primitiveTopology = (uint8)primitiveType;
	Infos.threadsPerPatch = PrimitiveVertexCount;
	Infos.maxInputPrimitivesPerMeshThreadgroup = maxInputPrimitivesPerMeshThreadgroup;
	Infos.objectThreadgroupVertexStride = (uint16)(MaxPrimsPerObjectThreadgroup * PrimitiveVertexCount);
	Infos.meshThreadgroupPrimitiveStride = (uint16)maxInputPrimitivesPerMeshThreadgroup;
	Infos.gsInstanceCount = (uint16)instanceCount;
	Infos.patchesPerObjectThreadgroup = (uint16)MaxPrimsPerObjectThreadgroup;
	Infos.inputControlPointsPerPatch = (uint8)PrimitiveVertexCount;
	
	return Infos;
}
#endif

void FMetalRHICommandContext::RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
	RHI_DISPATCH_CALL_INC();

    MTL_SCOPED_AUTORELEASE_POOL;
    
	ThreadGroupCountX = FMath::Max(ThreadGroupCountX, 1u);
	ThreadGroupCountY = FMath::Max(ThreadGroupCountY, 1u);
	ThreadGroupCountZ = FMath::Max(ThreadGroupCountZ, 1u);
	
	BeginComputeEncoder();
	check(CurrentEncoder.GetCommandBuffer());
	check(CurrentEncoder.IsComputeCommandEncoderActive());

	PrepareToDispatch();
	
	// Bind shader resources
	if(!IsMetalBindlessEnabled())
	{
		StateCache.CommitResourceTable(EMetalShaderStages::Compute, MTL::FunctionTypeKernel, CurrentEncoder);
		
		FMetalComputeShader const* ComputeShader = StateCache.GetComputeShader();
		if (ComputeShader->SideTableBinding >= 0)
		{
			CurrentEncoder.SetShaderSideTable(MTL::FunctionTypeKernel, ComputeShader->SideTableBinding);
			StateCache.SetShaderBuffer(EMetalShaderStages::Compute, nullptr, nullptr, 0, 0, ComputeShader->SideTableBinding, MTL::ResourceUsage(0));
		}
	}
	
	StateCache.SetComputePipelineState(CurrentEncoder);
	
	TRefCountPtr<FMetalComputeShader> ComputeShader = StateCache.GetComputeShader();
	check(ComputeShader);
	
	MTL::Size ThreadgroupCounts = MTL::Size(ComputeShader->NumThreadsX, ComputeShader->NumThreadsY, ComputeShader->NumThreadsZ);
	check(ComputeShader->NumThreadsX > 0 && ComputeShader->NumThreadsY > 0 && ComputeShader->NumThreadsZ > 0);
	MTL::Size Threadgroups = MTL::Size(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
	CurrentEncoder.GetComputeCommandEncoder()->dispatchThreadgroups(Threadgroups, ThreadgroupCounts);
	
	EndComputeEncoder();
}

void FMetalRHICommandContext::RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DISPATCH_CALL_INC();

    MTL_SCOPED_AUTORELEASE_POOL;
    
	if (Device.SupportsFeature(EMetalFeaturesIndirectBuffer))
	{
		FMetalRHIBuffer* VertexBuffer = ResourceCast(ArgumentBufferRHI);
		
		check(VertexBuffer);
		{
			BeginComputeEncoder();
			
			check(CurrentEncoder.GetCommandBuffer());
			check(CurrentEncoder.IsComputeCommandEncoderActive());
			
			PrepareToDispatch();
			
			// Bind shader resources
			if(!IsMetalBindlessEnabled())
			{
				StateCache.CommitResourceTable(EMetalShaderStages::Compute, MTL::FunctionTypeKernel, CurrentEncoder);
				
				FMetalComputeShader const* ComputeShader = StateCache.GetComputeShader();
				if (ComputeShader->SideTableBinding >= 0)
				{
					CurrentEncoder.SetShaderSideTable(MTL::FunctionTypeKernel, ComputeShader->SideTableBinding);
					StateCache.SetShaderBuffer(EMetalShaderStages::Compute, nullptr, nullptr, 0, 0, ComputeShader->SideTableBinding, MTL::ResourceUsage(0));
				}
			}
			
			StateCache.SetComputePipelineState(CurrentEncoder);
			
			TRefCountPtr<FMetalComputeShader> ComputeShader = StateCache.GetComputeShader();
			check(ComputeShader);

			MTL::Size ThreadgroupCounts = MTL::Size(ComputeShader->NumThreadsX, ComputeShader->NumThreadsY, ComputeShader->NumThreadsZ);
			check(ComputeShader->NumThreadsX > 0 && ComputeShader->NumThreadsY > 0 && ComputeShader->NumThreadsZ > 0);

			CurrentEncoder.GetComputeCommandEncoder()->dispatchThreadgroups(VertexBuffer->GetCurrentBuffer()->GetMTLBuffer(),
																	VertexBuffer->GetCurrentBuffer()->GetOffset() + ArgumentOffset, ThreadgroupCounts);
			
			EndComputeEncoder();
		}
	}
	else
	{
		NOT_SUPPORTED("RHIDispatchIndirectComputeShader");
	}
}

void FMetalRHICommandContext::RHISetViewport(float MinX, float MinY,float MinZ, float MaxX, float MaxY,float MaxZ)
{
    MTL_SCOPED_AUTORELEASE_POOL;

	MTL::Viewport Viewport;
	Viewport.originX = MinX;
	Viewport.originY = MinY;
	Viewport.width = MaxX - MinX;
	Viewport.height = MaxY - MinY;
	Viewport.znear = MinZ;
	Viewport.zfar = MaxZ;
	
	StateCache.SetViewport(Viewport);
}

void FMetalRHICommandContext::RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ)
{
	if (Device.SupportsFeature(EMetalFeaturesMultipleViewports))
	{
        MTL_SCOPED_AUTORELEASE_POOL;
        
		MTL::Viewport Viewport[2];
		
		Viewport[0].originX = LeftMinX;
		Viewport[0].originY = LeftMinY;
		Viewport[0].width = LeftMaxX - LeftMinX;
		Viewport[0].height = LeftMaxY - LeftMinY;
		Viewport[0].znear = MinZ;
		Viewport[0].zfar = MaxZ;
		
		Viewport[1].originX = RightMinX;
		Viewport[1].originY = RightMinY;
		Viewport[1].width = RightMaxX - RightMinX;
		Viewport[1].height = RightMaxY - RightMinY;
		Viewport[1].znear = MinZ;
		Viewport[1].zfar = MaxZ;
		
		StateCache.SetViewports(Viewport, 2);
	}
	else
	{
		NOT_SUPPORTED("RHISetStereoViewport");
	}
}

void FMetalRHICommandContext::RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data)
{ 
	NOT_SUPPORTED("RHISetMultipleViewports");
}

void FMetalRHICommandContext::RHISetScissorRect(bool bEnable,uint32 MinX,uint32 MinY,uint32 MaxX,uint32 MaxY)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	MTL::ScissorRect Scissor;
	Scissor.x = MinX;
	Scissor.y = MinY;
	Scissor.width = MaxX - MinX;
	Scissor.height = MaxY - MinY;

	// metal doesn't support 0 sized scissor rect
	if (bEnable == false || Scissor.width == 0 || Scissor.height == 0)
	{
		MTL::Viewport const& Viewport = StateCache.GetViewport(0);
		CGSize FBSize = StateCache.GetFrameBufferSize();
		
		Scissor.x = Viewport.originX;
		Scissor.y = Viewport.originY;
		Scissor.width = (Viewport.originX + Viewport.width <= FBSize.width) ? Viewport.width : FBSize.width - Viewport.originX;
		Scissor.height = (Viewport.originY + Viewport.height <= FBSize.height) ? Viewport.height : FBSize.height - Viewport.originY;
	}
	StateCache.SetScissorRect(bEnable, Scissor);
}

void FMetalRHICommandContext::RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    FMetalGraphicsPipelineState* PipelineState = ResourceCast(GraphicsState);
    if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelResetOnBind && StateCache.GetGraphicsPSO() != PipelineState)
    {
		CurrentEncoder.ResetLive();
    }
	StateCache.SetGraphicsPipelineState(PipelineState);

    RHISetStencilRef(StencilRef);
    RHISetBlendFactor(FLinearColor(1.0f, 1.0f, 1.0f));

    if (bApplyAdditionalState)
    {
#if PLATFORM_SUPPORTS_MESH_SHADERS
        ApplyStaticUniformBuffersOnContext(*this, StateCache, PipelineState->MeshShader);
        ApplyStaticUniformBuffersOnContext(*this, StateCache, PipelineState->AmplificationShader);
#endif
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
        ApplyStaticUniformBuffersOnContext(*this, StateCache, PipelineState->GeometryShader);
#endif
        ApplyStaticUniformBuffersOnContext(*this, StateCache, PipelineState->VertexShader);
        ApplyStaticUniformBuffersOnContext(*this, StateCache, PipelineState->PixelShader);
    }
}

void FMetalRHICommandContext::RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers)
{
	FMemory::Memzero(StaticUniformBuffers.GetData(), StaticUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));

	if (const FRHIShaderBindingLayout* Layout = InUniformBuffers.GetShaderBindingLayout())
	{
		check(InUniformBuffers.GetUniformBufferCount() == Layout->GetNumUniformBufferEntries());

		for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
		{
			StaticUniformBuffers[Index] = InUniformBuffers.GetUniformBuffer(Index);
			checkf(StaticUniformBuffers[Index], TEXT("Static uniform buffer at index %d is referenced in the shader binding layout but is not provided"), Index);
		}

		ShaderBindingLayout = Layout;
	}
	else
	{
		for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
		{
			FUniformBufferStaticSlot Slot = InUniformBuffers.GetSlot(Index);
			StaticUniformBuffers[Slot] = InUniformBuffers.GetUniformBuffer(Index);
		}

		ShaderBindingLayout = nullptr;
	}
}

void FMetalRHICommandContext::RHISetStaticUniformBuffer(FUniformBufferStaticSlot InSlot, FRHIUniformBuffer* InBuffer)
{
	StaticUniformBuffers[InSlot] = InBuffer;
}

struct FMetalShaderBinder
{
	FMetalStateCache& StateCache;
	const EMetalShaderStages Stage;
	FMetalShaderParameterCache& ShaderParameters;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	const bool bBindlessResources;
	const bool bBindlessSamplers;
#endif

	FMetalShaderBinder(FMetalStateCache& InStateCache, EShaderFrequency ShaderFrequency)
		: StateCache(InStateCache)
		, Stage(GetMetalShaderFrequency(ShaderFrequency))
		, ShaderParameters(StateCache.GetShaderParameters(Stage))
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		, bBindlessResources(IsMetalBindlessEnabled())
		, bBindlessSamplers(IsMetalBindlessEnabled())
#endif
	{
	}

	void SetParameter(uint32 BufferIndex, uint32 BaseIndex, uint32 NumBytes, const void* NewValue)
	{
		ShaderParameters.Set(BufferIndex, BaseIndex, NumBytes, NewValue);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetBindlessHandle(const FRHIDescriptorHandle& Handle, uint32 Offset)
	{
		if (Handle.IsValid())
		{
			const uint32 BindlessIndex = Handle.GetIndex();
			ShaderParameters.Set(0, Offset, 4, &BindlessIndex);
		}
	}
#endif

	void SetUAV(FRHIUnorderedAccessView* InUnorderedAccessView, uint32 Index, bool bClearResources = false)
	{
		FMetalUnorderedAccessView* UAV = ResourceCast(InUnorderedAccessView);
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessResources)
		{
			StateCache.IRMakeUAVResident(Stage, UAV);
		}
		else
#endif
		{
			StateCache.SetShaderUnorderedAccessView(Stage, Index, UAV);
		}
	}

	void SetSRV(FRHIShaderResourceView* InShaderResourceView, uint32 Index)
	{
		FMetalShaderResourceView* SRV = ResourceCast(InShaderResourceView);
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessResources)
		{
			StateCache.IRMakeSRVResident(Stage, SRV);
		}
		else
#endif
		{
			StateCache.SetShaderResourceView(Stage, Index, SRV);
		}
	}

	void SetTexture(FRHITexture* InTexture, uint32 Index)
	{
		FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(InTexture);
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessResources)
		{
			StateCache.IRMakeTextureResident(Stage, Surface->Texture.get());
		}
		else
#endif
		{
			if (Surface)
			{
				if (Surface->Texture || !EnumHasAnyFlags(Surface->GetDesc().Flags, ETextureCreateFlags::Presentable))
				{
					StateCache.SetShaderTexture(Stage, Surface->Texture.get(), Index, (MTL::ResourceUsage)(MTL::ResourceUsageRead | MTL::ResourceUsageSample));
				}
				else
				{
					MTLTexturePtr Tex = Surface->GetCurrentTexture();
					StateCache.SetShaderTexture(Stage, Tex.get(), Index, (MTL::ResourceUsage)(MTL::ResourceUsageRead | MTL::ResourceUsageSample));
				}
			}
			else
			{
				StateCache.SetShaderTexture(Stage, nullptr, Index, MTL::ResourceUsage(0));
			}
		}
	}

	void SetSampler(FRHISamplerState* InSampler, uint32 Index)
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (bBindlessSamplers)
		{
		}
		else
#endif
		{
			FMetalSamplerState* Sampler = ResourceCast(InSampler);
			StateCache.SetShaderSamplerState(Stage, Sampler, Index);
		}
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
	{
		FMetalResourceCollection* MetalResourceCollection = ResourceCast(ResourceCollection);
		SetSRV(MetalResourceCollection->GetShaderResourceView(), Index);
	}
#endif
};

FRHIDescriptorHandle GetDescriptorHandleFromTexture(FRHITexture* InTexture)
{
	if (FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(InTexture))
	{
		FRHIDescriptorHandle Handle = Surface->GetDefaultBindlessHandle();
		if (Handle.IsValid())
		{
			return Handle;
		}

		if (Surface->Viewport)
		{
			// We need to update the drawable texture on the surface and create a bindless handle, we can ignore the return value.
			Surface->GetDrawableTexture();
			return Surface->GetDefaultBindlessHandle();
		}
	}

	return FRHIDescriptorHandle();
}

static void SetShaderParameters(
	FMetalStateCache& StateCache
	, FRHIShader* Shader
	, EShaderFrequency ShaderFrequency
	, TArrayView<const uint8> InParametersData
	, TArrayView<const FRHIShaderParameter> InParameters
	, TArrayView<const FRHIShaderParameterResource> InResourceParameters
	, TArrayView<const FRHIShaderParameterResource> InBindlessParameters)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	FMetalShaderBinder Binder(StateCache, ShaderFrequency);

	for (const FRHIShaderParameter& Parameter : InParameters)
	{
		Binder.SetParameter(Parameter.BufferIndex, Parameter.BaseIndex, Parameter.ByteSize, &InParametersData[Parameter.ByteOffset]);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	for (const FRHIShaderParameterResource& Parameter : InBindlessParameters)
	{
		if (FRHIResource* Resource = Parameter.Resource)
		{
			if (Parameter.Type == FRHIShaderParameterResource::EType::UniformBuffer)
			{
				UE::RHI::Private::SetAllUniformBufferResourcesForBindless(Binder, static_cast<FRHIUniformBuffer*>(Resource));
			}
			else
			{
				FRHIDescriptorHandle Handle;

				switch (Parameter.Type)
				{
				case FRHIShaderParameterResource::EType::Texture:
					Handle = GetDescriptorHandleFromTexture(static_cast<FRHITexture*>(Resource));
					Binder.SetTexture(static_cast<FRHITexture*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceView:
					Handle = static_cast<FRHIShaderResourceView*>(Resource)->GetBindlessHandle();
					Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::UnorderedAccessView:
					Handle = static_cast<FRHIUnorderedAccessView*>(Resource)->GetBindlessHandle();
					Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Resource), Parameter.Index, true);
					break;
				case FRHIShaderParameterResource::EType::Sampler:
					Handle = static_cast<FRHISamplerState*>(Resource)->GetBindlessHandle();
					Binder.SetSampler(static_cast<FRHISamplerState*>(Resource), Parameter.Index);
					break;
				case FRHIShaderParameterResource::EType::ResourceCollection:
					Handle = static_cast<FRHIResourceCollection*>(Resource)->GetBindlessHandle();
					Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Resource), Parameter.Index);
					break;
				}

				checkf(Handle.IsValid(), TEXT("Metal resource did not provide a valid descriptor handle. Please validate that all D3D12 types can provide this or that the resource is still valid."));
				Binder.SetBindlessHandle(Handle, Parameter.Index);
			}
		}
	}
#endif 

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		if (Parameter.Type == FRHIShaderParameterResource::EType::UnorderedAccessView)
		{
			if (ShaderFrequency == SF_Pixel || ShaderFrequency == SF_Compute)
			{
				Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), Parameter.Index, true);
			}
			else
			{
				checkf(false, TEXT("TShaderRHI Can't have compute shader to be set. UAVs are not supported on vertex, tessellation and geometry shaders."));
			}
		}
	}

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			Binder.SetTexture(static_cast<FRHITexture*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			Binder.SetSampler(static_cast<FRHISamplerState*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			BindUniformBuffer(Binder.StateCache, Shader, Binder.Stage, Parameter.Index, static_cast<FRHIUniformBuffer*>(Parameter.Resource));
			break;
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		case FRHIShaderParameterResource::EType::ResourceCollection:
			Binder.SetResourceCollection(static_cast<FRHIResourceCollection*>(Parameter.Resource), Parameter.Index);
			break;
#endif
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}

void FMetalRHICommandContext::RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
    const EShaderFrequency ShaderFrequency = Shader->GetFrequency();
    if (IsValidGraphicsFrequency(ShaderFrequency))
    {
        SetShaderParameters(
			StateCache
			, Shader
            , ShaderFrequency
            , InParametersData
            , InParameters
            , InResourceParameters
            , InBindlessParameters
        );
    }
    else
    {
        checkf(0, TEXT("Unsupported FRHIGraphicsShader Type '%s'!"), GetShaderFrequencyString(ShaderFrequency, false));
    }
}

void FMetalRHICommandContext::RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
    SetShaderParameters(
		StateCache
		, Shader
        , SF_Compute
        , InParametersData
        , InParameters
        , InResourceParameters
        , InBindlessParameters
    );
}

void FMetalRHICommandContext::RHISetStencilRef(uint32 StencilRef)
{
	StateCache.SetStencilRef(StencilRef);
}

void FMetalRHICommandContext::RHISetBlendFactor(const FLinearColor& BlendFactor)
{
	StateCache.SetBlendFactor(BlendFactor);
}

void FMetalRHICommandContext::SetRenderTargets(uint32 NumSimultaneousRenderTargets, const FRHIRenderTargetView* NewRenderTargets,
	const FRHIDepthRenderTargetView* NewDepthStencilTargetRHI)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FRHIDepthRenderTargetView DepthView;
	if (NewDepthStencilTargetRHI)
	{
		DepthView = *NewDepthStencilTargetRHI;
	}
	else
	{
		DepthView = FRHIDepthRenderTargetView(nullptr, ERenderTargetLoadAction::EClear, ERenderTargetStoreAction::ENoAction);
	}

	FRHISetRenderTargetsInfo Info(NumSimultaneousRenderTargets, NewRenderTargets, DepthView);
	SetRenderTargetsAndClear(Info);
}

void FMetalRHICommandContext::SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo)
{
    MTL_SCOPED_AUTORELEASE_POOL;
		
	FRHIRenderPassInfo PassInfo;
	bool bHasTarget = (RenderTargetsInfo.DepthStencilRenderTarget.Texture != nullptr);
	
	for (uint32 i = 0; i < RenderTargetsInfo.NumColorRenderTargets; i++)
	{
		if (RenderTargetsInfo.ColorRenderTarget[i].Texture)
		{
			PassInfo.ColorRenderTargets[i].RenderTarget = RenderTargetsInfo.ColorRenderTarget[i].Texture;
			PassInfo.ColorRenderTargets[i].ArraySlice = RenderTargetsInfo.ColorRenderTarget[i].ArraySliceIndex;
			PassInfo.ColorRenderTargets[i].MipIndex = RenderTargetsInfo.ColorRenderTarget[i].MipIndex;
			PassInfo.ColorRenderTargets[i].Action = MakeRenderTargetActions(RenderTargetsInfo.ColorRenderTarget[i].LoadAction, RenderTargetsInfo.ColorRenderTarget[i].StoreAction);
		bHasTarget = (RenderTargetsInfo.ColorRenderTarget[i].Texture != nullptr);
		}
	}
		
	if (RenderTargetsInfo.DepthStencilRenderTarget.Texture)
	{
		PassInfo.DepthStencilRenderTarget.DepthStencilTarget = RenderTargetsInfo.DepthStencilRenderTarget.Texture;
		PassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil = RenderTargetsInfo.DepthStencilRenderTarget.GetDepthStencilAccess();
		PassInfo.DepthStencilRenderTarget.Action = MakeDepthStencilTargetActions(MakeRenderTargetActions(RenderTargetsInfo.DepthStencilRenderTarget.DepthLoadAction, RenderTargetsInfo.DepthStencilRenderTarget.DepthStoreAction), MakeRenderTargetActions(RenderTargetsInfo.DepthStencilRenderTarget.StencilLoadAction, RenderTargetsInfo.DepthStencilRenderTarget.GetStencilStoreAction()));
	}
		
	PassInfo.NumOcclusionQueries = UINT16_MAX;
	PassInfo.bOcclusionQueries = true;

	if (bHasTarget)
	{
		StateCache.SetRenderPassInfo(PassInfo, QueryBuffer->GetCurrentQueryBuffer());

		// Set the viewport to the full size of render target 0.
		if (RenderTargetsInfo.ColorRenderTarget[0].Texture)
		{
			const FRHIRenderTargetView& RenderTargetView = RenderTargetsInfo.ColorRenderTarget[0];
			FMetalSurface* RenderTarget = GetMetalSurfaceFromRHITexture(RenderTargetView.Texture);

			uint32 Width = FMath::Max((uint32)(RenderTarget->Texture->width() >> RenderTargetView.MipIndex), (uint32)1);
			uint32 Height = FMath::Max((uint32)(RenderTarget->Texture->height() >> RenderTargetView.MipIndex), (uint32)1);

			RHISetViewport(0, 0, 0.0f, Width, Height, 1.0f);
		}
	}
}

void FMetalRHICommandContext::CommitRenderResourceTables(void)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalCommitRenderResourceTablesTime);
	
	StateCache.CommitRenderResources(&CurrentEncoder);
	
	if(!IsMetalBindlessEnabled())
	{
		StateCache.CommitResourceTable(EMetalShaderStages::Vertex, MTL::FunctionTypeVertex, CurrentEncoder);
		
		FMetalGraphicsPipelineState const* BoundShaderState = StateCache.GetGraphicsPSO();
		
		if (BoundShaderState->VertexShader->SideTableBinding >= 0)
		{
			CurrentEncoder.SetShaderSideTable(MTL::FunctionTypeVertex, BoundShaderState->VertexShader->SideTableBinding);
			StateCache.SetShaderBuffer(EMetalShaderStages::Vertex, nullptr, nullptr, 0, 0, BoundShaderState->VertexShader->SideTableBinding, MTL::ResourceUsage(0));
		}
		
		if (IsValidRef(BoundShaderState->PixelShader))
		{
			StateCache.CommitResourceTable(EMetalShaderStages::Pixel, MTL::FunctionTypeFragment, CurrentEncoder);
			if (BoundShaderState->PixelShader->SideTableBinding >= 0)
			{
				CurrentEncoder.SetShaderSideTable(MTL::FunctionTypeFragment, BoundShaderState->PixelShader->SideTableBinding);
				StateCache.SetShaderBuffer(EMetalShaderStages::Pixel, nullptr, nullptr, 0, 0, BoundShaderState->PixelShader->SideTableBinding, MTL::ResourceUsage(0));
			}
		}
	}
}

bool FMetalRHICommandContext::PrepareToDraw(uint32 PrimitiveType)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalPrepareDrawTime);
	TRefCountPtr<FMetalGraphicsPipelineState> CurrentPSO = StateCache.GetGraphicsPSO();
	check(IsValidRef(CurrentPSO));
	
	FMetalHashedVertexDescriptor const& VertexDesc = CurrentPSO->VertexDeclaration->Layout;
	
	// Validate the vertex layout in debug mode, or when the validation layer is enabled for development builds.
	// Other builds will just crash & burn if it is incorrect.
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	if(Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)
	{
		MTLVertexDescriptorPtr Layout = VertexDesc.VertexDesc;
		
		if(Layout && Layout->layouts())
		{
			for (uint32 i = 0; i < MaxVertexElementCount; i++)
			{
				auto Attribute = Layout->attributes()->object(i);
				if(Attribute && Attribute->format() > MTL::VertexFormatInvalid)
				{
					auto BufferLayout = Layout->layouts()->object(Attribute->bufferIndex());
					uint32 BufferLayoutStride = BufferLayout ? BufferLayout->stride() : 0;
					
					uint32 BufferIndex = METAL_TO_UNREAL_BUFFER_INDEX(Attribute->bufferIndex());
					
					if (CurrentPSO->VertexShader->Bindings.InOutMask.IsFieldEnabled(BufferIndex))
					{
						uint64 MetalSize = StateCache.GetVertexBufferSize(BufferIndex);
						
						// If the vertex attribute is required and either no Metal buffer is bound or the size of the buffer is smaller than the stride, or the stride is explicitly specified incorrectly then the layouts don't match.
						if (BufferLayoutStride > 0 && MetalSize < BufferLayoutStride)
						{
							FString Report = FString::Printf(TEXT("Vertex Layout Mismatch: Index: %lld, Len: %lld, Decl. Stride: %d"), Attribute->bufferIndex(), MetalSize, BufferLayoutStride);
							UE_LOGF(LogMetal, Warning, "%ls", *Report);
						}
					}
				}
			}
		}
	}
#endif
	
	return true;
}

void FMetalRHICommandContext::PrepareToRender(uint32 PrimitiveType)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalPrepareToRenderTime);
	
	check(CurrentEncoder.IsRenderCommandEncoderActive());
	
	// Set raster state
	StateCache.SetRenderState(CurrentEncoder);
	
	// Bind shader resources
	CommitRenderResourceTables();
	
	StateCache.SetRenderPipelineState(CurrentEncoder);
}

void FMetalRHICommandContext::PrepareToDispatch()
{
	SCOPE_CYCLE_COUNTER(STAT_MetalPrepareToDispatchTime);
	
	check(CurrentEncoder.GetCommandBuffer());
	check(CurrentEncoder.IsComputeCommandEncoderActive());
	
	// Bind shader resources
	StateCache.CommitComputeResources(&CurrentEncoder);

	if(!IsMetalBindlessEnabled())
	{
		StateCache.CommitResourceTable(EMetalShaderStages::Compute, MTL::FunctionTypeKernel, CurrentEncoder);
		
		FMetalComputeShader const* ComputeShader = StateCache.GetComputeShader();
		if (ComputeShader->SideTableBinding >= 0)
		{
			CurrentEncoder.SetShaderSideTable(MTL::FunctionTypeKernel, ComputeShader->SideTableBinding);
			StateCache.SetShaderBuffer(EMetalShaderStages::Compute, nullptr, nullptr, 0, 0, ComputeShader->SideTableBinding, MTL::ResourceUsage(0));
		}
	}
	
	StateCache.SetComputePipelineState(CurrentEncoder);
}

void FMetalRHICommandContext::RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	SCOPE_CYCLE_COUNTER(STAT_MetalDrawCallTime);
   
	EPrimitiveType PrimitiveType = StateCache.GetPrimitiveType();

	// how many verts to render
	uint32 NumVertices = GetVertexCountForPrimitiveCount(NumPrimitives, PrimitiveType);
	
	RHI_DRAW_CALL_STATS(PrimitiveType, NumVertices, NumPrimitives, NumInstances);

	NumInstances = FMath::Max(NumInstances, 1u);
	
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	if (IsValidRef(StateCache.GetGraphicsPSO()->GeometryShader))
	{
		FMetalVertexShader* VertexShader = (FMetalVertexShader*)StateCache.GetGraphicsPSO()->VertexShader.GetReference();
		FMetalGeometryShader* GeometryShader = (FMetalGeometryShader*)StateCache.GetGraphicsPSO()->GeometryShader.GetReference();
		check(VertexShader);
		check(GeometryShader);
		
		PrepareToRender(PrimitiveType);
		
		IRRuntimeDrawInfo DrawInfos = IRRuntimeCalculateDrawInfoForGSEmulation((IRRuntimePrimitiveType)TranslatePrimitiveType(PrimitiveType),
																			  VertexShader->Bindings.OutputSizeVS,
																			  GeometryShader->Bindings.MaxInputPrimitivesPerMeshThreadgroupGS,
																			  NumInstances);
		
		MTL::Size objectThreadgroupCountTemp = IRRuntimeCalculateObjectTgCountForTessellationAndGeometryEmulation(NumVertices,
															DrawInfos.objectThreadgroupVertexStride,
															(IRRuntimePrimitiveType)TranslatePrimitiveType(PrimitiveType),
															NumInstances);
		MTL::Size objectThreadgroupCount = MTL::Size::Make(objectThreadgroupCountTemp.width,
														   objectThreadgroupCountTemp.height,
														   objectThreadgroupCountTemp.depth);
		
		uint32 ObjectThreadgroupSize = 0;
		uint32 MeshThreadgroupSize = 0;
		
		IRRuntimeCalculateThreadgroupSizeForGeometry((IRRuntimePrimitiveType)TranslatePrimitiveType(PrimitiveType),
													 GeometryShader->Bindings.MaxInputPrimitivesPerMeshThreadgroupGS,
													 DrawInfos.objectThreadgroupVertexStride,
													 &ObjectThreadgroupSize,
													 &MeshThreadgroupSize);
		
		IRRuntimeDrawParams DrawParams;
		IRRuntimeDrawArgument& DrawArgs = DrawParams.draw;
		DrawArgs = { 0 };
		DrawArgs.instanceCount = NumInstances;
		DrawArgs.startInstanceLocation = 0;
		DrawArgs.vertexCountPerInstance = NumVertices;
		DrawArgs.startVertexLocation = BaseVertexIndex;
		
		CurrentEncoder.GetRenderCommandEncoder()->setMeshBytes(&DrawParams, sizeof(IRRuntimeDrawParams), kIRArgumentBufferDrawArgumentsBindPoint);
		CurrentEncoder.GetRenderCommandEncoder()->setMeshBytes(&DrawInfos, sizeof(IRRuntimeDrawInfo), kIRArgumentBufferUniformsBindPoint);
		
		CurrentEncoder.GetRenderCommandEncoder()->setObjectBytes(&DrawParams, sizeof(IRRuntimeDrawParams), kIRArgumentBufferDrawArgumentsBindPoint);
		CurrentEncoder.GetRenderCommandEncoder()->setObjectBytes(&DrawInfos, sizeof(IRRuntimeDrawInfo), kIRArgumentBufferUniformsBindPoint);
		
		StateCache.IRMapVertexBuffers(CurrentEncoder.GetRenderCommandEncoder(), true);
		
		CurrentEncoder.GetRenderCommandEncoder()->drawMeshThreadgroups(objectThreadgroupCount,
																	MTL::Size::Make(ObjectThreadgroupSize, 1, 1),
																	MTL::Size::Make(MeshThreadgroupSize, 1, 1));
	}
#endif

	PrepareToRender(PrimitiveType);

#if METAL_USE_METAL_SHADER_CONVERTER
	if(IsMetalBindlessEnabled())
	{
		StateCache.IRMapVertexBuffers(CurrentEncoder.GetRenderCommandEncoder());
		IRRuntimeDrawPrimitives(CurrentEncoder.GetRenderCommandEncoder(), TranslatePrimitiveType(PrimitiveType), BaseVertexIndex, NumVertices, NumInstances, 0);
	}
	else
#endif
	{
		CurrentEncoder.GetRenderCommandEncoder()->drawPrimitives(TranslatePrimitiveType(PrimitiveType), BaseVertexIndex, NumVertices, NumInstances);
	}
}

void FMetalRHICommandContext::RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    if (Device.SupportsFeature(EMetalFeaturesIndirectBuffer))
    {
		check(CurrentEncoder.IsRenderCommandEncoderActive());
		
        SCOPE_CYCLE_COUNTER(STAT_MetalDrawCallTime);
        EPrimitiveType PrimitiveType = StateCache.GetPrimitiveType();
        
        RHI_DRAW_CALL_INC();
        FMetalRHIBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);
		
		FMetalBufferPtr TheBackingBuffer = ArgumentBuffer->GetCurrentBuffer();
		check(TheBackingBuffer);
		
		PrepareToRender(PrimitiveType);
		
#if METAL_USE_METAL_SHADER_CONVERTER
		if(IsMetalBindlessEnabled())
		{
			uint32 NullBuffer = 0x0;
			CurrentEncoder.GetRenderCommandEncoder()->setVertexBytes(&NullBuffer, sizeof(uint32), kIRArgumentBufferUniformsBindPoint);
			CurrentEncoder.GetRenderCommandEncoder()->useResource(TheBackingBuffer->GetMTLBuffer(), MTL::ResourceUsageRead);
			
			StateCache.IRMapVertexBuffers(CurrentEncoder.GetRenderCommandEncoder());
			IRRuntimeDrawPrimitives(CurrentEncoder.GetRenderCommandEncoder(), TranslatePrimitiveType(PrimitiveType), TheBackingBuffer->GetMTLBuffer(), TheBackingBuffer->GetOffset() + ArgumentOffset);		
		}
		else
#endif
		{
			CurrentEncoder.GetRenderCommandEncoder()->drawPrimitives(TranslatePrimitiveType(PrimitiveType),
																	 TheBackingBuffer->GetMTLBuffer(), TheBackingBuffer->GetOffset() + ArgumentOffset);
		}
	}
    else
    {
        NOT_SUPPORTED("RHIDrawPrimitiveIndirect");
    }
}

#if PLATFORM_SUPPORTS_MESH_SHADERS
void FMetalRHICommandContext::RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{
    MTL_SCOPED_AUTORELEASE_POOL;

	RHI_DRAW_CALL_INC();

#if METAL_USE_METAL_SHADER_CONVERTER
	checkNoEntry();
#else
	NOT_SUPPORTED("RHIDispatchMeshShader");
#endif
}

void FMetalRHICommandContext::RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
    MTL_SCOPED_AUTORELEASE_POOL;

	RHI_DRAW_CALL_INC();

#if METAL_USE_METAL_SHADER_CONVERTER
	uint32 PrimitiveType = StateCache.GetPrimitiveType();
	PrepareToRender(PrimitiveType);
	
	FMetalRHIBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);
	
	FMetalBufferPtr TheBackingBuffer = ArgumentBuffer->GetCurrentBuffer();
	check(TheBackingBuffer);
	
	PrepareToRender(PrimitiveType);

	if(IsMetalBindlessEnabled())
	{
		IRBindIndirectMeshDrawArguments(CurrentEncoder.GetRenderCommandEncoder(), TranslatePrimitiveType(PrimitiveType), TheBackingBuffer, ArgumentOffset, StateCache);
	}
	
	// TODO: Cache this at RHI init time?
	const uint32 MSThreadGroupSize = FDataDrivenShaderPlatformInfo::GetMaxMeshShaderThreadGroupSize(GMaxRHIShaderPlatform);
	CurrentEncoder.GetRenderCommandEncoder()->drawMeshThreadgroups(TheBackingBuffer->GetMTLBuffer(),
																ArgumentOffset,
																MTL::Size::Make(MSThreadGroupSize, 1, 1),
																MTL::Size::Make(MSThreadGroupSize, 1, 1));
#else
	NOT_SUPPORTED("RHIDispatchIndirectMeshShader");
#endif
}
#endif

void FMetalRHICommandContext::RHIDrawIndexedPrimitive(FRHIBuffer* IndexBufferRHI, int32 BaseVertexIndex, uint32 FirstInstance,
	uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	SCOPE_CYCLE_COUNTER(STAT_MetalDrawCallTime);
	checkf(GRHISupportsBaseVertexIndex || BaseVertexIndex == 0, TEXT("BaseVertexIndex must be 0, see GRHISupportsBaseVertexIndex"));
	checkf(GRHISupportsFirstInstance || FirstInstance == 0, TEXT("FirstInstance must be 0, see GRHISupportsFirstInstance"));
	check(CurrentEncoder.IsRenderCommandEncoderActive());
	
	EPrimitiveType PrimitiveType = StateCache.GetPrimitiveType();
	
	RHI_DRAW_CALL_STATS(PrimitiveType, NumVertices, NumPrimitives, NumInstances);

	FMetalRHIBuffer* IndexBuffer = ResourceCast(IndexBufferRHI);
	
	if(!PrepareToDraw(PrimitiveType))
	{
		return;
	}
	
	// We need at least one to cover all use cases
	NumInstances = FMath::Max(NumInstances,1u);
	
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	{
		FMetalGraphicsPipelineState* PipelineState = StateCache.GetGraphicsPSO();
		check(PipelineState != nullptr);
		FMetalVertexDeclaration* VertexDecl = PipelineState->VertexDeclaration;
		check(VertexDecl != nullptr);
		
		// Set our local copy and try to disprove the passed in value
		uint32 ClampedNumInstances = NumInstances;
		const CrossCompiler::FShaderBindingInOutMask& InOutMask = PipelineState->VertexShader->Bindings.InOutMask;

		// I think it is valid to have no elements in this list
		for(int VertexElemIdx = 0;VertexElemIdx < VertexDecl->Elements.Num();++VertexElemIdx)
		{
			FVertexElement const & VertexElem = VertexDecl->Elements[VertexElemIdx];
			if(VertexElem.Stride > 0 && VertexElem.bUseInstanceIndex && InOutMask.IsFieldEnabled(VertexElem.AttributeIndex))
			{
				uint32 AvailElementCount = 0;
				
				uint32 BufferSize = StateCache.GetVertexBufferSize(VertexElem.StreamIndex);
				uint32 ElementCount = (BufferSize / VertexElem.Stride);
				
				if(ElementCount > FirstInstance)
				{
					AvailElementCount = ElementCount - FirstInstance;
				}
				
				ClampedNumInstances = FMath::Clamp<uint32>(ClampedNumInstances, 0, AvailElementCount);
				
				if(ClampedNumInstances < NumInstances)
				{
					// Setting NumInstances to ClampedNumInstances would fix any visual rendering bugs resulting from this bad call but these draw calls are wrong - don't hide the issue
					UE_LOGF(LogMetal, Error, "Metal DrawIndexedPrimitive requested to draw %d Instances but vertex stream only has %d instance data available. ShaderName: %ls, Deficient Attribute Index: %u", NumInstances, ClampedNumInstances,
						   PipelineState->PixelShader->GetShaderName(), VertexElem.AttributeIndex);
				}
			}
		}
	}
#endif
	
	PrepareToRender(PrimitiveType);
	
	NS::UInteger NumIndices = GetVertexCountForPrimitiveCount(NumPrimitives, PrimitiveType);
	uint32 IndexStride = IndexBuffer->GetStride();
	
#if METAL_USE_METAL_SHADER_CONVERTER
	if(IsMetalBindlessEnabled())
	{
		FMetalBufferPtr IndexBufferPtr = IndexBuffer->GetCurrentBuffer();
		uint32 BaseIndexLocation = IndexBufferPtr->GetOffset() + (StartIndex * IndexStride);
		MTL::IndexType IndexType = ((IndexStride == 2) ? MTL::IndexTypeUInt16 : MTL::IndexTypeUInt32);
		
		StateCache.IRMapVertexBuffers(CurrentEncoder.GetRenderCommandEncoder());
	
		CurrentEncoder.GetRenderCommandEncoder()->useResource(IndexBufferPtr->GetMTLBuffer(), MTL::ResourceUsageRead);
		
		IRRuntimeDrawIndexedPrimitives(CurrentEncoder.GetRenderCommandEncoder(), TranslatePrimitiveType(PrimitiveType), NumIndices, IndexType, IndexBufferPtr->GetMTLBuffer(), BaseIndexLocation, NumInstances, BaseVertexIndex, FirstInstance);
	}
	else
#endif
	{
		FMetalBufferPtr IndexBufferPtr = IndexBuffer->GetCurrentBuffer();

		if (GRHISupportsBaseVertexIndex && GRHISupportsFirstInstance)
		{
			CurrentEncoder.GetRenderCommandEncoder()->drawIndexedPrimitives(TranslatePrimitiveType(PrimitiveType), NumIndices,
																			((IndexStride == 2) ? MTL::IndexTypeUInt16 : MTL::IndexTypeUInt32),
																			IndexBufferPtr->GetMTLBuffer(), IndexBufferPtr->GetOffset() + (StartIndex * IndexStride),
																			NumInstances, BaseVertexIndex, FirstInstance);
		}
		else
		{
			CurrentEncoder.GetRenderCommandEncoder()->drawIndexedPrimitives(TranslatePrimitiveType(PrimitiveType),
																			NumIndices,
																			((IndexStride == 2) ? MTL::IndexTypeUInt16 : MTL::IndexTypeUInt32),
																			IndexBufferPtr->GetMTLBuffer(), IndexBufferPtr->GetOffset() + (StartIndex * IndexStride),
																			NumInstances);
		}
	}
}

void FMetalRHICommandContext::RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBufferRHI, FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	if (Device.SupportsFeature(EMetalFeaturesIndirectBuffer))
	{
		SCOPE_CYCLE_COUNTER(STAT_MetalDrawCallTime);
		check(CurrentEncoder.IsRenderCommandEncoderActive());
		
		EPrimitiveType PrimitiveType = StateCache.GetPrimitiveType();
		
		if(!PrepareToDraw(PrimitiveType))
		{
			return;
		}
		
		RHI_DRAW_CALL_INC();
		FMetalRHIBuffer* IndexBuffer = ResourceCast(IndexBufferRHI);
		FMetalRHIBuffer* ArgumentsBuffer = ResourceCast(ArgumentBufferRHI);
		
		FMetalBufferPtr TheBackingIndexBuffer = IndexBuffer->GetCurrentBuffer();
		FMetalBufferPtr TheBackingBuffer = ArgumentsBuffer->GetCurrentBuffer();
		
		check(TheBackingIndexBuffer);
		check(TheBackingBuffer);
		
		PrepareToRender(PrimitiveType);
		
#if METAL_USE_METAL_SHADER_CONVERTER
		if(IsMetalBindlessEnabled())
		{
			// TODO: Carl - Remove this when API validation is fixed
			// Binding to uniforms bind point to work around error in API validation
			uint32 NullBuffer = 0x0;
			CurrentEncoder.GetRenderCommandEncoder()->setVertexBytes(&NullBuffer, sizeof(uint32), kIRArgumentBufferUniformsBindPoint);
			
			StateCache.IRMapVertexBuffers(CurrentEncoder.GetRenderCommandEncoder());
			
			CurrentEncoder.GetRenderCommandEncoder()->useResource(TheBackingBuffer->GetMTLBuffer(), MTL::ResourceUsageRead);
			CurrentEncoder.GetRenderCommandEncoder()->useResource(TheBackingIndexBuffer->GetMTLBuffer(), MTL::ResourceUsageRead);
			
			IRRuntimeDrawIndexedPrimitives(CurrentEncoder.GetRenderCommandEncoder(), TranslatePrimitiveType(PrimitiveType), IndexBuffer->GetIndexType(), TheBackingIndexBuffer->GetMTLBuffer(), TheBackingIndexBuffer->GetOffset(), TheBackingBuffer->GetMTLBuffer(), TheBackingBuffer->GetOffset() + ArgumentOffset);
		}
		else
#endif
		{
			CurrentEncoder.GetRenderCommandEncoder()->drawIndexedPrimitives(
				TranslatePrimitiveType(PrimitiveType),
				IndexBuffer->GetIndexType(),
				TheBackingIndexBuffer->GetMTLBuffer(),
				TheBackingIndexBuffer->GetOffset(),
				TheBackingBuffer->GetMTLBuffer(), 
				TheBackingBuffer->GetOffset() + ArgumentOffset
			);
		}
	}
	else
	{
		NOT_SUPPORTED("RHIDrawIndexedPrimitiveIndirect");
	}
}

void FMetalRHICommandContext::RHIClearMRT(bool bClearColor,int32 NumClearColors,const FLinearColor* ClearColorArray,bool bClearDepth,float Depth,bool bClearStencil,uint32 Stencil)
{
	NOT_SUPPORTED("RHIClearMRT");
}

void FMetalRHICommandContext::RHISetDepthBounds(float MinDepth, float MaxDepth)
{
	METAL_IGNORED(FMetalRHICommandContextSetDepthBounds);
}

#if PLATFORM_USES_FIXED_RHI_CLASS
#define INTERNAL_DECORATOR(Method) ((FMetalRHICommandContext&)CmdList.GetContext()).FMetalRHICommandContext::Method
#include "RHICommandListCommandExecutes.inl"
#endif
