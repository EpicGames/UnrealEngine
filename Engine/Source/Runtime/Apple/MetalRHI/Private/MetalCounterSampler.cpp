// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalCounterSampler.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalDynamicRHI.h"

static TAutoConsoleVariable<bool> CVarMetalRHIInsertCounterSampleBarrier(
	TEXT("rhi.Metal.InsertCounterSampleBarrier"),
	true,
	TEXT("Whether to insert a counter sampler barrier to provide the most accurate timings. (default: true)\n"),
	ECVF_ReadOnly);

void FMetalCounterSample::ResolveStageCounters(uint64_t& InStartTime, uint64_t& InEndTime)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	if(!bResolved)
	{
		NS::Data* Data;
		
		if(SampleType == EMetalCounterSampleType::RenderStage)
		{
			NS::Range Range = NS::Range::Make(Offset, 2);
			Data = SampleBuffer->resolveCounterRange(Range);
		}
		else
		{
			NS::Range Range = NS::Range::Make(Offset, 2);
			Data = SampleBuffer->resolveCounterRange(Range);
		}
		
		// Convert the contents of the counter sample buffer into the standard data format.
		check(Data);
		if (Data == nullptr) 
		{
			return;
		}
		
		MTL::CounterResultTimestamp* Timestamps = (MTL::CounterResultTimestamp *)(Data->mutableBytes());
		
		if(SampleType == EMetalCounterSampleType::RenderStage)
		{
			uint64_t StartTimeVertex = Timestamps[0].timestamp;
			uint64_t EndTimeFragment = Timestamps[1].timestamp;
			
			StartTime = StartTimeVertex;
			EndTime = EndTimeFragment;
		}
		else
		{
			StartTime = Timestamps[0].timestamp;
			EndTime = Timestamps[1].timestamp;
		}
		
		if(StartTime == 0 || EndTime == 0)
		{
			StartTime = EndTime = FMath::Max(StartTime, EndTime);
		}
		else if (StartTime > EndTime)
		{
			// Looks like a driver bug that randomly causes StartVertex to be a huge value even though EndFragment is valid
			StartTime = EndTime;
		}
		
		double OneOverNanosToCycles = 1.0 / 1000000000.0 / FPlatformTime::GetSecondsPerCycle64();
		StartTime = FMath::TruncToInt64(StartTime * OneOverNanosToCycles);
		EndTime = FMath::TruncToInt64(EndTime * OneOverNanosToCycles);
		
		bResolved = true;
	}
	
	InStartTime = StartTime;
	InEndTime = EndTime;
}

void FMetalCounterSample::ResolveBoundaryCounter(uint64_t& Time)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	if(!bResolved)
	{
		NS::Range Range = NS::Range::Make(Offset, 1);
		
		// Convert the contents of the counter sample buffer into the standard data format.
		NS::Data* Data = SampleBuffer->resolveCounterRange(Range);
		check(Data);
		if (Data == nullptr) 
		{
			return;
		}
		
		MTL::CounterResultTimestamp* Timestamps = (MTL::CounterResultTimestamp *)(Data->mutableBytes());
		EndTime = Timestamps->timestamp;
		
		bResolved = true;
	}
	Time = EndTime;
}

TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> FMetalCounterSample::MemoryPool;

FMetalCounterSampler::FMetalCounterSampler(FMetalDevice* InDevice, uint32_t SampleCount) : Device(InDevice)
{
	Size = SampleCount;
}

FMetalCounterSampler::~FMetalCounterSampler()
{
	SampleBuffer->release();
}

MTL::CounterSampleBuffer* FMetalCounterSampler::SwapOrAllocateBuffer(uint32_t SampleSize, uint32_t& OutOffset)
{
	FScopeLock Lock(&Mutex);
	OutOffset = Offset;
	
	Offset += SampleSize;
	if(Offset > Size || !SampleBuffer)
	{
		// Re-add Counter Sampler on next frame
		if(SampleBuffer)
		{
			FMetalDynamicRHI::Get().DeferredDelete([this, InSampleBuffer=SampleBuffer](){
				SampleBufferFreePool.Add(InSampleBuffer);
			});
		}
		
		if(SampleBufferFreePool.Num() > 0)
		{
			SampleBuffer = SampleBufferFreePool.Pop();
			Offset = 0;
			OutOffset = 0;
			return SampleBuffer;
		}
		
		MTL::CounterSampleBufferDescriptor* BufferDesc;
		BufferDesc = MTL::CounterSampleBufferDescriptor::alloc()->init();
		
		NS::Array* CounterSets = Device->GetDevice()->counterSets();
		const MTL::CounterSet* CounterSet = (MTL::CounterSet*)CounterSets->object(0);
		BufferDesc->setCounterSet(CounterSet);
		BufferDesc->setStorageMode(MTL::StorageModeShared);
		
		BufferDesc->setSampleCount(Size);
		
		NS::Error* DeviceError;
		SampleBuffer = Device->GetDevice()->newCounterSampleBuffer(BufferDesc, &DeviceError);
		
		BufferDesc->release();
		
		if(SampleBuffer)
		{
			Offset = 0;
			OutOffset = 0;
		}
		else
		{
			static bool bLoggedWarning = false;
			if (!bLoggedWarning)
			{
				UE_LOGF(LogMetal, Warning, "We have run out of Metal counter samplers which will negatively affect the GPU stats, you can try disabling Blit counter samplers with rhi.Metal.SampleBlitEncoderTimings=0");
				bLoggedWarning = true;
			}
		}
		
		return SampleBuffer;
	}
	
	check(Offset <= Size);
	
	return SampleBuffer;
}

FMetalCounterSamplePtr FMetalCounterSampler::SetupStageCounters(MTL::ComputePassDescriptor* ComputePassDesc)
{
	uint32_t OutOffset;
	MTL::CounterSampleBuffer* Buffer = SwapOrAllocateBuffer(2, OutOffset);
	if(!Buffer)
	{
		return nullptr;
	}
	
	FMetalCounterSamplePtr Sample = FMetalCounterSamplePtr(new FMetalCounterSample(EMetalCounterSampleType::ComputeStage, Buffer, OutOffset));
	MTL::ComputePassSampleBufferAttachmentDescriptor* SampleDesc = ComputePassDesc->sampleBufferAttachments()->object(0);
	
	SampleDesc->setSampleBuffer(Buffer);
	SampleDesc->setStartOfEncoderSampleIndex(OutOffset);
	SampleDesc->setEndOfEncoderSampleIndex(OutOffset+1);
	
	return Sample;
}

FMetalCounterSamplePtr FMetalCounterSampler::SetupStageCounters(MTL::BlitPassDescriptor* BlitPassDesc)
{
	uint32_t OutOffset;
	MTL::CounterSampleBuffer* Buffer = SwapOrAllocateBuffer(2, OutOffset);
	if(!Buffer)
	{
		return nullptr;
	}
	
	FMetalCounterSamplePtr Sample = FMetalCounterSamplePtr(new FMetalCounterSample(EMetalCounterSampleType::BlitStage, Buffer, OutOffset));
	MTL::BlitPassSampleBufferAttachmentDescriptor* SampleDesc = BlitPassDesc->sampleBufferAttachments()->object(0);
	
	SampleDesc->setSampleBuffer(Buffer);
	SampleDesc->setStartOfEncoderSampleIndex(OutOffset);
	SampleDesc->setEndOfEncoderSampleIndex(OutOffset+1);
	
	return Sample;
}

FMetalCounterSamplePtr FMetalCounterSampler::SetupStageCounters(MTL::RenderPassDescriptor* InRenderPassDesc)
{
	uint32_t OutOffset;
	MTL::CounterSampleBuffer* Buffer = SwapOrAllocateBuffer(2, OutOffset);
	if(!Buffer)
	{
		return nullptr;
	}
	
	FMetalCounterSamplePtr Sample = FMetalCounterSamplePtr(new FMetalCounterSample(EMetalCounterSampleType::RenderStage, Buffer, OutOffset));
	MTL::RenderPassSampleBufferAttachmentDescriptor* SampleDesc = InRenderPassDesc->sampleBufferAttachments()->object(0);
	
	SampleDesc->setSampleBuffer(Buffer);
	SampleDesc->setStartOfVertexSampleIndex(OutOffset);
	SampleDesc->setEndOfFragmentSampleIndex(OutOffset+1);
	
	return Sample;
}

FMetalCounterSamplePtr FMetalCounterSampler::SetupBoundaryCounters(MTL::RenderCommandEncoder* RenderCommandEncoder)
{
	uint32_t OutOffset;
	MTL::CounterSampleBuffer* Buffer = SwapOrAllocateBuffer(1, OutOffset);
	if(!Buffer)
	{
		return nullptr;
	}
	
	FMetalCounterSamplePtr Sample = FMetalCounterSamplePtr(new FMetalCounterSample(EMetalCounterSampleType::DrawBoundary, Buffer, OutOffset));
	
	RenderCommandEncoder->sampleCountersInBuffer(Buffer, OutOffset++, CVarMetalRHIInsertCounterSampleBarrier.GetValueOnAnyThread());
	
	return Sample;
}

FMetalCounterSamplePtr FMetalCounterSampler::SetupBoundaryCounters(MTL::ComputeCommandEncoder* ComputeCommandEncoder)
{
	uint32_t OutOffset;
	MTL::CounterSampleBuffer* Buffer = SwapOrAllocateBuffer(1, OutOffset);
	if(!Buffer)
	{
		return nullptr;
	}
	
	FMetalCounterSamplePtr Sample = FMetalCounterSamplePtr(new FMetalCounterSample(EMetalCounterSampleType::DispatchBoundary, Buffer, OutOffset));
	
	ComputeCommandEncoder->sampleCountersInBuffer(Buffer, OutOffset++, CVarMetalRHIInsertCounterSampleBarrier.GetValueOnAnyThread());
	
	return Sample;
}

FMetalCounterSamplePtr FMetalCounterSampler::SetupBoundaryCounters(MTL::BlitCommandEncoder* BlitCommandEncoder)
{
	uint32_t OutOffset;
	MTL::CounterSampleBuffer* Buffer = SwapOrAllocateBuffer(1, OutOffset);
	if(!Buffer)
	{
		return nullptr;
	}
	
	FMetalCounterSamplePtr Sample = FMetalCounterSamplePtr(new FMetalCounterSample(EMetalCounterSampleType::BlitBoundary, Buffer, OutOffset));
	
	BlitCommandEncoder->sampleCountersInBuffer(Buffer, OutOffset++, CVarMetalRHIInsertCounterSampleBarrier.GetValueOnAnyThread());
	
	return Sample;
}
