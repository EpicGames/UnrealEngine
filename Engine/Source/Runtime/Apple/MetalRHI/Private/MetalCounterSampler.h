// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalThirdParty.h"
#include "Containers/LockFreeList.h"

enum class EMetalCounterSampleType
{
	RenderStage = 0,
	ComputeStage,
	BlitStage,
	DrawBoundary,
	DispatchBoundary,
	BlitBoundary,
};

class FMetalDevice;
struct FMetalCounterSample
{
	FMetalCounterSample(EMetalCounterSampleType Type, MTL::CounterSampleBuffer* Buffer, uint32_t InOffset) 
		: SampleType(Type)
		, Offset(InOffset)
		, SampleBuffer(Buffer)
	{}
	
	bool IsStageType()
	{
		return SampleType <= EMetalCounterSampleType::BlitStage;
	}
	
	void ResolveStageCounters(uint64_t& InStartTime, uint64_t& InEndTime);
	void ResolveBoundaryCounter(uint64_t& Time);
	
	void* operator new(size_t Size)
	{
		check(Size == sizeof(FMetalCounterSample));

		void* Memory = MemoryPool.Pop();
		if (Memory == nullptr)
		{
			Memory = FMemory::Malloc(sizeof(FMetalCounterSample), alignof(FMetalCounterSample));
		}

		return Memory;
	}

	void operator delete(void* Pointer)
	{
		MemoryPool.Push(Pointer);
	}

	static TLockFreePointerListUnordered<void, PLATFORM_CACHE_LINE_SIZE> MemoryPool;
	
	EMetalCounterSampleType SampleType;
	uint32_t Offset = 0;
	MTL::CounterSampleBuffer* SampleBuffer;
	bool bResolved = false;
	uint64_t StartTime, EndTime;
};

typedef TSharedPtr<FMetalCounterSample> FMetalCounterSamplePtr;

class FMetalCounterSampler
{
public:
	FMetalCounterSampler(FMetalDevice* InDevice, uint32_t SampleCount);
	~FMetalCounterSampler();
	
	FMetalCounterSamplePtr SetupStageCounters(MTL::ComputePassDescriptor* ComputePassDesc);
	FMetalCounterSamplePtr SetupStageCounters(MTL::BlitPassDescriptor* BlitPassDesc);
	FMetalCounterSamplePtr SetupStageCounters(MTL::RenderPassDescriptor* InRenderPassDesc);
	FMetalCounterSamplePtr SetupBoundaryCounters(MTL::RenderCommandEncoder* RenderCommandEncoder);
	FMetalCounterSamplePtr SetupBoundaryCounters(MTL::ComputeCommandEncoder* ComputeCommandEncoder);
	FMetalCounterSamplePtr SetupBoundaryCounters(MTL::BlitCommandEncoder* BlitCommandEncoder);
	
	MTL::CounterSampleBuffer* SwapOrAllocateBuffer(uint32_t SampleSize, uint32_t& OutOffset);
	
private:
	FMetalDevice* Device;
	MTL::CounterSampleBuffer* SampleBuffer = nullptr;
	TArray<MTL::CounterSampleBuffer*> SampleBufferFreePool;
	FCriticalSection Mutex;
	
	uint32_t Offset = 0;
	uint32_t Size = 0;
};
