// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalDevice.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalRHI.h"
#include "MetalRHIPrivate.h"
#include "MetalRHIRenderQuery.h"
#include "MetalVertexDeclaration.h"
#include "MetalShaderTypes.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalCommandEncoder.h"
#include "MetalRHIContext.h"
#include "Misc/App.h"
#if PLATFORM_IOS
#include "IOS/IOSAppDelegate.h"
#endif
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformFramePacer.h"
#include "Runtime/HeadMountedDisplay/Public/IHeadMountedDisplayModule.h"

#include "MetalStaticSamplers.h"
#include "MetalProfiler.h"
#include "MetalCommandBuffer.h"

#include "MetalBindlessDescriptors.h"
#include "MetalTempAllocator.h"

#if PLATFORM_MAC
#include "Mac/MacGPUDescriptor.h"
#include "Misc/MessageDialog.h"
#endif

int32 GMetalSupportsIntermediateBackBuffer = 0;
static FAutoConsoleVariableRef CVarMetalSupportsIntermediateBackBuffer(
	TEXT("rhi.Metal.SupportsIntermediateBackBuffer"),
	GMetalSupportsIntermediateBackBuffer,
	TEXT("When enabled (> 0) allocate an intermediate texture to use as the back-buffer & blit from there into the actual device back-buffer (Off by default (0))"), ECVF_ReadOnly);

#if PLATFORM_MAC
static int32 GMetalCommandQueueSize = 5120; // This number is large due to texture streaming - currently each texture is its own command-buffer.
// The whole MetalRHI needs to be changed to use MTLHeaps/MTLFences & reworked so that operations with the same synchronisation requirements are collapsed into a single blit command-encoder/buffer.
#else
static int32 GMetalCommandQueueSize = 0;
#endif

#if METAL_DEBUG_OPTIONS
int32 GMetalBufferScribble = 0; // Deliberately not static, see InitFrame_UniformBufferPoolCleanup
static FAutoConsoleVariableRef CVarMetalBufferScribble(
	TEXT("rhi.Metal.BufferScribble"),
	GMetalBufferScribble,
	TEXT("Debug option: when enabled will scribble over the buffer contents with a single value when releasing buffer objects, or regions thereof. (Default: 0, Off)"));

static int32 GMetalResourceDeferDeleteNumFrames = 0;
static FAutoConsoleVariableRef CVarMetalResourceDeferDeleteNumFrames(
	TEXT("rhi.Metal.ResourceDeferDeleteNumFrames"),
	GMetalResourceDeferDeleteNumFrames,
	TEXT("Debug option: set to the number of frames that must have passed before resource free-lists are processed and resources disposed of. (Default: 0, Off)"));
#endif

int32 GMetalResourcePurgeOnDelete = 1;
static FAutoConsoleVariableRef CVarMetalResourcePurgeOnDelete(
	TEXT("rhi.Metal.ResourcePurgeOnDelete"),
	GMetalResourcePurgeOnDelete,
	TEXT("When enabled all MTLResource objects will have their backing stores purged on release - any subsequent access will be invalid and cause a command-buffer failure. Useful for making intermittent resource lifetime errors more common and easier to track. (Default: 0, Off)"));

#if UE_BUILD_SHIPPING
int32 GMetalRuntimeDebugLevel = 0;
#else
int32 GMetalRuntimeDebugLevel = 1;
#endif
static FAutoConsoleVariableRef CVarMetalRuntimeDebugLevel(
	TEXT("rhi.Metal.RuntimeDebugLevel"),
	GMetalRuntimeDebugLevel,
	TEXT("The level of debug validation performed by MetalRHI in addition to the underlying Metal API & validation layer.\n")
	TEXT("Each subsequent level adds more tests and reporting in addition to the previous level.\n")
	TEXT("*LEVELS >= 3 ARE IGNORED IN SHIPPING AND TEST BUILDS*. (Default: 1 (Debug, Development), 0 (Test, Shipping))\n")
	TEXT("\t0: Off,\n")
	TEXT("\t1: Enable light-weight validation of resource bindings & API usage,\n")
	TEXT("\t2: Reset resource bindings when binding a PSO/Compute-Shader to simplify GPU debugging,\n")
	TEXT("\t3: Allow rhi.Metal.CommandBufferCommitThreshold to break command-encoders (except when MSAA is enabled),\n")
	TEXT("\t4: Enable slower, more extensive validation checks for resource types & encoder usage,\n")
    TEXT("\t5: Wait for each command-buffer to complete immediately after submission."));

float GMetalPresentFramePacing = 0.0f;
#if !PLATFORM_MAC
static FAutoConsoleVariableRef CVarMetalPresentFramePacing(
	TEXT("rhi.Metal.PresentFramePacing"),
	GMetalPresentFramePacing,
	TEXT("Specify the desired frame rate for presentation (iOS 10.3+ only, default: 0.0f, off"));
#endif

#if PLATFORM_MAC
static int32 GMetalDefaultUniformBufferAllocation = 1024 * 1024 * 2;
#else
static int32 GMetalDefaultUniformBufferAllocation = 1024 * 256;
#endif
static FAutoConsoleVariableRef CVarMetalDefaultUniformBufferAllocation(
    TEXT("rhi.Metal.DefaultUniformBufferAllocation"),
    GMetalDefaultUniformBufferAllocation,
    TEXT("Default size of a uniform buffer allocation."));

#if PLATFORM_MAC
static int32 GMetalTargetUniformAllocationLimit = 1024 * 1024 * 50;
#else
static int32 GMetalTargetUniformAllocationLimit = 1024 * 1024 * 5;
#endif
static FAutoConsoleVariableRef CVarMetalTargetUniformAllocationLimit(
     TEXT("rhi.Metal.TargetUniformAllocationLimit"),
     GMetalTargetUniformAllocationLimit,
     TEXT("Target Allocation limit for the uniform buffer pool."));

#if PLATFORM_MAC
static int32 GMetalTargetTransferAllocatorLimit = 1024*1024*50;
#else
static int32 GMetalTargetTransferAllocatorLimit = 1024*1024*2;
#endif
static FAutoConsoleVariableRef CVarMetalTargetTransferAllocationLimit(
	TEXT("rhi.Metal.TargetTransferAllocationLimit"),
	GMetalTargetTransferAllocatorLimit,
	TEXT("Target Allocation limit for the upload staging buffer pool."));

#if PLATFORM_MAC
static int32 GMetalDefaultTransferAllocation = 1024*1024*10;
#else
static int32 GMetalDefaultTransferAllocation = 1024*1024*1;
#endif
static FAutoConsoleVariableRef CVarMetalDefaultTransferAllocation(
	TEXT("rhi.Metal.DefaultTransferAllocation"),
	GMetalDefaultTransferAllocation,
	TEXT("Default size of a single entry in the upload pool."));

static int32 GForceNoMetalFence = 1;
static FAutoConsoleVariableRef CVarMetalForceNoFence(
	TEXT("rhi.Metal.ForceNoFence"),
	GForceNoMetalFence,
	TEXT("[IOS] When enabled, act as if -nometalfence was on the commandline\n")
	TEXT("(On by default (1))"));

static int32 GForceNoMetalHeap = 1;
static FAutoConsoleVariableRef CVarMetalForceNoHeap(
	TEXT("rhi.Metal.ForceNoHeap"),
	GForceNoMetalHeap,
	TEXT("[IOS] When enabled, act as if -nometalheap was on the commandline\n")
	TEXT("(On by default (1))"));

int32 GMetalShaderValidationType = 0;
static FAutoConsoleVariableRef CVarMetalShaderValidationAll(
	TEXT("rhi.Metal.ShaderValidation.Type"),
	GMetalShaderValidationType,
	TEXT("Enable to set shader validation on specific types\n")
	TEXT("0: All shaders (slow, default) \n")
	TEXT("1: All compute shaders \n")
	TEXT("2: All render pipeline shaders \n")
	TEXT("3: Match shader name \n")
	TEXT("Enable to set shader validation on specific types\n"),
	ECVF_ReadOnly);

FString GMetalShaderValidationShaderName;
static FAutoConsoleVariableRef CVarMetalShaderValidationShaderName(
	TEXT("rhi.Metal.ShaderValidation.ShaderName"),
	GMetalShaderValidationShaderName,
	TEXT("Enable to set shader validation on compute shaders with a name\n"), ECVF_ReadOnly);


#if PLATFORM_MAC
static NS::Object* GMetalDeviceObserver;
#endif

MTL::PrimitiveTopologyClass TranslatePrimitiveTopology(uint32 PrimitiveType)
{
	switch (PrimitiveType)
	{
		case PT_TriangleList:
		case PT_TriangleStrip:
			return MTL::PrimitiveTopologyClassTriangle;
		case PT_LineList:
			return MTL::PrimitiveTopologyClassLine;
		case PT_PointList:
			return MTL::PrimitiveTopologyClassPoint;
		default:
			UE_LOGF(LogMetal, Fatal, "Unsupported primitive topology %d", (int32)PrimitiveType);
			return MTL::PrimitiveTopologyClassTriangle;
	}
}

FMetalDevice* FMetalDevice::CreateDevice()
{
#if PLATFORM_VISIONOS && UE_USE_SWIFT_UI_MAIN
	// get the device from the compositor layer
	MTL::Device* Device = (__bridge MTL::Device*)cp_layer_renderer_get_device([IOSAppDelegate GetDelegate].SwiftLayer);
#elif PLATFORM_IOS
	MTL::Device* Device = [IOSAppDelegate GetDelegate].IOSView->MetalDevice;
#else
	MTL::Device* Device = MTL::CreateSystemDefaultDevice();
	if (!Device)
	{
		static const FText UnsupportedMetal = NSLOCTEXT("MetalRHI", "UnsupportedMetal", "The graphics card in this Mac doesn't support Metal graphics technology, which is required to run this application. The application will now exit.");
		UE_LOGF(LogMetal, Log, "%ls", *UnsupportedMetal.ToString());
		FMessageDialog::Open(EAppMsgType::Ok, UnsupportedMetal); 
		FPlatformMisc::RequestExit(true, TEXT("FMetalDynamicRHI::FMetalDynamicRHI.UnsupportedMetal"));
	}
	// Ensure the GPU support is at least a M1
	else if(!Device->supportsFamily(MTL::GPUFamilyApple7))
	{
		static const FText UnsupportedIntelMac = NSLOCTEXT("MetalRHI", "UnsupportedIntelMac", "Rendering support for Intel based Mac has been removed in this version of the engine.");
		UE_LOGF(LogMetal, Log, "%ls", *UnsupportedIntelMac.ToString());
		FMessageDialog::Open(EAppMsgType::Ok, UnsupportedIntelMac); 
		FPlatformMisc::RequestExit(true, TEXT("FMetalDynamicRHI::FMetalDynamicRHI.UnsupportedIntelMac"));
	}
#endif
	
	uint32 MetalDebug = GMetalRuntimeDebugLevel;
	const bool bOverridesMetalDebug = FParse::Value( FCommandLine::Get(), TEXT( "MetalRuntimeDebugLevel=" ), MetalDebug );
	if (bOverridesMetalDebug)
	{
		GMetalRuntimeDebugLevel = MetalDebug;
	}
	
	FMetalDevice* MetalDevice = new FMetalDevice(Device);
	
#if !UE_BUILD_SHIPPING
	bool bShaderValidationEnabled = FParse::Param(FCommandLine::Get(), TEXT("metalshadervalidation"));
	MetalDevice->bShaderValidationEnabled = bShaderValidationEnabled;
#endif
	
	if (MetalDevice->SupportsFeature(EMetalFeaturesFences))
	{
		FMetalFencePool::Get().Initialise(Device);
	}
	
	return MetalDevice;
}

FMetalDevice::FMetalDevice(MTL::Device* MetalDevice)
	: Device(MetalDevice)
	, Heap(*this)
	, FrameCounter(0)
	, PSOManager(0)
	, FrameNumberRHIThread(0)
{
	Device->retain();
		
	EnumerateFeatureSupport();
	
	for(uint32_t Idx = 0; Idx < (uint32_t)EMetalQueueType::Count; ++Idx)
	{
		CommandQueues.Add(new FMetalCommandQueue(*this, GMetalCommandQueueSize));
		check(CommandQueues[Idx]);
	}
		
	RuntimeDebuggingLevel = GMetalRuntimeDebugLevel;
	
	CaptureManager = new FMetalCaptureManager(MetalDevice, *CommandQueues[(uint32_t)EMetalQueueType::Direct]);

	// Hook into the ios framepacer, if it's enabled for this platform.
	FrameReadyEvent = NULL;
	if( FPlatformRHIFramePacer::IsEnabled() )
	{
		FrameReadyEvent = FPlatformProcess::GetSynchEventFromPool();
		FPlatformRHIFramePacer::InitWithEvent( FrameReadyEvent );
		
		// A bit dirty - this allows the present frame pacing to match the CPU pacing by default unless you've overridden it with the CVar
		// In all likelihood the CVar is only useful for debugging.
		if (GMetalPresentFramePacing <= 0.0f)
		{
			FString FrameRateLockAsEnum;
			GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("FrameRateLock"), FrameRateLockAsEnum, GEngineIni);
	
			uint32 FrameRateLock = 0;
			FParse::Value(*FrameRateLockAsEnum, TEXT("PUFRL_"), FrameRateLock);
			if (FrameRateLock > 0)
			{
				GMetalPresentFramePacing = (float)FrameRateLock;
			}
		}
	}
	
    const bool bIsVisionOS = PLATFORM_VISIONOS;
	if (bIsVisionOS || FParse::Param(FCommandLine::Get(), TEXT("MetalIntermediateBackBuffer")) || FParse::Param(FCommandLine::Get(), TEXT("MetalOffscreenOnly")))
	{
		GMetalSupportsIntermediateBackBuffer = 1;
	}
    
    // initialize uniform and transfer allocators
    UniformBufferAllocator = new FMetalTempAllocator(*this, GMetalDefaultUniformBufferAllocation, GMetalTargetUniformAllocationLimit, BufferOffsetAlignment);
	TransferBufferAllocator = new FMetalTempAllocator(*this, GMetalDefaultTransferAllocation, GMetalTargetTransferAllocatorLimit, BufferBackedLinearTextureOffsetAlignment);
	
	PSOManager = new FMetalPipelineStateCacheManager(*this);

	FrameSemaphore = dispatch_semaphore_create(FParse::Param(FCommandLine::Get(),TEXT("gpulockstep")) ? 1 : 3);
}

FMetalDevice::~FMetalDevice()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.SubmitAndBlockUntilGPUIdle();
	
	for(uint32_t Idx = 0; Idx < (uint32_t)EMetalQueueType::Count; ++Idx)
	{
		delete CommandQueues[Idx];
	}

	delete PSOManager;
    delete UniformBufferAllocator;
	delete CaptureManager;
	delete CounterSampler;

    ShutdownPipelineCache();
    
#if METAL_RHI_RAYTRACING
	CleanUpRayTracing();
#endif
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
    delete BindlessDescriptorManager;
	delete StaticSamplers;
#endif
	
#if PLATFORM_MAC
    MTL::RemoveDeviceObserver(GMetalDeviceObserver);
#endif
	
	Device->release();
}

void FMetalDevice::PostInit()
{
#if METAL_RHI_RAYTRACING
	InitializeRayTracing();
#endif
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	BindlessDescriptorManager = new FMetalBindlessDescriptorManager(*this);
	StaticSamplers = new FMetalStaticSamplers(*this);
#endif
	
	CounterSampler = new FMetalCounterSampler(this, 4096);
	Heap.Init(GetCommandQueue(EMetalQueueType::Direct));
}

void FMetalDevice::EnumerateFeatureSupport()
{
	FString DeviceName(Device->name()->cString(NS::UTF8StringEncoding));
	
	if (!GForceNoMetalFence && FParse::Param(FCommandLine::Get(),TEXT("metalfence")))
	{
		Features |= EMetalFeaturesFences;
	}
	
#if !PLATFORM_MAC
	if (!GForceNoMetalHeap
		&& FParse::Param(FCommandLine::Get(),TEXT("metalheap")))
#endif
	{
		Features |= EMetalFeaturesHeaps;
	}
	
	if(Device->supportsFamily(MTL::GPUFamilyApple3))
	{
		Features |= EMetalFeaturesCountingQueries | EMetalFeaturesBaseVertexInstance | EMetalFeaturesIndirectBuffer | EMetalFeaturesMSAADepthResolve | EMetalFeaturesMSAAStoreAndResolve;
	}
	
#if PLATFORM_MAC // on iOS we use emulate_cube_array with SPIRV
	if( Device->supportsFamily(MTL::GPUFamilyApple4))
	{
		Features |= EMetalFeaturesCubemapArrays;
	}
#endif // PLATFORM_MAC
	
	if(Device->supportsFamily(MTL::GPUFamilyApple5))
	{
		Features |= EMetalFeaturesLayeredRendering;
	}
	
	// Check the device supports RT and for iPhones hardware RT specifically
	if(Device->supportsRaytracing() 
#if PLATFORM_IOS
	   && Device->supportsFamily(MTL::GPUFamilyApple9)
#endif
		)
	{
		Features |= EMetalFeaturesRayTracing;
	}
	
#if PLATFORM_IOS
	Features |= EMetalFeaturesPrivateBufferSubAllocation;
	Features |= EMetalFeaturesBufferSubAllocation;
	
	if(Device->supportsFamily(MTL::GPUFamilyApple4))
	{
		Features |= EMetalFeaturesTileShaders;
	}
					
#if !PLATFORM_TVOS
	Features |= EMetalFeaturesPresentMinDuration;	
#endif

	// Turning the below option on will allocate more buffer memory which isn't generally desirable on iOS
	// Features |= EMetalFeaturesEfficientBufferBlits;

#else // Assume that Mac & other platforms all support these from the start. They can diverge later.

	// Using Private Memory & BlitEncoders for Vertex & Index data should be *much* faster.
	Features |= EMetalFeaturesEfficientBufferBlits;
	Features |= EMetalFeaturesBufferSubAllocation;
#endif
	
#if !UE_BUILD_SHIPPING
	Class MTLDebugDevice = NSClassFromString(@"MTLDebugDevice");
	id<MTLDevice> ObjCDevice = (__bridge id<MTLDevice>)Device;
	if ([ObjCDevice isKindOfClass:MTLDebugDevice])
	{
		Features |= EMetalFeaturesValidation;
	}
#endif
	
#if WITH_PROFILEGPU
	// Counter Sampling Features
	if(Device->supportsCounterSampling(MTL::CounterSamplingPointAtStageBoundary))
	{
		Features |= EMetalFeaturesStageCounterSampling;
	}
		
	if(Device->supportsCounterSampling(MTL::CounterSamplingPointAtDrawBoundary) &&
	   Device->supportsCounterSampling(MTL::CounterSamplingPointAtDispatchBoundary) &&
	   Device->supportsCounterSampling(MTL::CounterSamplingPointAtBlitBoundary))
	{
		Features |= EMetalFeaturesBoundaryCounterSampling;
	}
#endif
}

void FMetalDevice::EndDrawingViewport(const FRHIPresentArgs& InPresentArgs)
{
	// We may be limiting our framerate to the display link
	if( FrameReadyEvent != nullptr  )
	{
		bool bIgnoreThreadIdleStats = true; // Idle time is already counted by the caller
		FrameReadyEvent->Wait(MAX_uint32, bIgnoreThreadIdleStats);
	}
	
	if(InPresentArgs.bPresent)
	{
		CaptureManager->PresentFrame(FrameCounter++);
	}
}

void FMetalDevice::DrainHeap()
{
	Heap.Compact(false);
}

void FMetalDevice::GarbageCollect()
{
	DrainHeap();
	
	TransferBufferAllocator->Cleanup();
	UniformBufferAllocator->Cleanup();
}

MTLTexturePtr FMetalDevice::CreateTexture(FMetalSurface* Surface, MTL::TextureDescriptor* Descriptor)
{
	MTLTexturePtr Tex = Heap.CreateTexture(Descriptor, Surface);
	if (GMetalResourcePurgeOnDelete && !Tex->heap())
	{
		Tex->setPurgeableState(MTL::PurgeableStateNonVolatile);
	}
	
	return Tex;
}

FMetalBufferPtr FMetalDevice::CreatePooledBuffer(FMetalPooledBufferArgs const& Args)
{
	NS::UInteger CpuResourceOption = ((NS::UInteger)Args.CpuCacheMode) << MTL::ResourceCpuCacheModeShift;
	
	uint32 RequestedBufferOffsetAlignment = BufferOffsetAlignment;
	
	if(EnumHasAnyFlags(Args.Flags, BUF_UnorderedAccess | BUF_ShaderResource))
	{
		// Buffer backed linear textures have specific align requirements
		// We don't know upfront the pixel format that may be requested for an SRV so we can't use minimumLinearTextureAlignmentForPixelFormat:
		RequestedBufferOffsetAlignment = BufferBackedLinearTextureOffsetAlignment;
	}
	
	MTL::ResourceOptions HazardTrackingMode = MTL::ResourceHazardTrackingModeUntracked;
	static bool bSupportsHeaps = SupportsFeature(EMetalFeaturesHeaps);
	if(bSupportsHeaps)
	{
		HazardTrackingMode = MTL::ResourceHazardTrackingModeTracked;
	}
	
    FMetalBufferPtr Buffer = Heap.CreateBuffer(Args.Size, RequestedBufferOffsetAlignment, Args.Flags, FMetalCommandQueue::GetCompatibleResourceOptions((MTL::ResourceOptions)(CpuResourceOption | HazardTrackingMode | ((NS::UInteger)Args.Storage << MTL::ResourceStorageModeShift))));
	
    check(Buffer);

    MTL::Buffer* MTLBuffer = Buffer->GetMTLBuffer();
	if (GMetalResourcePurgeOnDelete && !MTLBuffer->heap())
	{
        MTLBuffer->setPurgeableState(MTL::PurgeableStateNonVolatile);
	}
	
	return Buffer;
}

MTLEventPtr FMetalDevice::CreateEvent()
{
	MTLEventPtr Event = NS::TransferPtr(Device->newEvent());
	return Event;
}

MTLSharedEventPtr FMetalDevice::CreateSharedEvent()
{
	MTLSharedEventPtr Event = NS::TransferPtr(Device->newSharedEvent());
	return Event;
}

#if METAL_DEBUG_OPTIONS
void FMetalDevice::AddActiveBuffer(MTL::Buffer* Buffer, const NS::Range& Range)
{
    if(GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)
    {
        FScopeLock Lock(&ActiveBuffersMutex);
        
        NS::Range DestRange = NS::Range::Make(Range.location, Range.length);
        TArray<NS::Range>* Ranges = ActiveBuffers.Find(Buffer);
        if (!Ranges)
        {
            ActiveBuffers.Add(Buffer, TArray<NS::Range>());
            Ranges = ActiveBuffers.Find(Buffer);
        }
        Ranges->Add(DestRange);
    }
}

static bool operator==(NSRange const& A, NSRange const& B)
{
    return NSEqualRanges(A, B);
}

void FMetalDevice::RemoveActiveBuffer(MTL::Buffer* Buffer, const NS::Range& Range)
{
    if(GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)
    {
        FScopeLock Lock(&ActiveBuffersMutex);
        
        TArray<NS::Range>& Ranges = ActiveBuffers.FindChecked(Buffer);
        int32 i = Ranges.RemoveSingle(Range);
        check(i > 0);
    }
}

bool FMetalDevice::ValidateIsInactiveBuffer(MTL::Buffer* Buffer, const NS::Range& DestRange)
{
    if(GetRuntimeDebuggingLevel() >= EMetalDebugLevelValidation)
    {
        FScopeLock Lock(&ActiveBuffersMutex);
        
        TArray<NS::Range>* Ranges = ActiveBuffers.Find(Buffer);
        if (Ranges)
        {
            for (NS::Range Range : *Ranges)
            {
                if(DestRange.location < Range.location + Range.length ||
                   Range.location < DestRange.location + DestRange.length)
                {
                    continue;
                }
                
                UE_LOGF(LogMetal, Error, "ValidateIsInactiveBuffer failed on overlapping ranges ({%d, %d} vs {%d, %d}) of buffer %p.", (uint32)Range.location, (uint32)Range.length, (uint32)DestRange.location, (uint32)DestRange.length, Buffer);
                return false;
            }
        }
    }
    return true;
}
#endif
