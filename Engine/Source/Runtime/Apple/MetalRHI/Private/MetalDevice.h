// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "MetalViewport.h"
#include "MetalCommandQueue.h"
#include "MetalBuffer.h"
#include "MetalCaptureManager.h"
#include "MetalTempAllocator.h"
#include "MetalStateCache.h"
#include "MetalCounterSampler.h"

#if PLATFORM_IOS
#include "IOS/IOSView.h"
#endif
#include "Containers/LockFreeList.h"

// Defines a unique command queue type within a Metal Device (owner by the command list managers).
// Currently only implements direct
enum class EMetalQueueType
{
	Direct = 0,
	Count,
};

/**
 * Enumeration of features which are present only on some OS/device combinations.
 * These have to be checked at runtime as well as compile time to ensure backward compatibility.
 */
typedef NS_OPTIONS(uint32, EMetalFeatures)
{
	/** Supports NSUInteger counting visibility queries */
	EMetalFeaturesCountingQueries = 1 << 1,
	/** Supports base vertex/instance for draw calls */
	EMetalFeaturesBaseVertexInstance = 1 << 2,
	/** Supports indirect buffers for draw calls */
	EMetalFeaturesIndirectBuffer = 1 << 3,
	/** Supports layered rendering */
	EMetalFeaturesLayeredRendering = 1 << 4,
	/** Supports framework-level validation */
	EMetalFeaturesValidation = 1 << 5,
	/** Supports the explicit MTLHeap APIs */
	EMetalFeaturesHeaps = 1 << 6,
	/** Supports the explicit MTLFence APIs */
	EMetalFeaturesFences = 1 << 7,
	/** Supports MSAA Depth Resolves */
	EMetalFeaturesMSAADepthResolve = 1 << 8,
	/** Supports Store & Resolve in a single store action */
	EMetalFeaturesMSAAStoreAndResolve = 1 << 9,
	/** Supports the use of cubemap arrays */
	EMetalFeaturesCubemapArrays = 1 << 10,
	/** Supports the specification of multiple viewports and scissor rects */
	EMetalFeaturesMultipleViewports = 1 << 11,
	/** Supports minimum on-glass duration for drawables */
	EMetalFeaturesPresentMinDuration = 1 << 12,
	/** Supports efficient buffer-blits */
	EMetalFeaturesEfficientBufferBlits = 1 << 13,
	/** Supports any kind of buffer sub-allocation */
	EMetalFeaturesBufferSubAllocation = 1 << 14,
	/** Supports private buffer sub-allocation */
	EMetalFeaturesPrivateBufferSubAllocation = 1 << 15,
	/** Supports tile shaders */
	EMetalFeaturesTileShaders = 1 << 16,
	/** Supports counter sampling on encoder stages */
	EMetalFeaturesStageCounterSampling = 1 << 17,
	/** Supports counter sampling on the stage boundary */
	EMetalFeaturesBoundaryCounterSampling = 1 << 18,
	/** Whether the device supports ray tracing */
	EMetalFeaturesRayTracing = 1 << 19,
};

/**
 * EMetalDebugLevel: Level of Metal debug features to be enabled.
 */
enum EMetalDebugLevel
{
	EMetalDebugLevelOff,
	EMetalDebugLevelFastValidation,
	EMetalDebugLevelResetOnBind,
	EMetalDebugLevelConditionalSubmit,
	EMetalDebugLevelValidation,
	EMetalDebugLevelWaitForComplete,
};

/**
 * EMetalShaderValidationType: Type of Metal Shader Validation to perform
 */
enum class EMetalShaderValidationType
{
	All,
	ComputeOnly,
	RenderPipelineOnly,
	ShaderName,
};

class FMetalBindlessDescriptorManager;
class FMetalStaticSamplers;
class FMetalRayTracingCompactionRequestHandler;
class FMetalRayTracingPipelineCache;

class FMetalDevice
{
public:
	static FMetalDevice* CreateDevice();
	virtual ~FMetalDevice();
	
	void PostInit();
	
	void EnumerateFeatureSupport();
	inline bool SupportsFeature(EMetalFeatures InFeature) { return ((Features & InFeature) != 0); }
	
	inline FMetalResourceHeap& GetResourceHeap(void) { return Heap; }
	
	void EndDrawingViewport(const FRHIPresentArgs& InPresentArgs);
	
	MTLTexturePtr CreateTexture(FMetalSurface* Surface, MTL::TextureDescriptor* Descriptor);
	FMetalBufferPtr CreatePooledBuffer(FMetalPooledBufferArgs const& Args);
	MTLEventPtr CreateEvent();
	MTLSharedEventPtr CreateSharedEvent();
	
	void DrainHeap();
	void GarbageCollect();
	
	FMetalTempAllocator* GetTransferAllocator()
	{
		return TransferBufferAllocator;
	}
    
    FMetalTempAllocator* GetUniformAllocator()
    {
        return UniformBufferAllocator;
    }
    
    uint32 GetFrameNumberRHIThread()
    {
        return FrameNumberRHIThread;
    }
	
	FMetalCommandQueue& GetCommandQueue(EMetalQueueType QueueType)
	{
		check(QueueType < EMetalQueueType::Count);
		return *CommandQueues[(uint32_t)QueueType];
	}
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
    FMetalBindlessDescriptorManager* GetBindlessDescriptorManager()
    {
        return BindlessDescriptorManager;
    }
	
	FMetalStaticSamplers* GetStaticSamplers()
	{
		return StaticSamplers;
	}
#endif
	
#if METAL_DEBUG_OPTIONS
    void AddActiveBuffer(MTL::Buffer* Buffer, const NS::Range& Range);
    void RemoveActiveBuffer(MTL::Buffer* Buffer, const NS::Range& Range);
	bool ValidateIsInactiveBuffer(MTL::Buffer* Buffer, const NS::Range& Range);
#endif
	
	MTL::Device* GetDevice()
	{
		return Device;
	}
	
	inline int32 GetRuntimeDebuggingLevel(void) const
	{
		return RuntimeDebuggingLevel;
	}
	
	void IncrementFrameRHIThread()
	{
		FrameNumberRHIThread++;
	}
	
	dispatch_semaphore_t& GetFrameSemaphore()
	{
		return FrameSemaphore;
	}
	
	FMetalCounterSampler* GetCounterSampler()
	{
		return CounterSampler;
	}
	
	bool IsShaderValidationEnabled()
	{
		return bShaderValidationEnabled;
	}
	
	void ForceHeapSupport()
	{
		Features |= EMetalFeaturesHeaps;
	}
private:
	FMetalDevice(MTL::Device* MetalDevices);
	
	void FlushFreeList(bool const bFlushFences = true);
	
private:
	MTL::Device* Device;
	
	TArray<FMetalCommandQueue*, TInlineAllocator<(uint32)EMetalQueueType::Count>> CommandQueues;
	
	/** A sempahore used to ensure that wait for previous frames to complete if more are in flight than we permit */
	dispatch_semaphore_t FrameSemaphore;
	
	/** Dynamic memory heap */
	FMetalResourceHeap Heap;
	
	/** GPU Frame Capture Manager */
	FMetalCaptureManager* CaptureManager;
	
    FMetalTempAllocator* UniformBufferAllocator;
	FMetalTempAllocator* TransferBufferAllocator;
	
#if METAL_DEBUG_OPTIONS
	/** The list of fences for the current frame */
	TArray<FMetalFence*> FrameFences;
    
    FCriticalSection ActiveBuffersMutex;
    
    /** These are the active buffers that cannot be CPU modified */
    TMap<MTL::Buffer*, TArray<NS::Range>> ActiveBuffers;
#endif
	
	/** Critical section for FreeList */
	FCriticalSection FreeListMutex;
	
	/** Event for coordinating pausing of render thread to keep inline with the ios display link. */
	FEvent* FrameReadyEvent;
	
	/** Internal frame counter, used to ensure that we only drain the buffer pool one after each frame within RHIEndFrame. */
	uint32 FrameCounter = 0;
	
	/** Bitfield of supported Metal features with varying availability depending on OS/device */
	uint64 Features = 0;
	
	/** PSO cache manager */
	FMetalPipelineStateCacheManager* PSOManager;

    /** Thread index owned by the RHI Thread. Monotonically increases every call to EndFrame() */
    uint32 FrameNumberRHIThread = 0;

	int32 RuntimeDebuggingLevel = 0;
	
	FMetalCounterSampler* CounterSampler = nullptr;
	
	bool bShaderValidationEnabled = false;
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
    /** Bindless Descriptor Heaps manager. */
    FMetalBindlessDescriptorManager* BindlessDescriptorManager;
	FMetalStaticSamplers* StaticSamplers;
#endif

#if METAL_RHI_RAYTRACING
	FMetalRayTracingCompactionRequestHandler* RayTracingCompactionRequestHandler;
	FMetalRayTracingPipelineCache* RayTracingPipelineCache;
	
	void InitializeRayTracing();
	void CleanUpRayTracing();
	
public:
	void UpdateRayTracing(FMetalRHICommandContext& Context);

	inline FMetalRayTracingCompactionRequestHandler* GetRayTracingCompactionRequestHandler() const { return RayTracingCompactionRequestHandler; }
	inline FMetalRayTracingPipelineCache* GetRayTracingPipelineCache() const { return RayTracingPipelineCache; }
	
	// Dummy Index buffer used when creating descriptors
	FMetalBufferPtr DummyIndexBuffer;
#endif // METAL_RHI_RAYTRACING
};

