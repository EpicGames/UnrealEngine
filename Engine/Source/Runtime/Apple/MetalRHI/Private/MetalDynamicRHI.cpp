// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalDynamicRHI.cpp: Metal Dynamic RHI Class Implementation.
=============================================================================*/

#include "MetalDynamicRHI.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalRHIPrivate.h"
#include "MetalRHIRenderQuery.h"
#include "MetalRHIStagingBuffer.h"
#include "MetalShaderTypes.h"
#include "MetalVertexDeclaration.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalTransitionData.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Vertex Declaration Methods -

FVertexDeclarationRHIRef FMetalDynamicRHI::RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    uint32 Key = FCrc::MemCrc32(Elements.GetData(), Elements.Num() * sizeof(FVertexElement));

    // look up an existing declaration
    FVertexDeclarationRHIRef* VertexDeclarationRefPtr = VertexDeclarationCache.Find(Key);
    if (VertexDeclarationRefPtr == NULL)
    {
        // create and add to the cache if it doesn't exist.
        VertexDeclarationRefPtr = &VertexDeclarationCache.Add(Key, new FMetalVertexDeclaration(Elements));
    }

    return *VertexDeclarationRefPtr;
}


//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Pipeline State Methods -


FGraphicsPipelineStateRHIRef FMetalDynamicRHI::RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;

    TRefCountPtr<FMetalGraphicsPipelineState> State = new FMetalGraphicsPipelineState(Initializer);

#if METAL_USE_METAL_SHADER_CONVERTER
	
	if(IsMetalBindlessEnabled())
	{
		FMetalVertexShader* VertexShader = ResourceCast(Initializer.BoundShaderState.VertexShaderRHI);
		
		if (VertexShader != nullptr)
		{
			FMetalVertexDeclaration* VertexDeclaration = ResourceCast(Initializer.BoundShaderState.VertexDeclarationRHI);
			
			IRMetalLibBinary* StageInMetalLib = IRMetalLibBinaryCreate();
			
			const FString& SerializedJSON = VertexShader->Bindings.IRConverterReflectionJSON;
			IRShaderReflection* VertexReflection = IRShaderReflectionCreateFromJSON(TCHAR_TO_ANSI(*SerializedJSON));
			
			bool bStageInCreationSuccessful = IRMetalLibSynthesizeStageInFunction(CompilerInstance,
																				  VertexReflection,
																				  &VertexDeclaration->InputDescriptor,
																				  StageInMetalLib);
			check(bStageInCreationSuccessful)
			
			// Store bytecode for lib/stagein function creation.
			size_t MetallibSize = IRMetalLibGetBytecodeSize(StageInMetalLib);
			State->StageInFunctionBytecode.SetNum(MetallibSize);
			size_t WrittenBytes = IRMetalLibGetBytecode(StageInMetalLib, reinterpret_cast<uint8_t*>(State->StageInFunctionBytecode.GetData()));
			check(MetallibSize == WrittenBytes);
			
			IRMetalLibBinaryDestroy(StageInMetalLib);
			IRShaderReflectionDestroy(VertexReflection);
		}
	}
#endif

    if(!State->Compile())
    {
        // Compilation failures are propagated up to the caller.
        return nullptr;
    }

    State->VertexDeclaration = ResourceCast(Initializer.BoundShaderState.VertexDeclarationRHI);
#if PLATFORM_SUPPORTS_MESH_SHADERS
    State->MeshShader = ResourceCast(Initializer.BoundShaderState.GetMeshShader());
    State->AmplificationShader = ResourceCast(Initializer.BoundShaderState.GetAmplificationShader());
#endif
    State->VertexShader = ResourceCast(Initializer.BoundShaderState.VertexShaderRHI);
    State->PixelShader = ResourceCast(Initializer.BoundShaderState.PixelShaderRHI);
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    State->GeometryShader = ResourceCast(Initializer.BoundShaderState.GetGeometryShader());
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS

    State->DepthStencilState = ResourceCast(Initializer.DepthStencilState);
    State->RasterizerState = ResourceCast(Initializer.RasterizerState);

    return FGraphicsPipelineStateRHIRef(MoveTemp(State));
}

TRefCountPtr<FRHIComputePipelineState> FMetalDynamicRHI::RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	return new FRHIComputePipelineState(Initializer.ComputeShader);
}


//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Staging Buffer Methods -


FStagingBufferRHIRef FMetalDynamicRHI::RHICreateStagingBuffer()
{
	return new FMetalRHIStagingBuffer(*Device);
}

void* FMetalDynamicRHI::RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI)
{
	if (Fence && !Fence->Poll())
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		RHICmdList.SubmitAndBlockUntilGPUIdle();
		
		ResourceCast(Fence)->Wait(RHICmdList, FRHIGPUMask::All());
	}
	
	FMetalRHIStagingBuffer* Buffer = ResourceCast(StagingBuffer);
	return Buffer->Lock(Offset, SizeRHI);
}

void FMetalDynamicRHI::RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer)
{
	FMetalRHIStagingBuffer* Buffer = ResourceCast(StagingBuffer);
	Buffer->Unlock();
}


//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Resource Transition Methods -


void FMetalDynamicRHI::RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo)
{
	// Construct the data in-place on the transition instance
	new (Transition->GetPrivateData<FMetalTransitionData>()) FMetalTransitionData(CreateInfo.SrcPipelines, CreateInfo.DstPipelines, CreateInfo.Flags, CreateInfo.TransitionInfos);
}

void FMetalDynamicRHI::RHIReleaseTransition(FRHITransition* Transition)
{
	// Destruct the private data object of the transition instance.
	Transition->GetPrivateData<FMetalTransitionData>()->~FMetalTransitionData();
}


//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Render Query Methods -


FRenderQueryRHIRef FMetalDynamicRHI::RHICreateRenderQuery(ERenderQueryType QueryType)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    FRenderQueryRHIRef Query = new FMetalRHIRenderQuery(*Device, QueryType);
	return Query;
}

bool FMetalDynamicRHI::RHIGetRenderQueryResult(FRHIRenderQuery* QueryRHI, uint64& OutNumPixels, bool bWait, uint32 GPUIndex)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalRHIRenderQuery* Query = ResourceCast(QueryRHI);
	return Query->GetResult(OutNumPixels, bWait, GPUIndex);
}

uint64 FMetalDynamicRHI::RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	// When compute precache PSO hash we assume a valid state precache PSO hash is already provided
	uint64 StatePrecachePSOHash = Initializer.StatePrecachePSOHash;
	if (StatePrecachePSOHash == 0)
	{
		StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(Initializer);
	}

	// All members which are not part of the state objects and influence the PSO on Metal
	struct FNonStateHashKey
	{
		uint64							StatePrecachePSOHash;

		uint32							RenderTargetsEnabled;
		FGraphicsPipelineStateInitializer::TRenderTargetFormats RenderTargetFormats;
		EPixelFormat					DepthStencilTargetFormat;
		uint16							NumSamples;
		EConservativeRasterization		ConservativeRasterization;
	} HashKey;

	FMemory::Memzero(&HashKey, sizeof(FNonStateHashKey));

	HashKey.StatePrecachePSOHash			= StatePrecachePSOHash;

	HashKey.RenderTargetsEnabled			= Initializer.RenderTargetsEnabled;
	HashKey.RenderTargetFormats				= Initializer.RenderTargetFormats;
	HashKey.DepthStencilTargetFormat		= Initializer.DepthStencilTargetFormat;
	HashKey.NumSamples						= Initializer.NumSamples;
	HashKey.ConservativeRasterization		= Initializer.ConservativeRasterization;

	return CityHash64((const char*)&HashKey, sizeof(FNonStateHashKey));
}

bool FMetalDynamicRHI::RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS)
{
	// first check non pointer objects
	if (LHS.PrimitiveType != RHS.PrimitiveType ||
		LHS.bDepthBounds != RHS.bDepthBounds ||
		LHS.MultiViewCount != RHS.MultiViewCount ||
		LHS.ShadingRate != RHS.ShadingRate ||
		LHS.bHasFragmentDensityAttachment != RHS.bHasFragmentDensityAttachment ||
		LHS.RenderTargetsEnabled != RHS.RenderTargetsEnabled ||
		LHS.RenderTargetFormats != RHS.RenderTargetFormats ||
		LHS.DepthStencilTargetFormat != RHS.DepthStencilTargetFormat ||
		LHS.NumSamples != RHS.NumSamples ||
		LHS.ConservativeRasterization != RHS.ConservativeRasterization)
	{
		return false;
	}

	// check the RHI shaders (pointer check for shaders should be fine)
	if (LHS.BoundShaderState.GetVertexShader() != RHS.BoundShaderState.GetVertexShader() ||
		LHS.BoundShaderState.GetPixelShader() != RHS.BoundShaderState.GetPixelShader() ||
		LHS.BoundShaderState.GetMeshShader() != RHS.BoundShaderState.GetMeshShader() ||
		LHS.BoundShaderState.GetAmplificationShader() != RHS.BoundShaderState.GetAmplificationShader() ||
		LHS.BoundShaderState.GetGeometryShader() != RHS.BoundShaderState.GetGeometryShader())
	{
		return false;
	}

	// Compare the VertexDecl
	FMetalHashedVertexDescriptor LHSVertexElements;
	if (LHS.BoundShaderState.VertexDeclarationRHI)
	{
		LHSVertexElements = ((FMetalVertexDeclaration*)LHS.BoundShaderState.VertexDeclarationRHI)->Layout;
	}
	FMetalHashedVertexDescriptor RHSVertexElements;
	if (RHS.BoundShaderState.VertexDeclarationRHI)
	{
		RHSVertexElements = ((FMetalVertexDeclaration*)RHS.BoundShaderState.VertexDeclarationRHI)->Layout;
	}
	if (!(LHSVertexElements == RHSVertexElements))
	{
		return false;
	}

	// Check actual state content (each initializer can have it's own state and not going through a factory)
	if (!MatchRHIState<FRHIBlendState, FBlendStateInitializerRHI>(LHS.BlendState, RHS.BlendState) ||
		!MatchRHIState<FRHIRasterizerState, FRasterizerStateInitializerRHI>(LHS.RasterizerState, RHS.RasterizerState) ||
		!MatchRHIState<FRHIDepthStencilState, FDepthStencilStateInitializerRHI>(LHS.DepthStencilState, RHS.DepthStencilState))
	{
		return false;
	}

	return true;
}

void FMetalDynamicRHI::RHIRunOnQueue(TFunction<void(MTL::CommandQueue*)>&& CodeToRun, bool bWaitForSubmission)
{
	FGraphEventRef SubmissionEvent;

	TArray<FMetalPayload*> Payloads;
	FMetalPayload* Payload = new FMetalPayload(Device->GetCommandQueue(EMetalQueueType::Direct));
	Payloads.Add(Payload);

	Payload->PreExecuteCallback = MoveTemp(CodeToRun);

	if (bWaitForSubmission)
	{
		SubmissionEvent = FGraphEvent::CreateGraphEvent();
		Payload->SubmissionEvent = SubmissionEvent;
	}

	SubmitPayloads(MoveTemp(Payloads));
	
	if (SubmissionEvent && !SubmissionEvent->IsComplete())
	{
		SubmissionEvent->Wait();
	}
}
