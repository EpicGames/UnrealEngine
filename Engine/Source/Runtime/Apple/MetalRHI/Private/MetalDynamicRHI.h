// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalDynamicRHI.h: Public Metal RHI definitions..
=============================================================================*/

#pragma once 

#include "CoreMinimal.h"
#include "IMetalDynamicRHI.h"

// Metal RHI public headers.
#include "MetalThirdParty.h"
#include "MetalState.h"
#include "MetalResources.h"
#include "MetalRHIContext.h"
#include "MetalViewport.h"
#include "MetalSubmission.h"

class FMetalDevice;

#if METAL_RHI_RAYTRACING
class FMetalRayTracingCompactionRequestHandler;
#endif // METAL_RHI_RAYTRACING

struct FMetalDeferredDeleteObject
{
	typedef TVariant<FMetalBufferPtr, 
					MTLTexturePtr,
					NS::Object*,
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
					FRHIDescriptorHandle,
#endif
					FMetalFence*,
#if METAL_RHI_RAYTRACING
					FMetalAccelerationStructure*,
#endif
					TUniqueFunction<void()>*> TObjectStorage;
	
	TObjectStorage Storage;
	
	explicit FMetalDeferredDeleteObject(FMetalBufferPtr InBuffer) : Storage(TInPlaceType<FMetalBufferPtr>(), InBuffer)
	{}

	explicit FMetalDeferredDeleteObject(MTLTexturePtr InTexture) : Storage(TInPlaceType<MTLTexturePtr>(), InTexture)
	{}
	
	explicit FMetalDeferredDeleteObject(NS::Object* InObject) : Storage(TInPlaceType<NS::Object*>(), InObject)
	{}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	explicit FMetalDeferredDeleteObject(FRHIDescriptorHandle InHandle) : Storage(TInPlaceType<FRHIDescriptorHandle>(), InHandle)
	{}
#endif

	explicit FMetalDeferredDeleteObject(FMetalFence* InFence) : Storage(TInPlaceType<FMetalFence*>(), InFence)
	{}
	
#if METAL_RHI_RAYTRACING
	explicit FMetalDeferredDeleteObject(FMetalAccelerationStructure* InAccelerationStructure) : Storage(TInPlaceType<FMetalAccelerationStructure*>(), InAccelerationStructure)
	{}
#endif
	
	explicit FMetalDeferredDeleteObject(TUniqueFunction<void()>&& Func) : 
			Storage(TInPlaceType<TUniqueFunction<void()>*>(), new TUniqueFunction<void()>(MoveTemp(Func)))
	{}
};

struct FMetalFinalizedCommands final : public IRHIPlatformCommandList, public TArray<FMetalPayload*> 
{};

/** The interface which is implemented by the dynamically bound RHI. */
class FMetalDynamicRHI : public IMetalDynamicRHI
{
	static inline FMetalDynamicRHI* Singleton = nullptr;
public:
	enum class EQueueStatus
	{
		None = 0,

		// Work was processed through the queue.
		Processed = 1 << 0,

		// The queue has further, unprocessed work.
		Pending = 1 << 1
	};
	FRIEND_ENUM_CLASS_FLAGS(EQueueStatus);

	struct FProcessResult
	{
		EQueueStatus Status = EQueueStatus::None;
	};
	
	static inline FMetalDynamicRHI& Get() { return *Singleton; }

	/** Initialization constructor. */
	FMetalDynamicRHI(ERHIFeatureLevel::Type RequestedFeatureLevel);

	/** Destructor */
	~FMetalDynamicRHI();
	
	// FDynamicRHI interface.
	virtual void Init();
	virtual void Shutdown() {}
	virtual const TCHAR* GetName() override { return TEXT("Metal"); }

	virtual void RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList) final override;
	virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) final override;

	virtual FRHIShaderLibraryRef RHICreateShaderLibrary(EShaderPlatform Platform, FString const& FilePath, FString const& Name) final override;
	virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) final override;
	virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) final override;
	virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) final override;
	virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) final override;
	virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) final override;
	virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
#if PLATFORM_SUPPORTS_MESH_SHADERS
    virtual FMeshShaderRHIRef RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
    virtual FAmplificationShaderRHIRef RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
#endif
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) final override;
	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual TRefCountPtr<FRHIComputePipelineState> RHICreateComputePipelineState(const FComputePipelineStateInitializer& Initializer) final override;
	virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) final override;

	[[nodiscard]] virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) final override;

	virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) final override;
	virtual void RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr) final override;
	
	virtual void * RHILockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode) final override;
	virtual void RHIUnlockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
	virtual void* LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode) final override;
	virtual void UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#endif

	// SRV / UAV creation functions
	virtual FShaderResourceViewRHIRef  RHICreateShaderResourceView (class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;
	virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	virtual FRHIResourceCollectionRef RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers) final override;
	virtual void RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates) final override;
#endif

	virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) final override;
	virtual uint64 RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format) final override;
	virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) final override;
	virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData,int32 SizeX,int32 SizeY,int32 Pitch,int32 PixelSize) final override;
	[[nodiscard]] virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc) final override;
	virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) final override;
	virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) final override;

	virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;
	virtual void                  RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;

	virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) final override;
	virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) final override;
	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) final override;
	virtual void RHIReadSurfaceData(FRHITexture* Texture,FIntRect Rect,TArray<FColor>& OutData,FReadSurfaceDataFlags InFlags) final override;
	virtual void RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
#if PLATFORM_IOS
	virtual bool RHIReadViewportDataDirect(uint32& OutWidth, uint32& OutHeight, TArray<FColor>& OutData) final override;
#endif // PLATFORM_IOS
	virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) final override;
	virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) final override;
	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex) final override;
	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData) final override;
	virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) final override;
	virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) final override;
	virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) final override;
	virtual FTextureRHIRef RHIGetViewportBackBufferAtLayerIndex(FRHIViewport* Viewport, uint32 DrawLayerIndex) final override;
	virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) final override;
	virtual void RHITick(float DeltaTime) final override;
	virtual void RHIBlockUntilGPUIdle() final override;
	virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) final override;
	virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) final override;
	virtual MTL::Device* RHIGetDevice() final override;
	virtual void* METALRHI_API RHIGetNativeDevice() final override;
	virtual void* METALRHI_API RHIGetNativeGraphicsQueue() final override;
	virtual void* METALRHI_API RHIGetNativeComputeQueue() final override;
	
	virtual class IRHICommandContext* METALRHI_API RHIGetDefaultContext() final override;
	virtual IRHIUploadContext* RHIGetUploadContext() final override;
	
	virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) final override;
	virtual IRHICommandContext* RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass) final override;

	virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) final override;
	virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) final override;
	virtual void RHIProcessDeleteQueue() final override;

	virtual FUpdateTexture3DData RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion) final override;
	virtual void RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData) final override;

	virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo) final override;
	virtual void RHIReleaseTransition(FRHITransition* Transition) final override;

	virtual FGPUFenceRHIRef RHICreateGPUFence(const FName &Name) final override;
	virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI) final override;

	virtual FStagingBufferRHIRef RHICreateStagingBuffer() final override;
	virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) final override;
	virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer) final override;

	virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) final override;

	virtual uint16 RHIGetPlatformTextureMaxSampleCount() override;

#if METAL_RHI_RAYTRACING
	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer) final override;
	virtual FRayTracingAccelerationStructureSize RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer) final override;

	virtual FRayTracingGeometryRHIRef RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer) final override;
	virtual FRayTracingSceneRHIRef RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer) final override;
	virtual FRayTracingShaderRHIRef RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency) final override;
	virtual FRayTracingPipelineStateRHIRef RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer) final override;

	virtual FShaderBindingTableRHIRef RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer) final override;
#endif // METAL_RHI_RAYTRACING

	virtual FTextureReferenceRHIRef RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture) final override;
	virtual void RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* NewTexture) final override;


	virtual uint64 RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;
	virtual bool RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS) final override;
	
	virtual void RHIBeginRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override;
	virtual void RHIEndRenderQuery_TopOfPipe  (FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override;
	
	virtual void RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) final override;
	virtual void RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType) final override;
	
	virtual void 				RHIRunOnQueue(TFunction<void(MTL::CommandQueue*)>&& CodeToRun, bool bWaitForSubmission) final override;
	
	virtual FTextureRHIRef		RHICreateTexture2DFromCVMetalTexture(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, CVMetalTextureRef Resource) final override;
	
	template <typename ...Args>
	void DeferredDelete(Args&&... InArgs)
	{
		check(!IsInGameThread() || !IsRunningRHIInSeparateThread());
		FScopeLock Lock(&ObjectsToDeleteCS);
		ObjectsToDelete.Emplace(Forward<Args>(InArgs)...);
	}
	
	void SubmitCommands(TConstArrayView<struct FMetalFinalizedCommands*> Commands);
	void SubmitPayloads(TArray<FMetalPayload*>&& Payloads);
	
	// Ensures all prior batched command lists have reached the MetalQueue
	void FlushBatchedPayloads(FMetalCommandQueue::FPayloadArray& PayloadsToHandDown);

	// Processes the interrupt queue on the calling thread, until the specified GraphEvent is signaled.
	// If the GraphEvent is nullptr, processes the queue until no further progress is made.
	void ProcessInterruptQueueUntil(FGraphEvent* GraphEvent);
	
	void ProcessPendingCommandBuffers(TArrayView<FMetalCommandBuffer*> CommandBuffers);

	// Called when the GPU has crashed. This function will not return.
	void ProcessInterruptQueueOnGPUCrash();

	struct FMetalTimingArray : public TArray<TUniquePtr<FMetalTiming>, TInlineAllocator<GMetalMaxNumQueues>>
	{
		FMetalTiming* CreateNew(FMetalCommandQueue& Queue)
		{
			return Emplace_GetRef(MakeUnique<FMetalTiming>(Queue)).Get();
		}
	};

	FMetalTimingArray CurrentTimingPerQueue;
	
	void InitializeSubmissionPipe();
	void ShutdownSubmissionPipe();
	
	// Inserts a task graph task which is executed once all previously submitted GPU work has completed (across all queues, device and adapters).
	void EnqueueEndOfPipeTask(TUniqueFunction<void()> TaskFunc, TUniqueFunction<void(FMetalPayload&)> ModifyPayloadCallback = {});
	FGraphEventRef EopTask;
	
	FCriticalSection SubmissionCS;
	FCriticalSection InterruptCS;

	FProcessResult ProcessSubmissionQueue();
	FProcessResult ProcessInterruptQueue();
	
	// Enumerates all queues across all devices and active adapters
	void ForEachQueue(TFunctionRef<void(FMetalCommandQueue&)> Callback);

private:
	FMetalDevice* Device;
	FTextureMemoryStats MemoryStats;
	TMap<uint32, FVertexDeclarationRHIRef> VertexDeclarationCache;
	TLockFreePointerListUnordered<FMetalRHICommandContext, PLATFORM_CACHE_LINE_SIZE> MetalCommandContextPool;
	
	FCriticalSection ObjectsToDeleteCS;
	TArray<FMetalDeferredDeleteObject> ObjectsToDelete;

#if METAL_USE_METAL_SHADER_CONVERTER
    struct IRCompiler* CompilerInstance;
#endif
	
	friend class FMetalThread;
	class FMetalThread* SubmissionThread = nullptr;
	class FMetalThread* InterruptThread = nullptr;
	
	TQueue<TArray<FMetalPayload*>*, EQueueMode::Mpsc> PendingPayloadsForSubmission;
};

ENUM_CLASS_FLAGS(FMetalDynamicRHI::EQueueStatus);
