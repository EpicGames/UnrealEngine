// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MetalDynamicRHIModule.cpp: Metal Dynamic RHI Module Class Implementation.
==============================================================================*/

#include "MetalDynamicRHIModule.h"
#include "MetalDynamicRHI.h"
#include "MetalLLM.h"
#include "DynamicRHI.h"
#include "RHIValidation.h"
#include "Modules/ModuleManager.h"
#include "Shaders/MetalIoStoreShaderCodeArchive.h"


//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Module Implementation


IMPLEMENT_MODULE(FMetalDynamicRHIModule, MetalRHI);


//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Module Class Methods


bool FMetalDynamicRHIModule::IsSupported()
{
	return true;
}

FDynamicRHI* FMetalDynamicRHIModule::CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel)
{
	FIoStoreShaderCodeArchive::RegisterIoStoreShaderCodeArchiveFactory([](EShaderPlatform Platform, const FString& Name, FIoDispatcher& IoDispatcher)
	{
		return static_cast<FIoStoreShaderCodeArchive*>(new FMetalIoStoreShaderCodeArchive(Platform, Name, IoDispatcher));
	});

	LLM(MetalLLM::Initialise());
	FDynamicRHI* FinalRHI =  new FMetalDynamicRHI(RequestedFeatureLevel);
	
#if ENABLE_RHI_VALIDATION
	if (FParse::Param(FCommandLine::Get(), TEXT("RHIValidation")))
	{
		FinalRHI = new FValidationRHI(FinalRHI);
	}
#endif
	
	return FinalRHI;
}
