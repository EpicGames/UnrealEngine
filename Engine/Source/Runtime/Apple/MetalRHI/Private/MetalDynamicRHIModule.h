// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MetalDynamicRHIModule.h: Metal Dynamic RHI Module Class.
==============================================================================*/

#pragma once

#include "DynamicRHI.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Module Class


class FMetalDynamicRHIModule : public IDynamicRHIModule
{
public:
	virtual bool IsSupported() override final;

	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) override final;
};
