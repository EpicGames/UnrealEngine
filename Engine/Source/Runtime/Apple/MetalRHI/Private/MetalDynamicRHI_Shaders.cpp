// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalDynamicRHI_Shaders.cpp: Metal Dynamic RHI Class Shader Methods.
=============================================================================*/

#include "MetalDynamicRHI.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalRHIPrivate.h"
#include "MetalShaderTypes.h"
#include "Shaders/MetalShaderLibrary.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Serialization/StaticMemoryReader.h"
#include "Interfaces/IPluginManager.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Dynamic RHI Shader Methods


FVertexShaderRHIRef FMetalDynamicRHI::RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalVertexShader* Shader = new FMetalVertexShader(*Device, CreateShaderDesc);
	return Shader;
}

FPixelShaderRHIRef FMetalDynamicRHI::RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalPixelShader* Shader = new FMetalPixelShader(*Device, CreateShaderDesc);
	return Shader;
}

FGeometryShaderRHIRef FMetalDynamicRHI::RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    FMetalGeometryShader* Shader = new FMetalGeometryShader(*Device, CreateShaderDesc);
    return Shader;
#else
	return nullptr;
#endif
}

FComputeShaderRHIRef FMetalDynamicRHI::RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    return new FMetalComputeShader(*Device, CreateShaderDesc, MTLLibraryPtr());
}

#if PLATFORM_SUPPORTS_MESH_SHADERS
FMeshShaderRHIRef FMetalDynamicRHI::RHICreateMeshShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;

    return new FMetalMeshShader(*Device, CreateShaderDesc);
}

FAmplificationShaderRHIRef FMetalDynamicRHI::RHICreateAmplificationShader(const FRHICreateShaderDesc& CreateShaderDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;

    return new FMetalAmplificationShader(*Device, CreateShaderDesc);
}
#endif

#if METAL_RHI_RAYTRACING
FRayTracingShaderRHIRef FMetalDynamicRHI::RHICreateRayTracingShader(const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency ShaderFrequency)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	return new FMetalRayShader(*Device, CreateShaderDesc, ShaderFrequency);
}
#endif // METAL_RHI_RAYTRACING

FRHIShaderLibraryRef FMetalDynamicRHI::RHICreateShaderLibrary(EShaderPlatform Platform, FString const& FilePath, FString const& Name)
{
	FString METAL_MAP_EXTENSION(".metalmap");

    MTL_SCOPED_AUTORELEASE_POOL;
    
    FRHIShaderLibraryRef Result = nullptr;

    const FName PlatformName = FDataDrivenShaderPlatformInfo::GetName(Platform);
    const FName ShaderFormatName = LegacyShaderPlatformToShaderFormat(Platform);
    FString ShaderFormatAndPlatform = ShaderFormatName.ToString() + TEXT("-") + PlatformName.ToString();

    FString LibName = FString::Printf(TEXT("%s_%s"), *Name, *ShaderFormatAndPlatform);
    LibName.ToLowerInline();

    FString BinaryShaderFile = FilePath / LibName + METAL_MAP_EXTENSION;

    if (IFileManager::Get().FileExists(*BinaryShaderFile) == false)
    {
        // the metal map files are stored in UFS file system
        // for pak files this means they might be stored in a different location as the pak files will mount them to the project content directory
        // the metal libraries are stores non UFS and could be anywhere on the file system.
        // if we don't find the metalmap file straight away try the pak file path
        BinaryShaderFile = FPaths::ProjectContentDir() / LibName + METAL_MAP_EXTENSION;

		if (IFileManager::Get().FileExists(*BinaryShaderFile) == false)
		{
			// See if its in a Plugin
			const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(*Name);
			if(Plugin !=  nullptr)
			{
				BinaryShaderFile = FPaths::Combine(Plugin->GetContentDir(), LibName + METAL_MAP_EXTENSION);
			}
			else
			{
				// GFP might not be loaded yet
				BinaryShaderFile = FPaths::ProjectPluginsDir() / TEXT("GameFeatures") / Name / TEXT("Content") / LibName + METAL_MAP_EXTENSION;
			}
		}
    }

    FScopeLock Lock(&FMetalShaderLibrary::LoadedShaderLibraryMutex);

    FRHIShaderLibrary** FoundShaderLibrary = FMetalShaderLibrary::LoadedShaderLibraryMap.Find(BinaryShaderFile);
    if (FoundShaderLibrary)
    {
        return *FoundShaderLibrary;
    }

    auto SerializeShaderCode = [](FMetalShaderLibrary::FShaderCodeArrayType& Array, FArchive& Ar)
    {
#if !USE_MMAPPED_SHADERARCHIVE
		Ar << Array;
#else
       if (Ar.GetArchiveName() == TEXT("FStaticMemoryReader"))
       {
			using ArrayType = std::remove_cvref_t <decltype(Array)>;
			FStaticMemoryReader* MemReader = static_cast<FStaticMemoryReader*>(&Ar);

			typename ArrayType::SizeType SerializeNum;
			Ar << SerializeNum;

			uint64 ArrayBytes = SerializeNum * sizeof(typename ArrayType::ElementType);
			uint64 Offset = Ar.Tell();

			Array = FMetalShaderLibrary::FShaderCodeArrayType(MemReader->GetData() + Offset, SerializeNum);
			Ar.Seek(Offset + ArrayBytes);
       }
	   else
	   {
		   UE_LOGF(LogMetal, Fatal, "mmapped array must be serialized via FStaticMemoryReader");
	   }
#endif
	};

	auto MapShaderLibraryFile = [](const FString& FilePath, FMetalShaderLibrary::FShaderLibDataOwner& MemOwner)
	{
		if (FPlatformProperties::SupportsMemoryMappedFiles())
		{
			FOpenMappedResult Res = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(*FilePath);
			MemOwner.MappedCacheFile = Res.HasError() ? nullptr : Res.StealValue();
			if (MemOwner.MappedCacheFile.IsValid())
			{
				MemOwner.MappedRegion = TUniquePtr<IMappedFileRegion>(MemOwner.MappedCacheFile->MapRegion(0, MemOwner.MappedCacheFile->GetFileSize(), EMappedFileFlags::ENone));
			}
		}
	};

#if !USE_MMAPPED_SHADERARCHIVE
    FArchive* BinaryShaderAr = IFileManager::Get().CreateFileReader(*BinaryShaderFile);
#else
	FStaticMemoryReader* BinaryShaderAr = nullptr;
    TUniquePtr<FMetalShaderLibrary::FShaderLibDataOwner> MemOwner = MakeUnique<FMetalShaderLibrary::FShaderLibDataOwner>();
	MapShaderLibraryFile(BinaryShaderFile, *MemOwner);
	if (MemOwner->MappedRegion.IsValid())
	{
		UE_LOGF(LogMetal, Display, "mmapping %ls, %d bytes", *BinaryShaderFile, MemOwner->MappedCacheFile->GetFileSize());
		BinaryShaderAr = new FStaticMemoryReader(MemOwner->MappedRegion->GetMappedPtr(), MemOwner->MappedCacheFile->GetFileSize());
	}

	if(BinaryShaderAr == nullptr)
	{
		TArray<uint8>& FileData = MemOwner->Mem;
		if (FFileHelper::LoadFileToArray(FileData, *BinaryShaderFile))
		{
			UE_LOGF(LogMetal, Display, "emulating mmapping %ls, %d bytes!", *BinaryShaderFile, FileData.Num());
			BinaryShaderAr = new FStaticMemoryReader(FileData.GetData(), FileData.Num());
		}
	}
#endif
    if( BinaryShaderAr != NULL )
    {
        FMetalShaderLibraryHeader Header;
        FSerializedShaderArchive SerializedShaders;
        FMetalShaderLibrary::FShaderCodeArrayType ShaderCode;

        *BinaryShaderAr << Header;
        *BinaryShaderAr << SerializedShaders;
		SerializeShaderCode(ShaderCode,*BinaryShaderAr);
        BinaryShaderAr->Flush();
        delete BinaryShaderAr;

        // Would be good to check the language version of the library with the archive format here.
        if (Header.Format == ShaderFormatName.GetPlainNameString())
        {
			TArray<TUniquePtr<FMetalShaderLibrary::FLazyMetalLib>> LazyLibraries;
			LazyLibraries.Empty(Header.NumLibraries);

            for (uint32 i = 0; i < Header.NumLibraries; i++)
			{
				FString MetalLibraryFilePath = (FilePath / LibName) + FString::Printf(TEXT(".%d.metallib"), i);

				TUniquePtr<FMetalShaderLibrary::FShaderLibDataOwner> Data = MakeUnique<FMetalShaderLibrary::FShaderLibDataOwner>();
#if USE_MMAPPED_SHADERARCHIVE
				MapShaderLibraryFile(MetalLibraryFilePath, *Data);
#endif				

				TUniquePtr<FMetalShaderLibrary::FLazyMetalLib> LazyLibrary = MakeUnique<FMetalShaderLibrary::FLazyMetalLib>(MetalLibraryFilePath, MoveTemp(Data));
				LazyLibraries.Add(MoveTemp(LazyLibrary));
			}

			FMetalShaderLibrary* MtlLib = new FMetalShaderLibrary(*Device, Platform, Name, BinaryShaderFile, Header, MoveTemp(SerializedShaders), MoveTemp(ShaderCode), MoveTemp(LazyLibraries)
#if USE_MMAPPED_SHADERARCHIVE
				, MoveTemp(MemOwner)
#endif
			);
            Result = MtlLib;
            FMetalShaderLibrary::LoadedShaderLibraryMap.Add(BinaryShaderFile, Result.GetReference());
        }
		else
		{
		    UE_LOGF(LogMetal, Display, "Unknown shader format for %ls!", *LibName);
		}
    }
    else
    {
        UE_LOGF(LogMetal, Display, "No .metalmap file found for %ls!", *LibName);
    }

    return Result;
}

FBoundShaderStateRHIRef FMetalDynamicRHI::RHICreateBoundShaderState(
	FRHIVertexDeclaration* VertexDeclarationRHI,
	FRHIVertexShader* VertexShaderRHI,
	FRHIPixelShader* PixelShaderRHI,
	FRHIGeometryShader* GeometryShaderRHI)
{
	NOT_SUPPORTED("RHICreateBoundShaderState");
	return nullptr;
}
