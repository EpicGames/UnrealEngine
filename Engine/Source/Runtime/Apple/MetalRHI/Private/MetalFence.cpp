// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalFence.h"
#include "MetalRHIPrivate.h"
#include "MetalCommandBuffer.h"
#include "MetalCommandQueue.h"
#include "MetalProfiler.h"

uint32 FMetalFence::Release() const
{
	uint32 Refs = uint32(FPlatformAtomics::InterlockedDecrement(&NumRefs));
	if(Refs == 0)
	{
        FMetalFencePool::Get().ReleaseFence(const_cast<FMetalFence*>(this));
	}
	return Refs;
}

void FMetalFencePool::Initialise(MTL::Device* InDevice)
{
	Device = InDevice;
	for (int32 i = 0; i < FMetalFencePool::NumFences; i++)
	{
        FMetalFence* F = new FMetalFence;
        F->Set(Device->newFence());
        Lifo.Push(F);
	}
	Count = FMetalFencePool::NumFences;
    Allocated = 0;
}

FMetalFence* FMetalFencePool::AllocateFence()
{
	FMetalFence* Fence = Lifo.Pop();
	if (Fence)
	{
        INC_DWORD_STAT(STAT_MetalFenceCount);
        FPlatformAtomics::InterlockedDecrement(&Count);
        FPlatformAtomics::InterlockedIncrement(&Allocated);
	}
	check(Fence);
	Fence->Reset();
	return Fence;
}

void FMetalFencePool::ReleaseFence(FMetalFence* const InFence)
{
	if (InFence)
	{
        DEC_DWORD_STAT(STAT_MetalFenceCount);
        FPlatformAtomics::InterlockedDecrement(&Allocated);
		FPlatformAtomics::InterlockedIncrement(&Count);
		check(Count <= FMetalFencePool::NumFences);
		Lifo.Push(InFence);
	}
}
