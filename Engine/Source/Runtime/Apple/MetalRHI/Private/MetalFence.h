// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "MetalRHIPrivate.h"
#include "Containers/LockFreeList.h"

#pragma clang diagnostic ignored "-Wnullability-completeness"

class FMetalFence
{
public:
	FMetalFence()
	: NumRefs(0)
	{
		Reset();
	}
	
	explicit FMetalFence(FMetalFence const& Other)
	: NumRefs(0)
	{
		operator=(Other);
	}
	
	~FMetalFence()
	{
		check(!NumRefs);
	}
	
	uint32 AddRef() const
	{
		return uint32(FPlatformAtomics::InterlockedIncrement(&NumRefs));
	}
	uint32 Release() const;
	uint32 GetRefCount() const
	{
		return uint32(FPlatformAtomics::AtomicRead(&NumRefs));
	}
	
	FMetalFence& operator=(FMetalFence const& Other)
	{
		if (&Other != this)
		{
			Fence = Other.Fence;
		}
		return *this;
	}
	
	void Reset(void)
	{
		WriteNum = 0;
		WaitNum = 0;
	}
	
	void Write()
	{
		WriteNum++;
	}
	
	void Wait()
	{
		WaitNum++;
	}
	
	int8 NumWrite() const
	{
		return WriteNum;
	}
	
	int8 NumWait() const
	{
		return WaitNum;
	}
	
    MTL::Fence* Get() const
	{
		return Fence;
	}
	
	void Set(MTL::Fence* InFence)
	{
		Fence = InFence;
	}
	
private:
    MTL::Fence* Fence;
	int8 WriteNum;
	int8 WaitNum;
	mutable int32 NumRefs;	
};

class FMetalFencePool
{
	enum
	{
		NumFences = 2048
	};
public:
	FMetalFencePool() {}
	
	static FMetalFencePool& Get()
	{
		static FMetalFencePool sSelf;
		return sSelf;
	}
	
	void Initialise(MTL::Device* InDevice);
	
	FMetalFence* AllocateFence();
	void ReleaseFence(FMetalFence* InFence);
	
	int32 Max() const { return Count; }
	int32 Num() const { return Allocated; }
	
private:
	int32 Count;
	int32 Allocated;
	MTL::Device* Device;
#if METAL_DEBUG_OPTIONS
	TSet<FMetalFence*> Fences;
	FCriticalSection Mutex;
#endif
	TLockFreePointerListFIFO<FMetalFence, PLATFORM_CACHE_LINE_SIZE> Lifo;
};
