// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalGraphicsPipelineState.cpp: Metal RHI graphics pipeline state class.
=============================================================================*/

#include "MetalGraphicsPipelineState.h"

#include "MetalRHIPrivate.h"
#include "MetalVertexDeclaration.h"
#include "MetalShaderTypes.h"


//------------------------------------------------------------------------------

#pragma mark - Metal Graphics Pipeline State Support Routines


// From MetalPipeline.cpp:
extern FMetalShaderPipelinePtr GetMTLRenderPipeline(bool const bSync, FMetalGraphicsPipelineState const* State, const FGraphicsPipelineStateInitializer& Init);

extern void ReleaseMTLRenderPipeline(FMetalShaderPipelinePtr Pipeline);

//------------------------------------------------------------------------------

#pragma mark - Metal Graphics Pipeline State Class Implementation


FMetalGraphicsPipelineState::FMetalGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Init)
	: Initializer(Init)
	, PipelineState(nullptr)
{
	// void
}

bool FMetalGraphicsPipelineState::Compile()
{
	check(PipelineState == nullptr);
	PipelineState = GetMTLRenderPipeline(true, this, Initializer);
	return (PipelineState != nullptr);
}

FMetalGraphicsPipelineState::~FMetalGraphicsPipelineState()
{
	if (PipelineState != nullptr)
	{
		ReleaseMTLRenderPipeline(PipelineState);
		PipelineState = nullptr;
	}
}

FMetalShaderPipelinePtr FMetalGraphicsPipelineState::GetPipeline()
{
	if (PipelineState == nullptr)
	{
		PipelineState = GetMTLRenderPipeline(true, this, Initializer);
		check(PipelineState != nullptr);
	}

    return PipelineState;
}
