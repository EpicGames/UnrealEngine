// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalGraphicsPipelineState.h: Metal RHI graphics pipeline state class.
=============================================================================*/

#pragma once

#include "RHIResources.h"
#include "MetalState.h"
#include "Shaders/Types/MetalAmplificationShader.h"
#include "Shaders/Types/MetalGeometryShader.h"
#include "Shaders/Types/MetalMeshShader.h"
#include "Shaders/Types/MetalPixelShader.h"
#include "Shaders/Types/MetalVertexShader.h"

class FMetalGraphicsPipelineState : public FRHIGraphicsPipelineState
{
	friend class FMetalDynamicRHI;

public:
	virtual ~FMetalGraphicsPipelineState();

	FRHIGraphicsShader* GetShader(EShaderFrequency Frequency) const override
	{
		switch (Frequency)
		{
		case SF_Vertex: return VertexShader;
		case SF_Pixel: return PixelShader;

		case SF_Geometry:
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			return GeometryShader;
#else
			return nullptr;
#endif

		case SF_Mesh:
#if PLATFORM_SUPPORTS_MESH_SHADERS
			return MeshShader;
#else
			return nullptr;
#endif

		case SF_Amplification:
#if PLATFORM_SUPPORTS_MESH_SHADERS
			return AmplificationShader;
#else
			return nullptr;
#endif

		default: return nullptr;
		}
	}

	FMetalShaderPipelinePtr GetPipeline();

	/** Cached vertex structure */
	TRefCountPtr<FMetalVertexDeclaration> VertexDeclaration;

	/** Cached shaders */
	TRefCountPtr<FMetalVertexShader> VertexShader;
	TRefCountPtr<FMetalPixelShader> PixelShader;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	TRefCountPtr<FMetalGeometryShader> GeometryShader;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
    TRefCountPtr<FMetalMeshShader>          MeshShader;
    TRefCountPtr<FMetalAmplificationShader> AmplificationShader;
#endif

	/** Cached state objects */
	TRefCountPtr<FMetalDepthStencilState> DepthStencilState;
	TRefCountPtr<FMetalRasterizerState> RasterizerState;

	EPrimitiveType GetPrimitiveType()
	{
		return Initializer.PrimitiveType;
	}

	bool GetDepthBounds() const
	{
		return Initializer.bDepthBounds;
	}

#if METAL_USE_METAL_SHADER_CONVERTER
    TArray<uint8_t> StageInFunctionBytecode;
#endif

private:
	// This can only be created through the RHI to make sure Compile() is called.
	FMetalGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Init);

	// Compiles the underlying gpu pipeline objects. This must be called before usage.
	bool Compile();

	// Needed to runtime refine shaders currently.
	FGraphicsPipelineStateInitializer Initializer;

	FMetalShaderPipelinePtr PipelineState;
};
