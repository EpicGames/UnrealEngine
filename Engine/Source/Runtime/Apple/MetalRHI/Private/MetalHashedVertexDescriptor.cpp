// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalHashedVertexDescriptor.cpp: Metal RHI Hashed Vertex Descriptor.
=============================================================================*/

#include "MetalHashedVertexDescriptor.h"

#include "MetalRHIPrivate.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Hashed Vertex Descriptor


FMetalHashedVertexDescriptor::FMetalHashedVertexDescriptor()
	: VertexDescHash(0)
{
	// void
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
FMetalHashedVertexDescriptor::FMetalHashedVertexDescriptor(IRVersionedInputLayoutDescriptor& Desc, uint32 Hash)
	: VertexDescHash(Hash)
	, IRVertexDesc(Desc)
	, bUsesIRVertexDesc(true)	
{
	// void
}
#endif

FMetalHashedVertexDescriptor::FMetalHashedVertexDescriptor(MTLVertexDescriptorPtr Desc, uint32 Hash)
	: VertexDescHash(Hash)
	, VertexDesc(Desc)
{
	// void
}

FMetalHashedVertexDescriptor::FMetalHashedVertexDescriptor(FMetalHashedVertexDescriptor const& Other)
	: VertexDescHash(0)
{
	operator=(Other);
}

FMetalHashedVertexDescriptor::~FMetalHashedVertexDescriptor()
{
	// void
}

FMetalHashedVertexDescriptor& FMetalHashedVertexDescriptor::operator=(FMetalHashedVertexDescriptor const& Other)
{
	if (this != &Other)
	{
		VertexDescHash = Other.VertexDescHash;
		VertexDesc = Other.VertexDesc;
#if METAL_USE_METAL_SHADER_CONVERTER
		IRVertexDesc = Other.IRVertexDesc;
#endif
	}
	return *this;
}

bool FMetalHashedVertexDescriptor::operator==(FMetalHashedVertexDescriptor const& Other) const
{
	bool bEqual = false;

	if (this != &Other)
	{
		if (VertexDescHash == Other.VertexDescHash)
		{
			bEqual = true;
			
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if(bUsesIRVertexDesc)
			{
				bEqual &= (IRVertexDesc.desc_1_0.numElements == Other.IRVertexDesc.desc_1_0.numElements);
				
				if (bEqual)
				{
					for (uint32 ElementIdx = 0; ElementIdx < IRVertexDesc.desc_1_0.numElements; ElementIdx++)
					{
						bEqual &= (FCStringAnsi::Strcmp(IRVertexDesc.desc_1_0.semanticNames[ElementIdx], Other.IRVertexDesc.desc_1_0.semanticNames[ElementIdx]) == 0);
						bEqual &= (IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].format == Other.IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].format);
						bEqual &= (IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].alignedByteOffset == Other.IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].alignedByteOffset);
						bEqual &= (IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].inputSlot == Other.IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].inputSlot);
						bEqual &= (IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].inputSlotClass == Other.IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].inputSlotClass);
						bEqual &= (IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].instanceDataStepRate == Other.IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].instanceDataStepRate);
						bEqual &= (IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].semanticIndex == Other.IRVertexDesc.desc_1_0.inputElementDescs[ElementIdx].semanticIndex);
					}
				}
			}
			else
#endif
			if (VertexDesc != Other.VertexDesc)
			{
                MTL::VertexBufferLayoutDescriptorArray* Layouts = VertexDesc->layouts();
				MTL::VertexAttributeDescriptorArray* Attributes = VertexDesc->attributes();

				MTL::VertexBufferLayoutDescriptorArray* OtherLayouts = Other.VertexDesc->layouts();
				MTL::VertexAttributeDescriptorArray* OtherAttributes = Other.VertexDesc->attributes();
				check(Layouts && Attributes && OtherLayouts && OtherAttributes);

				for (uint32 i = 0; bEqual && i < MaxVertexElementCount; i++)
				{
					MTL::VertexBufferLayoutDescriptor* LayoutDesc = Layouts->object(i);
                    MTL::VertexBufferLayoutDescriptor* OtherLayoutDesc = OtherLayouts->object(i);

					bEqual &= ((LayoutDesc != nullptr) == (OtherLayoutDesc != nullptr));

					if (LayoutDesc && OtherLayoutDesc)
					{
						bEqual &= (LayoutDesc->stride() == OtherLayoutDesc->stride());
						bEqual &= (LayoutDesc->stepFunction() == OtherLayoutDesc->stepFunction());
						bEqual &= (LayoutDesc->stepRate() == OtherLayoutDesc->stepRate());
					}

					MTL::VertexAttributeDescriptor* AttrDesc = Attributes->object(i);
                    MTL::VertexAttributeDescriptor* OtherAttrDesc = OtherAttributes->object(i);

					bEqual &= ((AttrDesc != nullptr) == (OtherAttrDesc != nullptr));

					if (AttrDesc && OtherAttrDesc)
					{
						bEqual &= (AttrDesc->format() == OtherAttrDesc->format());
						bEqual &= (AttrDesc->offset() == OtherAttrDesc->offset());
						bEqual &= (AttrDesc->bufferIndex() == OtherAttrDesc->bufferIndex());
					}
				}
			}
		}
	}
	else
	{
		bEqual = true;
	}

	return bEqual;
}
