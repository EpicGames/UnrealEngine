// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalHashedVertexDescriptor.h: Metal RHI Hashed Vertex Descriptor.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Hashed Vertex Descriptor


/**
 * The MTLVertexDescriptor and a pre-calculated hash value used to simplify
 * comparisons (as vendor MTLVertexDescriptor implementations are not all
 * comparable).
 */
struct FMetalHashedVertexDescriptor
{
	NS::UInteger VertexDescHash;
	MTLVertexDescriptorPtr VertexDesc;
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FMetalHashedVertexDescriptor(IRVersionedInputLayoutDescriptor& Desc, uint32 Hash);
	IRVersionedInputLayoutDescriptor IRVertexDesc;
	bool bUsesIRVertexDesc = false;
#endif
	
	FMetalHashedVertexDescriptor();
	
	FMetalHashedVertexDescriptor(MTLVertexDescriptorPtr Desc, uint32 Hash);
	FMetalHashedVertexDescriptor(FMetalHashedVertexDescriptor const& Other);
	~FMetalHashedVertexDescriptor();

	FMetalHashedVertexDescriptor& operator=(FMetalHashedVertexDescriptor const& Other);
	bool operator==(FMetalHashedVertexDescriptor const& Other) const;

	friend uint32 GetTypeHash(FMetalHashedVertexDescriptor const& Hash)
	{
		return Hash.VertexDescHash;
	}
};
