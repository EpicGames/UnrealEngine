// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalLLM.h"
#include "MetalCommandQueue.h"
#include "MetalProfiler.h"
#include "MetalResources.h"
#include "MetalDevice.h"
#include "RenderUtils.h"
#include "HAL/LowLevelMemStats.h"

#include <objc/runtime.h>

#if ENABLE_LOW_LEVEL_MEM_TRACKER

struct FLLMTagInfoMetal
{
	const TCHAR* Name;
	FName StatName;				// shows in the LLMFULL stat group
	FName SummaryStatName;		// shows in the LLM summary stat group
};

DECLARE_LLM_MEMORY_STAT(TEXT("Metal Buffers"), STAT_MetalBuffersLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Metal Textures"), STAT_MetalTexturesLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Metal Heaps"), STAT_MetalHeapsLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Metal RenderTargets"), STAT_MetalRenderTargetsLLM, STATGROUP_LLMPlatform);
DECLARE_LLM_MEMORY_STAT(TEXT("Metal AccelerationStructures"), STAT_MetalAccelerationStructuresLLM, STATGROUP_LLMPlatform);

// *** order must match ELLMTagMetal enum ***
const FLLMTagInfoMetal ELLMTagNamesMetal[] =
{
	// csv name									// stat name										// summary stat name						// enum value
	{ TEXT("Metal Buffers"),					GET_STATFNAME(STAT_MetalBuffersLLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagMetal::Buffers
	{ TEXT("Metal Textures"),					GET_STATFNAME(STAT_MetalTexturesLLM),				GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagMetal::Textures
	{ TEXT("Metal Heaps"),						GET_STATFNAME(STAT_MetalHeapsLLM),					GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagMetal::Heaps
	{ TEXT("Metal Render Targets"),				GET_STATFNAME(STAT_MetalRenderTargetsLLM),			GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagMetal::RenderTargets
	{ TEXT("Metal Acceleration Structures"),	GET_STATFNAME(STAT_MetalAccelerationStructuresLLM),	GET_STATFNAME(STAT_EngineSummaryLLM) },		// ELLMTagMetal::AccelerationStructures	
};

/*
 * Register Metal tags with LLM
 */
void MetalLLM::Initialise()
{
	int32 TagCount = sizeof(ELLMTagNamesMetal) / sizeof(FLLMTagInfoMetal);

	for (int32 Index = 0; Index < TagCount; ++Index)
	{
		int32 Tag = (int32)ELLMTagApple::AppleMetalTagsStart + Index;
		const FLLMTagInfoMetal& TagInfo = ELLMTagNamesMetal[Index];

		FLowLevelMemTracker::Get().RegisterPlatformTag(Tag, TagInfo.Name, TagInfo.StatName, TagInfo.SummaryStatName);
	}
}

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER

@implementation FMetalDeallocHandler

APPLE_PLATFORM_OBJECT_ALLOC_OVERRIDES(FMetalDeallocHandler)

-(instancetype)initWithBlock:(dispatch_block_t)InBlock
{
	id Self = [super init];
	if (Self)
	{
		self->Block = Block_copy(InBlock);
	}
	return Self;
}
-(void)dealloc
{
	self->Block();
	Block_release(self->Block);
	[super dealloc];
}
@end

static MTL::PixelFormat FromSRGBFormat(MTL::PixelFormat Format)
{
    MTL::PixelFormat MTLFormat = Format;
	
	switch (Format)
	{
		case MTL::PixelFormatRGBA8Unorm_sRGB:
			MTLFormat = MTL::PixelFormatRGBA8Unorm;
			break;
		case MTL::PixelFormatBGRA8Unorm_sRGB:
			MTLFormat = MTL::PixelFormatBGRA8Unorm;
			break;
#if PLATFORM_MAC
		case MTL::PixelFormatBC1_RGBA_sRGB:
			MTLFormat = MTL::PixelFormatBC1_RGBA;
			break;
		case MTL::PixelFormatBC2_RGBA_sRGB:
			MTLFormat = MTL::PixelFormatBC2_RGBA;
			break;
		case MTL::PixelFormatBC3_RGBA_sRGB:
			MTLFormat = MTL::PixelFormatBC3_RGBA;
			break;
		case MTL::PixelFormatBC7_RGBAUnorm_sRGB:
			MTLFormat = MTL::PixelFormatBC7_RGBAUnorm;
			break;
#endif //PLATFORM_MAC
#if PLATFORM_IOS
		case MTL::PixelFormatR8Unorm_sRGB:
			MTLFormat = MTL::PixelFormatR8Unorm;
			break;
		case MTL::PixelFormatPVRTC_RGBA_2BPP_sRGB:
			MTLFormat = MTL::PixelFormatPVRTC_RGBA_2BPP;
			break;
		case MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB:
			MTLFormat = MTL::PixelFormatPVRTC_RGBA_4BPP;
			break;
		case MTL::PixelFormatASTC_4x4_sRGB:
			MTLFormat = MTL::PixelFormatASTC_4x4_LDR;
			break;
		case MTL::PixelFormatASTC_6x6_sRGB:
			MTLFormat = MTL::PixelFormatASTC_6x6_LDR;
			break;
		case MTL::PixelFormatASTC_8x8_sRGB:
			MTLFormat = MTL::PixelFormatASTC_8x8_LDR;
			break;
		case MTL::PixelFormatASTC_10x10_sRGB:
			MTLFormat = MTL::PixelFormatASTC_10x10_LDR;
			break;
		case MTL::PixelFormatASTC_12x12_sRGB:
			MTLFormat = MTL::PixelFormatASTC_12x12_LDR;
			break;
#endif //PLATFORM_IOS
		default:
			break;
	}
	
	return MTLFormat;
}

static EPixelFormat MetalToRHIPixelFormat(MTL::PixelFormat Format)
{
	Format = FromSRGBFormat(Format);
	for (uint32 i = 0; i < PF_MAX; i++)
	{
		if((MTL::PixelFormat)GPixelFormats[i].PlatformFormat == Format)
		{
			return (EPixelFormat)i;
		}
	}
	check(false);
	return PF_MAX;
}

void MetalLLM::LogAllocTexture(MTL::Texture* Texture)
{
	uint64 Size = Texture->allocatedSize();
	void* Ptr = (void*)Texture;
	
#if PLATFORM_IOS
	bool bMemoryless = (Texture->storageMode() == MTL::StorageModeMemoryless);
	if (!bMemoryless)
#endif
	{
		INC_MEMORY_STAT_BY(STAT_MetalTextureMemory, Size);
	}
	INC_DWORD_STAT(STAT_MetalTextureCount);
	
	if (Texture->usage() & MTL::TextureUsageRenderTarget)
	{
		LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::RenderTargets);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size, ELLMTag::Untagged, ELLMAllocType::System));
	}
	else
	{
		LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::Textures);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size, ELLMTag::Untagged, ELLMAllocType::System));
	}
}

void MetalLLM::LogFreeTexture(MTL::Texture* Texture)
{
	uint64 Size = Texture->allocatedSize();
	void* Ptr = (void*)Texture;
	
#if PLATFORM_IOS
	bool bMemoryless = (Texture->storageMode() == MTL::StorageModeMemoryless);
	if (!bMemoryless)
#endif
	{
		DEC_MEMORY_STAT_BY(STAT_MetalTextureMemory, Size);
	}
	DEC_DWORD_STAT(STAT_MetalTextureCount);
	
	if (Texture->usage() & MTL::TextureUsageRenderTarget)
	{
		LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::RenderTargets);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr, ELLMAllocType::System));
	}
	else
	{
		LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::Textures);
		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr, ELLMAllocType::System));
	}
}

void MetalLLM::LogAllocBufferNative(MTL::Buffer* Buffer)
{
    void* Ptr = (void*)Buffer;
    uint64 Size = Buffer->length();

	LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::Buffers);
    INC_MEMORY_STAT_BY(STAT_MetalBufferMemory, Size);
    INC_DWORD_STAT(STAT_MetalBufferCount);
    
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size, ELLMTag::Untagged, ELLMAllocType::System));
}

void MetalLLM::LogFreeBufferNative(MTL::Buffer* Buffer)
{
	void* Ptr = (void*)Buffer;
	uint64 Size = Buffer->length();
	
	LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::Buffers);
	DEC_MEMORY_STAT_BY(STAT_MetalBufferMemory, Size);
	DEC_DWORD_STAT(STAT_MetalBufferCount);
	
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr, ELLMAllocType::System));
}

void MetalLLM::LogAllocHeap(MTL::Heap* Heap)
{
	void* Ptr = (void*)Heap;
	uint64 Size = Heap->size();
	
	INC_MEMORY_STAT_BY(STAT_MetalHeapMemory, Size);
	INC_DWORD_STAT(STAT_MetalHeapCount);
	
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Platform, Ptr, Size, ELLMTag::Untagged, ELLMAllocType::System));
	// Assign a dealloc handler to untrack the memory - but don't track the dispatch block!
	{
		LLM_SCOPED_PAUSE_TRACKING(ELLMAllocType::System);
		
		objc_setAssociatedObject((__bridge id<MTLHeap>)Heap, (void*)&MetalLLM::LogAllocHeap,
								 [[[FMetalDeallocHandler alloc] initWithBlock:^{
			LLM_SCOPE_METAL(ELLMTagMetal::Heaps);
			LLM_PLATFORM_SCOPE_METAL(ELLMTagMetal::Heaps);
			
			LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Platform, Ptr, ELLMAllocType::System));
			
			DEC_MEMORY_STAT_BY(STAT_MetalHeapMemory, Size);
			DEC_DWORD_STAT(STAT_MetalHeapCount);
		}] autorelease],
		OBJC_ASSOCIATION_RETAIN);
	}
}

void MetalLLM::LogAliasTexture(MTL::Texture* Texture)
{
	objc_setAssociatedObject((__bridge id<MTLTexture>)Texture, (void*)&MetalLLM::LogAllocTexture, nullptr, OBJC_ASSOCIATION_RETAIN);
}

void MetalBufferStats::UpdateBufferStats(const FRHIBufferDesc& BufferDesc, int64 BufferSize, bool bAllocating)
{
	UE::RHICore::UpdateGlobalBufferStats(BufferDesc, BufferSize, bAllocating);
}

void MetalBufferStats::UpdateUniformBufferStats(int64 BufferSize, bool bAllocating)
{
	UE::RHICore::UpdateGlobalUniformBufferStats(BufferSize, bAllocating);
}
