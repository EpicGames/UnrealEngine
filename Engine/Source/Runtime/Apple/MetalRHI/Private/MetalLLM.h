// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"
#include "HAL/LowLevelMemTracker.h"
#include "Apple/AppleLLM.h"
#include "Apple/AppleMemoryObject.h"
#include "RHICoreStats.h"

class FMetalBuffer;
typedef TSharedPtr<FMetalBuffer, ESPMode::ThreadSafe> FMetalBufferPtr;

@interface FMetalDeallocHandler : FApplePlatformObject<NSObject>
{
	dispatch_block_t Block;
}
-(instancetype)initWithBlock:(dispatch_block_t)InBlock;
-(void)dealloc;
@end

#if ENABLE_LOW_LEVEL_MEM_TRACKER

#define LLM_SCOPE_METAL(Tag) LLM_SCOPE((ELLMTag)Tag)
#define LLM_PLATFORM_SCOPE_METAL(Tag) LLM_PLATFORM_SCOPE((ELLMTag)Tag)

enum class ELLMTagMetal : LLM_TAG_TYPE
{
	Buffers = (LLM_TAG_TYPE)ELLMTagApple::AppleMetalTagsStart,
	Textures,
	Heaps,
	RenderTargets,
	AccelerationStructures,
	
	Count
};

static_assert((int32)ELLMTagMetal::Count <= (int32)ELLMTagApple::AppleMetalTagsEnd, "too many ELLMTagMetal tags. Need to increase LLM_TAG_APPLE_NUM_METAL_TAGS_RESERVED");

namespace MetalLLM
{
	void Initialise();
}

#else

#define LLM_SCOPE_METAL(...)
#define LLM_PLATFORM_SCOPE_METAL(...)

#endif		// #if ENABLE_LOW_LEVEL_MEM_TRACKER

class FMetalDevice;
// These work without the LLM module
namespace MetalLLM
{
	void LogAllocTexture(MTL::Texture* Texture);
	void LogFreeTexture(MTL::Texture* Texture);
    void LogAllocBufferNative(MTL::Buffer* Buffer);
	void LogFreeBufferNative(MTL::Buffer* Buffer);
	void LogAllocHeap(MTL::Heap* Heap);
	
	void LogAliasTexture(MTL::Texture* Texture);
}

namespace MetalBufferStats
{
	void UpdateBufferStats(const FRHIBufferDesc& BufferDesc, int64 BufferSize, bool bAllocating);
	void UpdateUniformBufferStats(int64 BufferSize, bool bAllocating);
}
