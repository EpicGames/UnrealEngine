// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalPipeline.cpp: Metal shader pipeline RHI implementation.
=============================================================================*/

#include "MetalPipeline.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "MetalRHIPrivate.h"
#include "MetalVertexDeclaration.h"
#include "MetalShaderTypes.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalPipeline.h"
#include "MetalShaderResources.h"
#include "MetalProfiler.h"
#include "MetalCommandQueue.h"
#include "MetalCommandBuffer.h"
#include "RenderUtils.h"
#include "Misc/ScopeRWLock.h"
#include "HAL/PThreadEvent.h"
#include "PSOMetrics.h"
#include <objc/runtime.h>

static int32 GMetalCacheShaderPipelines = 1;
static FAutoConsoleVariableRef CVarMetalCacheShaderPipelines(
	TEXT("rhi.Metal.CacheShaderPipelines"),
	GMetalCacheShaderPipelines,
	TEXT("When enabled (1, default) cache all graphics pipeline state objects created in MetalRHI for the life of the program, this trades memory for performance as creating PSOs is expensive in Metal.\n")
	TEXT("When set to 2 it will not cache any graphics pipeline state objects\n")
	TEXT("Disable in the project configuration to allow PSOs to be released to save memory at the expense of reduced performance and increased hitching in-game\n. (On by default (1))"), ECVF_ReadOnly);

static int32 GMetalCacheMinSize = 32;
static FAutoConsoleVariableRef CVarMetalCacheMinSize(
	TEXT("r.ShaderPipelineCache.MetalCacheMinSizeInMB"),
	GMetalCacheMinSize,
	TEXT("Sets the minimum size that we expect the metal OS cache to be (in MB). This is used to determine if we need to cache PSOs again (Default: 32).\n"), ECVF_ReadOnly);

static int32 GMetalBinaryCacheDebugOutput = 0;
static FAutoConsoleVariableRef CVarMetalBinaryCacheDebugOutput(
    TEXT("rhi.Metal.BinaryCacheDebugOutput"),
    GMetalBinaryCacheDebugOutput,
    TEXT("Enable to output logging information for PSO Binary cache default(0) \n"), ECVF_ReadOnly);

extern int32 GMetalShaderValidationType;
extern FString GMetalShaderValidationShaderName;

static uint32 BlendBitOffsets[] = { Offset_BlendState0, Offset_BlendState1, Offset_BlendState2, Offset_BlendState3, Offset_BlendState4, Offset_BlendState5, Offset_BlendState6, Offset_BlendState7 };
static uint32 RTBitOffsets[] = { Offset_RenderTargetFormat0, Offset_RenderTargetFormat1, Offset_RenderTargetFormat2, Offset_RenderTargetFormat3, Offset_RenderTargetFormat4, Offset_RenderTargetFormat5, Offset_RenderTargetFormat6, Offset_RenderTargetFormat7 };
static_assert(Offset_BlendStateEnd < 64 && Offset_End < 128, "Offset_BlendStateEnd must be <= 64 && Offset_End < 128");

static float RoundUpNearestEven(const float f)
{
	const float ret = FMath::CeilToFloat(f);
	const float isOdd = (float)(((int)ret) & 1);
	return ret + isOdd;
}

struct FMetalGraphicsPipelineKey
{
	FMetalRenderPipelineHash RenderPipelineHash;
	FMetalHashedVertexDescriptor VertexDescriptorHash;
	FShaderHash VertexFunction;
#if PLATFORM_SUPPORTS_MESH_SHADERS
	FShaderHash MeshFunction;
	FShaderHash AmplificationFunction;
#endif
	FShaderHash PixelFunction;

	template<typename Type>
	inline void SetHashValue(uint32 Offset, uint32 NumBits, Type Value)
	{
		if (Offset < Offset_BlendStateEnd)
		{
			uint64 BitMask = ((((uint64)1ULL) << NumBits) - 1) << Offset;
			RenderPipelineHash.RasterBits = (RenderPipelineHash.RasterBits & ~BitMask) | (((uint64)Value << Offset) & BitMask);
		}
		else
		{
			Offset -= Offset_RenderTargetFormat0;
			uint64 BitMask = ((((uint64)1ULL) << NumBits) - 1) << Offset;
			RenderPipelineHash.TargetBits = (RenderPipelineHash.TargetBits & ~BitMask) | (((uint64)Value << Offset) & BitMask);
		}
	}

	bool operator==(FMetalGraphicsPipelineKey const& Other) const
	{
		return (RenderPipelineHash == Other.RenderPipelineHash
		&& VertexDescriptorHash == Other.VertexDescriptorHash
		&& VertexFunction == Other.VertexFunction
#if PLATFORM_SUPPORTS_MESH_SHADERS
        && MeshFunction == Other.MeshFunction
        && AmplificationFunction == Other.AmplificationFunction
#endif
		&& PixelFunction == Other.PixelFunction);
	}
	
	friend uint32 GetTypeHash(FMetalGraphicsPipelineKey const& Key)
	{
		uint32 H = FCrc::MemCrc32(&Key.RenderPipelineHash, sizeof(Key.RenderPipelineHash), GetTypeHash(Key.VertexDescriptorHash));
		H = FCrc::MemCrc32(&Key.VertexFunction.Hash, sizeof(Key.VertexFunction.Hash), H);
#if PLATFORM_SUPPORTS_MESH_SHADERS
        H = FCrc::MemCrc32(&Key.MeshFunction.Hash, sizeof(Key.MeshFunction.Hash), H);
        H = FCrc::MemCrc32(&Key.AmplificationFunction.Hash, sizeof(Key.AmplificationFunction.Hash), H);
#endif
		H = FCrc::MemCrc32(&Key.PixelFunction.Hash, sizeof(Key.PixelFunction.Hash), H);
		return H;
	}
	
	friend void InitMetalGraphicsPipelineKey(FMetalDevice& Device, FMetalGraphicsPipelineKey& Key,
											 const FGraphicsPipelineStateInitializer& Init)
	{
		uint32 const NumActiveTargets = Init.ComputeNumValidRenderTargets();
		check(NumActiveTargets <= MaxSimultaneousRenderTargets);
	
		FMetalBlendState* BlendState = (FMetalBlendState*)Init.BlendState;
		
		FMemory::Memzero(Key.RenderPipelineHash);
		
		bool bHasActiveTargets = false;
		for (uint32 i = 0; i < NumActiveTargets; i++)
		{
			EPixelFormat TargetFormat = (EPixelFormat)Init.RenderTargetFormats[i];

			if (TargetFormat == PF_Unknown)
				continue;

			MTL::PixelFormat MetalFormat = UEToMetalFormat(Device, TargetFormat, EnumHasAnyFlags(Init.RenderTargetFlags[i], TexCreate_SRGB));
			
			uint8 FormatKey = GetMetalPixelFormatKey(MetalFormat);
			Key.SetHashValue(RTBitOffsets[i], NumBits_RenderTargetFormat, FormatKey);
			Key.SetHashValue(BlendBitOffsets[i], NumBits_BlendState, BlendState->RenderTargetStates[i].BlendStateKey);
			
			bHasActiveTargets |= true;
		}
		
		uint8 DepthFormatKey = 0;
		switch(Init.DepthStencilTargetFormat)
		{
			case PF_DepthStencil:
			{
				MTL::PixelFormat MetalFormat = (MTL::PixelFormat)GPixelFormats[PF_DepthStencil].PlatformFormat;
				if (Init.DepthTargetLoadAction != ERenderTargetLoadAction::ENoAction || Init.DepthTargetStoreAction != ERenderTargetStoreAction::ENoAction)
				{
					DepthFormatKey = GetMetalPixelFormatKey(MetalFormat);
				}
				bHasActiveTargets |= true;
				break;
			}
			case PF_ShadowDepth:
			{
				DepthFormatKey = GetMetalPixelFormatKey((MTL::PixelFormat)GPixelFormats[PF_ShadowDepth].PlatformFormat);
				bHasActiveTargets |= true;
				break;
			}
			default:
			{
				break;
			}
		}
		
		// If the pixel shader writes depth then we must compile with depth access, so we may bind the dummy depth.
		// If the pixel shader writes to UAVs but not target is bound we must also bind the dummy depth.
		FMetalPixelShader* PixelShader = (FMetalPixelShader*)Init.BoundShaderState.PixelShaderRHI;
		if ( PixelShader && ( ( PixelShader->Bindings.InOutMask.IsFieldEnabled(CrossCompiler::FShaderBindingInOutMask::DepthStencilMaskIndex) && DepthFormatKey == 0 ) || (bHasActiveTargets == false && PixelShader->Bindings.NumUAVs > 0) ) )
		{
			MTL::PixelFormat MetalFormat = (MTL::PixelFormat)GPixelFormats[PF_DepthStencil].PlatformFormat;
			DepthFormatKey = GetMetalPixelFormatKey(MetalFormat);
		}
		
		Key.SetHashValue(Offset_DepthFormat, NumBits_DepthFormat, DepthFormatKey);

		Key.SetHashValue(Offset_SampleCount, NumBits_SampleCount, Init.NumSamples);

		Key.SetHashValue(Offset_AlphaToCoverage, NumBits_AlphaToCoverage, Init.NumSamples > 1 && BlendState->bUseAlphaToCoverage ? 1 : 0);
		
		Key.SetHashValue(Offset_PrimitiveTopology, NumBits_PrimitiveTopology, TranslatePrimitiveTopology(Init.PrimitiveType));

#if PLATFORM_SUPPORTS_MESH_SHADERS
        FMetalMeshShader* MeshShader = (FMetalMeshShader*)Init.BoundShaderState.GetMeshShader();
        FMetalGeometryShader* GeometryShader = (FMetalGeometryShader*)Init.BoundShaderState.GetGeometryShader();
        if (MeshShader)
        {
            Key.MeshFunction = MeshShader->GetHash();
            FMetalAmplificationShader* AmplificationShader = (FMetalAmplificationShader*)Init.BoundShaderState.GetAmplificationShader();
            if (AmplificationShader)
            {
                Key.AmplificationFunction = AmplificationShader->GetHash();
            }
        }
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
        else if (GeometryShader)
        {
            Key.MeshFunction = GeometryShader->GetHash();
        }
#endif
        else
#endif
        {
            FMetalVertexDeclaration* VertexDecl = (FMetalVertexDeclaration*)Init.BoundShaderState.VertexDeclarationRHI;
            Key.VertexDescriptorHash = VertexDecl->Layout;
            
            FMetalVertexShader* VertexShader = (FMetalVertexShader*)Init.BoundShaderState.VertexShaderRHI;
            Key.VertexFunction = VertexShader->GetHash();
        }

		if (PixelShader)
		{
			Key.PixelFunction = PixelShader->GetHash();
		}
	}
};

static FMetalShaderPipelinePtr CreateMTLRenderPipeline(FMetalDevice& Device, bool const bSync, FMetalGraphicsPipelineKey const& Key, const FGraphicsPipelineStateInitializer& Init, FMetalGraphicsPipelineState const* State);

class FMetalShaderPipelineCache
{
public:
	
	FMetalShaderPipelineCache(FMetalDevice& InDevice) : Device(InDevice)
	{}
	
	static void Create(FMetalDevice& Device)
	{
		Self = new FMetalShaderPipelineCache(Device);
	}
		
	static FMetalShaderPipelineCache& Get()
	{
		check(Self);
		return *Self;
	}
	
    FMetalShaderPipelinePtr GetRenderPipeline(bool const bSync, FMetalGraphicsPipelineState const* State, const FGraphicsPipelineStateInitializer& Init)
	{
		SCOPE_CYCLE_COUNTER(STAT_MetalPipelineStateTime);
		
		FMetalGraphicsPipelineKey Key;
		InitMetalGraphicsPipelineKey(Device, Key, Init);
		
		FMetalShaderPipelinePtr Desc = nullptr;
		// Don't cache anything if the cvar is set to 2
		if (GMetalCacheShaderPipelines != 2)
		{
			// By default there'll be more threads trying to read this than to write it.
			PipelineMutex.ReadLock();

			// Try to find the entry in the cache.
			Desc = Pipelines.FindRef(Key);

			PipelineMutex.ReadUnlock();

			if (Desc == nullptr)
			{
				// By default there'll be more threads trying to read this than to write it.
				EventsMutex.ReadLock();

				// Try to find a pipeline creation event for this key. If it's found, we already have a thread creating this pipeline and we just have to wait.
				TSharedPtr<FPThreadEvent, ESPMode::ThreadSafe> Event = PipelineEvents.FindRef(Key);

				EventsMutex.ReadUnlock();

				bool bCompile = false;
				if (!Event.IsValid())
				{
					// Create an event other threads can use to wait if they request the same pipeline this thread is creating
					EventsMutex.WriteLock();

					Event = PipelineEvents.FindRef(Key);
					if (!Event.IsValid())
					{
						Event = PipelineEvents.Add(Key, MakeShareable(new FPThreadEvent()));
						Event->Create(true);
						bCompile = true;
					}
					check(Event.IsValid());

					EventsMutex.WriteUnlock();
				}

				if (bCompile)
				{
					const double CompilationStartTime = FPlatformTime::Seconds();
					Desc = CreateMTLRenderPipeline(Device, bSync, Key, Init, State);
					const float CompilationDuration = static_cast<float>(FPlatformTime::Seconds() - CompilationStartTime);

					AccumulatePSOMetrics(CompilationDuration);


					if (Desc != nullptr)
					{
						PipelineMutex.WriteLock();

						Pipelines.Add(Key, Desc);
						ReverseLookup.Add(Desc, Key);

						PipelineMutex.WriteUnlock();
					}

					EventsMutex.WriteLock();

					Event->Trigger();
					PipelineEvents.Remove(Key);

					EventsMutex.WriteUnlock();
				}
				else
				{
					check(Event.IsValid());
					Event->Wait();

					PipelineMutex.ReadLock();
					Desc = Pipelines.FindRef(Key);
					PipelineMutex.ReadUnlock();
					check(Desc);
				}
			}
		}
		else
		{
			Desc = CreateMTLRenderPipeline(Device, bSync, Key, Init, State);
		}
		return Desc;
	}
	
	void ReleaseRenderPipeline(FMetalShaderPipelinePtr Pipeline)
	{
        // For render pipeline states we might need to remove the PSO from the cache when we aren't caching them for program lifetime
        if (GMetalCacheShaderPipelines == 0)
        {
            FMetalShaderPipelineCache::Get().RemoveRenderPipeline(Pipeline);
        }
	}
	
	void RemoveRenderPipeline(FMetalShaderPipelinePtr Pipeline)
	{
		check (GMetalCacheShaderPipelines == 0);
		{
			FRWScopeLock Lock(PipelineMutex, SLT_Write);
			FMetalGraphicsPipelineKey* Desc = ReverseLookup.Find(Pipeline);
			
			if (Desc)
			{   
				Pipelines.Remove(*Desc);
				ReverseLookup.Remove(Pipeline);
			}
		}
	}
    
    void Destroy()
    {
        FRWScopeLock Lock(PipelineMutex, SLT_Write);
        
        Pipelines.Empty();
        ReverseLookup.Empty();
        PipelineEvents.Empty();
    }
	
private:
	static FMetalShaderPipelineCache* Self;
	
	FMetalDevice& Device;
	FRWLock PipelineMutex;
	FRWLock EventsMutex;
	TMap<FMetalGraphicsPipelineKey, FMetalShaderPipelinePtr> Pipelines;
	TMap<FMetalShaderPipelinePtr, FMetalGraphicsPipelineKey> ReverseLookup;
	TMap<FMetalGraphicsPipelineKey, TSharedPtr<FPThreadEvent, ESPMode::ThreadSafe>> PipelineEvents;
};

FMetalShaderPipelineCache* FMetalShaderPipelineCache::Self = nullptr;

void ShutdownPipelineCache()
{
    FMetalShaderPipelineCache::Get().Destroy();
	delete &FMetalShaderPipelineCache::Get();
}

FMetalShaderPipeline::~FMetalShaderPipeline()
{
#if METAL_DEBUG_OPTIONS
    if(VertexSource)
    {
        VertexSource->release();
        VertexSource = nullptr;
    }
    
    if(FragmentSource)
    {
        FragmentSource->release();
        FragmentSource = nullptr;
    }
    
    if(ComputeSource)
    {
        ComputeSource->release();
        ComputeSource = nullptr;
    }
#endif
    
}

void FMetalShaderPipeline::InitResourceMask(const FGraphicsPipelineStateInitializer& Init)
{
	if (RenderPipelineReflection)
	{
		InitResourceMask(EMetalShaderVertex, ResourceCast(Init.BoundShaderState.GetVertexShader())->SideTableBinding);
		InitResourceMask(EMetalShaderFragment, Init.BoundShaderState.GetPixelShader() ? ResourceCast(Init.BoundShaderState.GetPixelShader())->SideTableBinding : -1);
		
		if (Device.GetRuntimeDebuggingLevel() < EMetalDebugLevelValidation)
		{
			RenderPipelineReflection.reset();
		}
	}
}

void FMetalShaderPipeline::InitResourceMask(const FComputePipelineStateInitializer& Init)
{
	if (ComputePipelineReflection)
	{
		InitResourceMask(EMetalShaderCompute, ResourceCast(Init.ComputeShader)->SideTableBinding);
		
		if (Device.GetRuntimeDebuggingLevel() < EMetalDebugLevelValidation)
		{
			ComputePipelineReflection.reset();
		}
	}
}

void FMetalShaderPipeline::InitResourceMask(EMetalShaderFrequency Frequency, uint8 SideTableBinding)
{
    if (@available(macOS 13.0, iOS 16.0, *))
    {
        NS::Array* Bindings = nullptr;
        switch(Frequency)
        {
            case EMetalShaderVertex:
            {
                check(RenderPipelineReflection);
                Bindings = RenderPipelineReflection->vertexBindings();
                break;
            }
            case EMetalShaderFragment:
            {
                check(RenderPipelineReflection);
                Bindings = RenderPipelineReflection->fragmentBindings();
                break;
            }
            case EMetalShaderCompute:
            {
                check(ComputePipelineReflection);
                Bindings = ComputePipelineReflection->bindings();
                break;
            }
            default:
                check(false);
                break;
        }
        
        for (uint32 i = 0; i < Bindings->count(); i++)
        {
            MTL::Binding* Binding = (MTL::Binding*)Bindings->object(i);
            check(Binding);
            
            if (!Binding->used())
            {
                continue;
            }
            
            switch(Binding->type())
            {
                case MTL::BindingTypeBuffer:
                {
                    MTL::BufferBinding* BufferBinding = (MTL::BufferBinding*)Binding;
                    checkf(Binding->index() < ML_MaxBuffers, TEXT("Metal buffer index exceeded!"));
                    if (NSStringToFString(Binding->name()) != TEXT("BufferSizes") && SideTableBinding != Binding->index())
                    {
                        ResourceMask[Frequency].BufferMask |= (1 << Binding->index());
                        
                        if(BufferDataSizes[Frequency].Num() < 31)
                            BufferDataSizes[Frequency].SetNumZeroed(31);
                        
                        BufferDataSizes[Frequency][Binding->index()] = BufferBinding->bufferDataSize();
                    }
                    break;
                }
                case MTL::BindingTypeThreadgroupMemory:
                {
                    break;
                }
                case MTL::BindingTypeTexture:
                {
                    MTL::TextureBinding* TextureBinding = (MTL::TextureBinding*)Bindings->object(i);
                    checkf(Binding->index() < ML_MaxTextures, TEXT("Metal texture index exceeded!"));
                    ResourceMask[Frequency].TextureMask |= (FMetalTextureMask(1) << Binding->index());
                    TextureTypes[Frequency].Add(Binding->index(), (uint8)TextureBinding->textureType());
                    break;
                }
                case MTL::BindingTypeSampler:
                {
                    checkf(Binding->index() < ML_MaxSamplers, TEXT("Metal sampler index exceeded!"));
                    ResourceMask[Frequency].SamplerMask |= (1 << Binding->index());
                    break;
                }
                default:
                    check(false);
                    break;
            }
        }
    }
    else
    {
        NS::Array* Arguments = nullptr;
        switch(Frequency)
        {
            case EMetalShaderVertex:
            {
                check(RenderPipelineReflection);
                Arguments = RenderPipelineReflection->vertexArguments();
                break;
            }
            case EMetalShaderFragment:
            {
                check(RenderPipelineReflection);
                Arguments = RenderPipelineReflection->fragmentArguments();
                break;
            }
            case EMetalShaderCompute:
            {
                check(ComputePipelineReflection);
                Arguments = ComputePipelineReflection->arguments();
                break;
            }
            default:
                check(false);
                break;
        }
        
        for (uint32 i = 0; i < Arguments->count(); i++)
        {
            MTL::Argument* Argument = (MTL::Argument*)Arguments->object(i);
            check(Argument);
            
            if (!Argument->active())
            {
                continue;
            }
            
            switch(Argument->type())
            {
                case MTL::ArgumentTypeBuffer:
                {
                    checkf(Argument->index() < ML_MaxBuffers, TEXT("Metal buffer index exceeded!"));
                    if (NSStringToFString(Argument->name()) != TEXT("BufferSizes") && SideTableBinding != Argument->index())
                    {
                        ResourceMask[Frequency].BufferMask |= (1 << Argument->index());
                        
                        if(BufferDataSizes[Frequency].Num() < 31)
                            BufferDataSizes[Frequency].SetNumZeroed(31);
                        
                        BufferDataSizes[Frequency][Argument->index()] = Argument->bufferDataSize();
                    }
                    break;
                }
                case MTL::ArgumentTypeThreadgroupMemory:
                {
                    break;
                }
                case MTL::ArgumentTypeTexture:
                {
                    checkf(Argument->index() < ML_MaxTextures, TEXT("Metal texture index exceeded!"));
                    ResourceMask[Frequency].TextureMask |= (FMetalTextureMask(1) << Argument->index());
                    TextureTypes[Frequency].Add(Argument->index(), (uint8)Argument->textureType());
                    break;
                }
                case MTL::ArgumentTypeSampler:
                {
                    checkf(Argument->index() < ML_MaxSamplers, TEXT("Metal sampler index exceeded!"));
                    ResourceMask[Frequency].SamplerMask |= (1 << Argument->index());
                    break;
                }
                default:
                    check(false);
                    break;
            }
        }
    }
}

static MTLVertexDescriptorPtr GetMaskedVertexDescriptor(MTLVertexDescriptorPtr InputDesc, const CrossCompiler::FShaderBindingInOutMask& InOutMask)
{
	for (uint32 Attr = 0; Attr < MaxMetalStreams; Attr++)
	{
		if (!InOutMask.IsFieldEnabled((int32)Attr) && InputDesc->attributes()->object(Attr) != nullptr)
		{
			MTLVertexDescriptorPtr Desc = NS::TransferPtr(InputDesc->copy());
			CrossCompiler::FShaderBindingInOutMask BuffersUsed;
			for (int32 MetalStreamIndex = 0; MetalStreamIndex < MaxMetalStreams; ++MetalStreamIndex)
			{
				if (!InOutMask.IsFieldEnabled(MetalStreamIndex))
				{
					Desc->attributes()->setObject(nullptr, MetalStreamIndex);
				}
				else
				{
					BuffersUsed.EnableField(Desc->attributes()->object(MetalStreamIndex)->bufferIndex());
				}
			}
			for (int32 BufferIndex = 0; BufferIndex < ML_MaxBuffers; ++BufferIndex)
			{
				if (!BuffersUsed.IsFieldEnabled(BufferIndex))
				{
                    Desc->layouts()->setObject(nullptr, BufferIndex);
				}
			}
			return Desc;
		}
	}
	
	return InputDesc;
}

template <class TDescriptorType>
static bool ConfigureRenderPipelineDescriptor(FMetalDevice& Device, TDescriptorType* RenderPipelineDesc,
											  FMetalGraphicsPipelineKey const& Key,
											  const FGraphicsPipelineStateInitializer& Init)
{
	constexpr bool bIsRenderPipelineDesc = std::is_same_v<TDescriptorType, MTL::RenderPipelineDescriptor>;
	
	FMetalPixelShader* PixelShader = (FMetalPixelShader*)Init.BoundShaderState.PixelShaderRHI;
	FMetalVertexShader* VertexShader = Init.BoundShaderState.VertexShaderRHI ? (FMetalVertexShader*)Init.BoundShaderState.VertexShaderRHI : nullptr;
	
	uint32 const NumActiveTargets = Init.ComputeNumValidRenderTargets();
	check(NumActiveTargets <= MaxSimultaneousRenderTargets);
	if (PixelShader)
	{
		const uint32 MaxNumActiveTargets = __builtin_popcount(PixelShader->Bindings.InOutMask.Bitmask & ((1u << CrossCompiler::FShaderBindingInOutMask::MaxIndex) - 1));
		UE_CLOG((NumActiveTargets < MaxNumActiveTargets), LogMetal, Verbose, TEXT("NumActiveTargets doesn't match pipeline's pixel shader output mask: %u, %hx"), NumActiveTargets, PixelShader->Bindings.InOutMask.Bitmask);
	}
	
#if !UE_BUILD_SHIPPING
	bool bEnableValidation = false;
	if(Device.IsShaderValidationEnabled())
	{
		switch((EMetalShaderValidationType)GMetalShaderValidationType)
		{
			case EMetalShaderValidationType::All:
			case EMetalShaderValidationType::RenderPipelineOnly:
				bEnableValidation = true;
				break;
			case EMetalShaderValidationType::ShaderName:
			{
				if(PixelShader && FString(PixelShader->GetShaderName()).Contains(GMetalShaderValidationShaderName) ||
				   VertexShader && FString(VertexShader->GetShaderName()).Contains(GMetalShaderValidationShaderName)) 
				{
					bEnableValidation = true;
				}
			}
			case EMetalShaderValidationType::ComputeOnly:
			default:
				break;
		}
		
		if(bEnableValidation)
		{
			RenderPipelineDesc->setShaderValidation(MTL::ShaderValidationEnabled);
		}
	}
#endif
	
	FMetalBlendState* BlendState = (FMetalBlendState*)Init.BlendState;
	
	MTL::RenderPipelineColorAttachmentDescriptorArray* ColorAttachments = RenderPipelineDesc->colorAttachments();
	
	uint32 TargetWidth = 0;
	for (uint32 ActiveTargetIndex = 0; ActiveTargetIndex < NumActiveTargets; ActiveTargetIndex++)
	{
		EPixelFormat TargetFormat = (EPixelFormat)Init.RenderTargetFormats[ActiveTargetIndex];
		
		const bool bIsActiveTargetBound = (PixelShader && PixelShader->Bindings.InOutMask.IsFieldEnabled(ActiveTargetIndex));
		METAL_FATAL_ASSERT(!(TargetFormat == PF_Unknown && bIsActiveTargetBound), TEXT("Pipeline pixel shader expects target %u to be bound but it isn't: %s."), ActiveTargetIndex, *NSStringToFString(PixelShader->GetSourceCode()));
		
		TargetWidth += GPixelFormats[TargetFormat].BlockBytes;
		
		MTL::PixelFormat MetalFormat = UEToMetalFormat(Device, TargetFormat, EnumHasAnyFlags(Init.RenderTargetFlags[ActiveTargetIndex], TexCreate_SRGB));
		
        MTL::RenderPipelineColorAttachmentDescriptor* Attachment = ColorAttachments->object(ActiveTargetIndex);
		Attachment->setPixelFormat(MetalFormat);
		
		MTL::RenderPipelineColorAttachmentDescriptor* Blend = BlendState->RenderTargetStates[ActiveTargetIndex].BlendState;
		if(TargetFormat != PF_Unknown)
		{
			// assign each property manually, would be nice if this was faster
			Attachment->setBlendingEnabled(Blend->blendingEnabled());
			Attachment->setSourceRGBBlendFactor(Blend->sourceRGBBlendFactor());
			Attachment->setDestinationRGBBlendFactor(Blend->destinationRGBBlendFactor());
			Attachment->setRgbBlendOperation(Blend->rgbBlendOperation());
			Attachment->setSourceAlphaBlendFactor(Blend->sourceAlphaBlendFactor());
			Attachment->setDestinationAlphaBlendFactor(Blend->destinationAlphaBlendFactor());
			Attachment->setAlphaBlendOperation(Blend->alphaBlendOperation());
			Attachment->setWriteMask(Blend->writeMask());
		}
		else
		{
			Attachment->setBlendingEnabled(NO);
			Attachment->setWriteMask(MTL::ColorWriteMaskNone);
		}
	}
	
	// don't allow a PSO that is too wide
	if (!GSupportsWideMRT && TargetWidth > 16)
	{
		return false;
	}
	
	switch(Init.DepthStencilTargetFormat)
	{
		case PF_X24_G8:
		case PF_DepthStencil:
		{
			MTL::PixelFormat MetalFormat = (MTL::PixelFormat)GPixelFormats[Init.DepthStencilTargetFormat].PlatformFormat;
			RenderPipelineDesc->setDepthAttachmentPixelFormat(MetalFormat);
			RenderPipelineDesc->setStencilAttachmentPixelFormat(MetalFormat);
			break;
		}
		case PF_ShadowDepth:
        case PF_D24:
        {
            RenderPipelineDesc->setDepthAttachmentPixelFormat((MTL::PixelFormat)GPixelFormats[Init.DepthStencilTargetFormat].PlatformFormat);
            break;
        }
		case PF_Unknown:
		{
			RenderPipelineDesc->setDepthAttachmentPixelFormat(MTL::PixelFormatInvalid);
			RenderPipelineDesc->setStencilAttachmentPixelFormat(MTL::PixelFormatInvalid);
			break;
		}
		default:
		{
			break;
		}
	}
	
	if (bIsRenderPipelineDesc)
    {
        check(Init.BoundShaderState.VertexShaderRHI != nullptr);
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
        check(Init.BoundShaderState.GetGeometryShader() == nullptr);
#endif
    }
	
	static bool bAllowMSAA = AllowMSAA();
	uint16 NumSamples = bAllowMSAA ? FMath::Max(Init.NumSamples, (uint16)1u) : (uint16)1u;
	if constexpr(bIsRenderPipelineDesc)
	{
		RenderPipelineDesc->setSampleCount(NumSamples);
	}
	else
	{
		RenderPipelineDesc->setRasterSampleCount(NumSamples);
	}

	RenderPipelineDesc->setAlphaToCoverageEnabled(NumSamples > 1 && BlendState->bUseAlphaToCoverage);
	
	if constexpr(bIsRenderPipelineDesc)
	{
		if (Device.SupportsFeature(EMetalFeaturesLayeredRendering))
		{
			RenderPipelineDesc->setInputPrimitiveTopology(TranslatePrimitiveTopology(Init.PrimitiveType));
		}
	}
	
	if constexpr(bIsRenderPipelineDesc)
	{
		MTL::PipelineBufferDescriptorArray* VertexPipelineBuffers = RenderPipelineDesc->vertexBuffers();
		FMetalShaderBindings& VertexBindings = VertexShader->Bindings;
		int8 VertexSideTable = VertexShader->SideTableBinding;
		{
			uint32 ImmutableBuffers = VertexBindings.ConstantBuffers | VertexBindings.ArgumentBuffers;
			while(ImmutableBuffers)
			{
				uint32 Index = __builtin_ctz(ImmutableBuffers);
				ImmutableBuffers &= ~(1 << Index);
				
				if (Index < ML_MaxBuffers)
				{
					MTL::PipelineBufferDescriptor* PipelineBuffer = VertexPipelineBuffers->object(Index);
					PipelineBuffer->setMutability(MTL::MutabilityImmutable);
				}
			}
			if (VertexSideTable > 0)
			{
				MTL::PipelineBufferDescriptor* PipelineBuffer = VertexPipelineBuffers->object(VertexSideTable);
				PipelineBuffer->setMutability(MTL::MutabilityImmutable);
			}
		}
	}
	
	if (PixelShader)
	{
		MTL::PipelineBufferDescriptorArray* FragmentPipelineBuffers = RenderPipelineDesc->fragmentBuffers();
		uint32 ImmutableBuffers = PixelShader->Bindings.ConstantBuffers | PixelShader->Bindings.ArgumentBuffers;
		while(ImmutableBuffers)
		{
			uint32 Index = __builtin_ctz(ImmutableBuffers);
			ImmutableBuffers &= ~(1 << Index);
			
			if (Index < ML_MaxBuffers)
			{
				MTL::PipelineBufferDescriptor* PipelineBuffer = FragmentPipelineBuffers->object(Index);
				PipelineBuffer->setMutability(MTL::MutabilityImmutable);
			}
		}
		if (PixelShader->SideTableBinding > 0)
		{
			MTL::PipelineBufferDescriptor* PipelineBuffer = FragmentPipelineBuffers->object(PixelShader->SideTableBinding);
			PipelineBuffer->setMutability(MTL::MutabilityImmutable);
		}
	}
	
	return true;
}

/*
 * PSO Harvesting and Reuse
 *
 * Usage:
 *
 * To Harvest, run the game with -MetalPSOCache=recreate
 * All Render and Compute PSOs created will be harvested into the MTLBinaryArchive
 * Console command r.Metal.ShaderPipelineCache.Save will trigger the serialization to file.
 * The binary archive's location will be printed to the log.
 *
 * To reuse, run the game with -MetalPSOCache=use
 * The binary archive will be opened from the saved location.
 * The binary archive can be moved to another device, as long as it's the same GPU
 * and OS build.
 *
 */

enum class CacheMode
{
	Uninitialized,
	Recreate,
	Append,
	Use,
	Ignore
};

static CacheMode GPSOCacheMode = CacheMode::Uninitialized;
static MTL::BinaryArchive* GPSOBinaryArchive = nullptr;
static uint32_t GPSOHarvestCount = 0;

static NS::URL* PipelineCacheSaveLocation()
{
	NSString* dir = NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES)[0];
	if (!dir)
	{
		UE_LOGF(LogMetal, Error, "Metal Pipeline Cache: Unable to find Documents folder");
	}
	NSString* path = [dir stringByAppendingString : @"/mtlarchive.bin"];
	NS::URL* NativeURL = NS::URL::fileURLWithPath(NS::String::string(path.UTF8String, NS::UTF8StringEncoding));
	return NativeURL;
}

static void InitializeMetalPipelineCache(FMetalDevice& Device)
{
	FString strCacheMode;
	FParse::Value(FCommandLine::Get(), TEXT("MetalPSOCache="), strCacheMode);
		
	if (strCacheMode.Compare(TEXT("recreate"), ESearchCase::IgnoreCase) == 0)
	{
		GPSOCacheMode = CacheMode::Recreate;
        UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: recreate PSO cache");
	}
	else if (strCacheMode.Compare(TEXT("append"), ESearchCase::IgnoreCase) == 0)
	{
		GPSOCacheMode = CacheMode::Append;
        UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: append to PSO cache");
	}
	else if (strCacheMode.Compare(TEXT("use"), ESearchCase::IgnoreCase) == 0)
	{
		GPSOCacheMode = CacheMode::Use;
        UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: use PSO cache");
	}
	else
	{
		GPSOCacheMode = CacheMode::Ignore;
        UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: ignore PSO cache");
	}
		
	if (GPSOCacheMode != CacheMode::Ignore)
	{
		NS::URL* url = PipelineCacheSaveLocation();
        UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: pso cache save location will be: %ls", *FString(url->fileSystemRepresentation()));
        
        MTL::BinaryArchiveDescriptor* archDesc = MTL::BinaryArchiveDescriptor::alloc()->init();
        check(archDesc);
        
		archDesc->setUrl(((GPSOCacheMode == CacheMode::Append) || (GPSOCacheMode == CacheMode::Use)) ? url : nullptr);
		MTL::Device* MTLDevice = Device.GetDevice();
		NS::Error * err = nullptr;
		GPSOBinaryArchive = MTLDevice->newBinaryArchive(archDesc, &err);
		if (err)
		{
			UE_LOGF(LogMetal, Error, "Error adding Pipeline Functions to Binary Archive: %ls", *FString(url->fileSystemRepresentation()));
		}
        archDesc->release();
	}
}

static void RelatePipelineStateToCache(const MTL::RenderPipelineDescriptor* PipelineDesc, NS::UInteger * outPipelineOpts)
{
	if (GPSOBinaryArchive && (GPSOCacheMode != CacheMode::Ignore))
	{
		if (GPSOCacheMode == CacheMode::Recreate || GPSOCacheMode == CacheMode::Append)
		{
			NS::Error * err = nullptr;
			bool bAddedBinaryPSO = GPSOBinaryArchive->addRenderPipelineFunctions(PipelineDesc, &err);
			if (err)
			{
				UE_LOGF(LogMetal, Warning, "Metal Pipeline Cache: Error adding Pipeline Functions to Binary Archive: %ls", *NSStringToFString(err->localizedDescription()));
			}
            else if(bAddedBinaryPSO)
            {
                GPSOHarvestCount++;
                if(GMetalBinaryCacheDebugOutput && GPSOHarvestCount % 100)
                {
                    UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: Harvested PSO count: %d", GPSOHarvestCount);
                }
            }
		}
	}
}

static void RelatePipelineStateToCache(const MTL::ComputePipelineDescriptor* PipelineDesc, NS::UInteger * outPipelineOpts)
{
	if (GPSOBinaryArchive && (GPSOCacheMode != CacheMode::Ignore))
	{
		if (GPSOCacheMode == CacheMode::Recreate || GPSOCacheMode == CacheMode::Append)
		{
			NS::Error* err = nullptr;
            bool bAddedBinaryPSO = GPSOBinaryArchive->addComputePipelineFunctions(PipelineDesc, &err);
			if (err)
			{
				UE_LOGF(LogMetal, Warning, "Metal Pipeline Cache: Error adding Pipeline Functions to Binary Archive: %ls", *NSStringToFString(err->localizedDescription()));
			}
            else if(bAddedBinaryPSO)
            {
                GPSOHarvestCount++;
                if(GMetalBinaryCacheDebugOutput && GPSOHarvestCount % 100)
                {
                    UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: Harvested PSO count: %d", GPSOHarvestCount);
                }
            }
		}
	}
}

void MetalConsoleCommandSavePipelineFileCache()
{
	UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: requesting PSO save...");
	{
		if (GPSOBinaryArchive && (GPSOCacheMode == CacheMode::Recreate || GPSOCacheMode == CacheMode::Append))
		{
			NS::URL* url = PipelineCacheSaveLocation();
			UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: Serialize harvested PSOs to: %ls", *FString(url->fileSystemRepresentation()));
            UE_LOGF(LogMetal, Log, "Metal Pipeline Cache: Serialized PSO Count: %d", GPSOHarvestCount);
            
			NS::Error* err = nullptr;
			GPSOBinaryArchive->serializeToURL(url, &err);
			if (err)
			{
				UE_LOGF(LogMetal, Error, "Metal Pipeline Cache: Error Serializing binary archive: %ls", *NSStringToFString(err->localizedDescription()));
			}
		}
	}
}

static FAutoConsoleCommand SavePipelineCacheCmd(
    TEXT("rhi.Metal.ShaderPipelineCache.Save"),
    TEXT("Save the current pipeline file cache."),
    FConsoleCommandDelegate::CreateStatic(MetalConsoleCommandSavePipelineFileCache));

static FMetalShaderPipelinePtr CreateMTLRenderPipeline(FMetalDevice& Device, bool const bSync, FMetalGraphicsPipelineKey const& Key, const FGraphicsPipelineStateInitializer& Init, FMetalGraphicsPipelineState const* State)
{
	if (GPSOCacheMode == CacheMode::Uninitialized)
	{
		InitializeMetalPipelineCache(Device);
	}

    FMetalVertexShader* VertexShader = (FMetalVertexShader*)Init.BoundShaderState.VertexShaderRHI;
    FMetalPixelShader* PixelShader = (FMetalPixelShader*)Init.BoundShaderState.PixelShaderRHI;
    
    MTLFunctionPtr vertexFunction = VertexShader ? VertexShader->GetFunction() : MTLFunctionPtr();
    MTLFunctionPtr fragmentFunction = PixelShader ? PixelShader->GetFunction() : MTLFunctionPtr();
    
#if PLATFORM_SUPPORTS_MESH_SHADERS
    FMetalMeshShader* MeshShader = (FMetalMeshShader*)Init.BoundShaderState.GetMeshShader();
    FMetalAmplificationShader* AmplificationShader = (FMetalAmplificationShader*)Init.BoundShaderState.GetAmplificationShader();
    
    MTLFunctionPtr meshFunction = MeshShader ? MeshShader->GetFunction() : MTLFunctionPtr();
    MTLFunctionPtr amplificationFunction = AmplificationShader ? AmplificationShader->GetFunction() : MTLFunctionPtr();
#endif
    
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    FMetalGeometryShader* GeometryShader = (FMetalGeometryShader*)Init.BoundShaderState.GetGeometryShader();
	MTLFunctionPtr geometryFunction = GeometryShader ? GeometryShader->GetFunction() : MTLFunctionPtr();
    
    if (geometryFunction)
    {
        vertexFunction = VertexShader->GetObjectFunctionForGeometryEmulation();
    }
#endif

    FMetalShaderPipeline* Pipeline = nullptr;
    if (vertexFunction && ((PixelShader != nullptr) == (fragmentFunction.get() != nullptr))
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
        && geometryFunction == MTLFunctionPtr()
#endif
	)
    {
		NS::Error* Error = nullptr;
		MTL::Device* MetalDevice = Device.GetDevice();

		uint32 const NumActiveTargets = Init.ComputeNumValidRenderTargets();
        check(NumActiveTargets <= MaxSimultaneousRenderTargets);
		
		Pipeline = new FMetalShaderPipeline(Device);
		METAL_DEBUG_OPTION(FMemory::Memzero(Pipeline->ResourceMask, sizeof(Pipeline->ResourceMask)));

		MTLRenderPipelineDescriptorPtr RenderPipelineDesc = NS::TransferPtr(MTL::RenderPipelineDescriptor::alloc()->init());
        check(RenderPipelineDesc);
        
		if ((GPSOCacheMode == CacheMode::Append) || (GPSOCacheMode == CacheMode::Use))
		{
			RenderPipelineDesc->setBinaryArchives(NS::Array::array(GPSOBinaryArchive));
		}
		
        MTLComputePipelineDescriptorPtr ComputePipelineDesc;
		
		if (!ConfigureRenderPipelineDescriptor(Device, RenderPipelineDesc.get(), Key, Init))
		{
			return nullptr;
		}
        
        FMetalVertexDeclaration* VertexDecl = (FMetalVertexDeclaration*)Init.BoundShaderState.VertexDeclarationRHI;
		
        MTLVertexDescriptorPtr MaskedVertexDesc = GetMaskedVertexDescriptor(VertexDecl->Layout.VertexDesc, VertexShader->Bindings.InOutMask);
		
		if(!IsMetalBindlessEnabled())
		{
			RenderPipelineDesc->setVertexDescriptor(MaskedVertexDesc.get());
		}
#if METAL_USE_METAL_SHADER_CONVERTER
		else
		{
			// Create and link stagein function.
			if (State->StageInFunctionBytecode.Num() > 0)
			{
				dispatch_data_t LibraryData = dispatch_data_create(State->StageInFunctionBytecode.GetData(), State->StageInFunctionBytecode.Num(), nil, DISPATCH_DATA_DESTRUCTOR_DEFAULT);
				
				NS::Error* error = nullptr;
				MTLLibraryPtr StageInLib = NS::RetainPtr(MetalDevice->newLibrary(LibraryData, &error));
				MTL::LinkedFunctions* StageInFunction = MTL::LinkedFunctions::alloc()->init();
				StageInFunction->setFunctions(NS::Array::array(
															   StageInLib->newFunction(NS::String::string("irconverter_stage_in_shader", NS::UTF8StringEncoding))
															   ));
				
				RenderPipelineDesc->setVertexLinkedFunctions(StageInFunction);
			}
		}
#endif
		
		RenderPipelineDesc->setVertexFunction(vertexFunction.get());
		RenderPipelineDesc->setFragmentFunction(fragmentFunction.get());
#if ENABLE_METAL_GPUPROFILE
		NS::String* VertexName = vertexFunction->name();
		NS::String* FragmentName = fragmentFunction ? fragmentFunction->name() : NS::String::string("", NS::UTF8StringEncoding);
        FString Label = FString::Printf(TEXT("%s+%s"), *NSStringToFString(VertexName), *NSStringToFString(FragmentName));
		RenderPipelineDesc->setLabel(FStringToNSString(Label));
#endif
		NS::UInteger RenderOption = MTL::PipelineOptionNone;
		if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
		{
			RenderOption = MTL::PipelineOptionArgumentInfo | MTL::PipelineOptionBufferTypeInfo;
		}

		{
			RelatePipelineStateToCache(RenderPipelineDesc.get(), &RenderOption);
            
#if METAL_DEBUG_OPTIONS
            Pipeline->RenderDesc = RenderPipelineDesc;
#endif
            MTL::RenderPipelineReflection* Reflection = nullptr;
            Pipeline->RenderPipelineState = NS::TransferPtr(MetalDevice->newRenderPipelineState(RenderPipelineDesc.get(), (MTL::PipelineOption)RenderOption, &Reflection, &Error));
            if(Reflection)
            {
                Pipeline->RenderPipelineReflection = NS::RetainPtr(Reflection);
            }
		}
		
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Failed to generate a pipeline state object: %ls", *NSStringToFString(Error->description()));
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Vertex shader: %ls", *NSStringToFString(VertexShader->GetSourceCode()));
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Pixel shader: %ls", PixelShader ? *NSStringToFString(PixelShader->GetSourceCode()) : TEXT("NULL"));
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Descriptor: %ls", *NSStringToFString(RenderPipelineDesc->description()));
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Vertex Descriptor: %ls", *NSStringToFString(VertexDecl->Layout.VertexDesc->description()));
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Masked Vertex Descriptor: %ls", *NSStringToFString(MaskedVertexDesc->description()));
		UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Failed to generate a render pipeline state object:\n\n %ls\n\n", *NSStringToFString(Error->localizedDescription()));
		
		// We need to pass a failure up the chain, so we'll clean up here.
		if(Pipeline->RenderPipelineState.get() == nullptr)
		{
            delete Pipeline;
			return nullptr;
		}
		
#if METAL_DEBUG_OPTIONS
		// Reset the pipeline descriptor to release MTLFunction/MTLLibrary references it retains.
		Pipeline->RenderDesc.reset();

		Pipeline->VertexSource = VertexShader->GetSourceCode();
        Pipeline->VertexSource->retain();
        
        Pipeline->FragmentSource = PixelShader ? PixelShader->GetSourceCode() : nullptr;
        if(Pipeline->FragmentSource)
        {
            Pipeline->FragmentSource->retain();
        }
#endif
		
#if METAL_DEBUG_OPTIONS
        if (GFrameCounter > 3)
        {
            UE_LOGF(LogMetal, Verbose, "Created a hitchy pipeline state for hash %llx %llx %llx", (uint64)Key.RenderPipelineHash.RasterBits, (uint64)(Key.RenderPipelineHash.TargetBits), (uint64)Key.VertexDescriptorHash.VertexDescHash);
        }
#endif
    }
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS // TODO: Merge Geometry Emulation/Mesh Shader paths
    else if (vertexFunction && geometryFunction)
	{
		NS::Error* Error;
		MTL::Device* MetalDevice = Device.GetDevice();
		
		uint32 const NumActiveTargets = Init.ComputeNumValidRenderTargets();
		check(NumActiveTargets <= MaxSimultaneousRenderTargets);
		
		Pipeline = new FMetalShaderPipeline(Device);
		METAL_DEBUG_OPTION(FMemory::Memzero(Pipeline->ResourceMask, sizeof(Pipeline->ResourceMask)));
		
		MTLMeshRenderPipelineDescriptorPtr MeshPipelineDesc = NS::TransferPtr(MTL::MeshRenderPipelineDescriptor::alloc()->init());
		MeshPipelineDesc->setObjectFunction(vertexFunction.get());
		MeshPipelineDesc->setMeshFunction(geometryFunction.get());
		MeshPipelineDesc->setFragmentFunction(fragmentFunction.get());
		
		if (!ConfigureRenderPipelineDescriptor(Device, MeshPipelineDesc.get(), Key, Init))
		{
			delete Pipeline;
			return nullptr;
		}
		
		// Create and link stagein function.
		if (State->StageInFunctionBytecode.Num() > 0)
		{
			dispatch_data_t LibraryData = dispatch_data_create(State->StageInFunctionBytecode.GetData(), State->StageInFunctionBytecode.Num(), nullptr, DISPATCH_DATA_DESTRUCTOR_DEFAULT);
			MTLLibraryPtr StageInLib = NS::RetainPtr(MetalDevice->newLibrary(LibraryData, nullptr));
			MTL::LinkedFunctions* StageInFunction = MTL::LinkedFunctions::alloc()->init();
			
			StageInFunction->setFunctions(NS::Array::array(
				StageInLib->newFunction(NS::String::string("irconverter_stage_in_shader", NS::UTF8StringEncoding))
			));
			
			MeshPipelineDesc->setObjectLinkedFunctions(StageInFunction);
		}
		
#if ENABLE_METAL_GPUPROFILE
		NS::String* MeshName = geometryFunction->name();
		NS::String* AmplificationName = vertexFunction->name();
		NS::String* FragmentName = fragmentFunction ? fragmentFunction->name() : NS::String::string();
		
		FString LabelName = FString::Printf(TEXT("%s+%s+%s"), *NSStringToFString(MeshName), *NSStringToFString(AmplificationName), *NSStringToFString(FragmentName));
		MeshPipelineDesc->setLabel(FStringToNSString(LabelName));
#endif
		
		NS::UInteger RenderOption = MTL::PipelineOptionNone;
		MTL::RenderPipelineReflection* Reflection = nullptr;
        if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
        {
            RenderOption = MTL::PipelineOptionArgumentInfo | MTL::PipelineOptionBufferTypeInfo;
        }

        {
            NS::Error* RenderError = nullptr;
#if 0
            // Binary Archive does not support Mesh shaders...
            RelatePipelineStateToCache(MeshPipelineDesc, &RenderOption);
#endif
            Pipeline->RenderPipelineState = NS::TransferPtr(MetalDevice->newRenderPipelineState(MeshPipelineDesc.get(), (MTL::PipelineOption)RenderOption, &Reflection, &RenderError));
			
            if (Reflection)
            {
                Pipeline->RenderPipelineReflection = NS::RetainPtr(Reflection);
#if METAL_DEBUG_OPTIONS
                Pipeline->MeshRenderDesc = MeshPipelineDesc;
#endif
            }
            Error = RenderError;
        }
        
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Failed to generate a pipeline state object: %ls", *NSStringToFString(Error->description()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Mesh shader: %ls", *NSStringToFString(GeometryShader->GetSourceCode()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Object shader: %ls", *NSStringToFString(VertexShader->GetSourceCode()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Pixel shader: %ls", PixelShader ? *NSStringToFString(PixelShader->GetSourceCode()) : TEXT("NULL"));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Descriptor: %ls", *NSStringToFString(MeshPipelineDesc->description()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Failed to generate a mesh render pipeline state object:\n\n %ls\n\n", *NSStringToFString(Error->localizedDescription()));
        
        // We need to pass a failure up the chain, so we'll clean up here.
		if(Pipeline->RenderPipelineState.get() == nullptr)
		{
			delete Pipeline;
			return nullptr;
		}

#if METAL_DEBUG_OPTIONS
		// Reset the pipeline descriptor to release MTLFunction/MTLLibrary references it retains.
		Pipeline->MeshRenderDesc.reset();

        Pipeline->MeshSource = MeshShader ? MeshShader->GetSourceCode() : GeometryShader->GetSourceCode();
        Pipeline->ObjectSource = AmplificationShader ? AmplificationShader->GetSourceCode() : nil;
        Pipeline->FragmentSource = PixelShader ? PixelShader->GetSourceCode() : nil;
#endif
    }
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
    else if (meshFunction)
    {
        NS::Error* Error;
        MTL::Device* MetalDevice = Device.GetDevice();

        uint32 const NumActiveTargets = Init.ComputeNumValidRenderTargets();
        check(NumActiveTargets <= MaxSimultaneousRenderTargets);
        
        Pipeline = new FMetalShaderPipeline(Device);
        METAL_DEBUG_OPTION(FMemory::Memzero(Pipeline->ResourceMask, sizeof(Pipeline->ResourceMask)));

		MTLMeshRenderPipelineDescriptorPtr MeshPipelineDesc = NS::TransferPtr(MTL::MeshRenderPipelineDescriptor::alloc()->init());
        MeshPipelineDesc->setObjectFunction(amplificationFunction.get());
        MeshPipelineDesc->setMeshFunction(meshFunction.get());
        MeshPipelineDesc->setFragmentFunction(fragmentFunction.get());
        
        if (!ConfigureRenderPipelineDescriptor(Device, MeshPipelineDesc.get(), Key, Init))
        {
			delete Pipeline;
			return nullptr;
        }
        
#if ENABLE_METAL_GPUPROFILE
        NS::String* MeshName = meshFunction->name();
		NS::String* AmplificationName = amplificationFunction ? amplificationFunction->name() : NS::String::string();
		NS::String* FragmentName = fragmentFunction ? fragmentFunction->name() : NS::String::string();
		
		FString LabelName = FString::Printf(TEXT("%s+%s+%s"), *NSStringToFString(MeshName), *NSStringToFString(AmplificationName), *NSStringToFString(FragmentName));
		MeshPipelineDesc->setLabel(FStringToNSString(LabelName));
#endif
        
		NS::UInteger RenderOption = MTL::PipelineOptionNone;
		MTL::RenderPipelineReflection* Reflection = nullptr;
		if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
		{
			RenderOption = MTL::PipelineOptionArgumentInfo | MTL::PipelineOptionBufferTypeInfo;
		}

        {
            NS::Error* RenderError;
#if 0
            // Binary Archive does not support Mesh shaders...
            RelatePipelineStateToCache(MeshPipelineDesc, &RenderOption);
#endif
            Pipeline->RenderPipelineState = NS::TransferPtr(MetalDevice->newRenderPipelineState(MeshPipelineDesc.get(),
																(MTL::PipelineOption)RenderOption, &Reflection, &RenderError));
            if (Reflection)
            {
                Pipeline->RenderPipelineReflection = NS::RetainPtr(Reflection);
#if METAL_DEBUG_OPTIONS
                Pipeline->MeshRenderDesc = MeshPipelineDesc;
#endif
            }
            Error = RenderError;
        }
        
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Failed to generate a pipeline state object: %ls", *NSStringToFString(Error->description()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Mesh shader: %ls", *NSStringToFString(MeshShader->GetSourceCode()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Object shader: %ls", AmplificationShader ? *NSStringToFString(AmplificationShader->GetSourceCode()) : TEXT("NULL"));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Pixel shader: %ls", PixelShader ? *NSStringToFString(PixelShader->GetSourceCode()) : TEXT("NULL"));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Descriptor: %ls", *NSStringToFString(MeshPipelineDesc->description()));
        UE_CLOGF((Pipeline->RenderPipelineState.get() == nullptr), LogMetal, Error, "Failed to generate a mesh render pipeline state object:\n\n %ls\n\n", *NSStringToFString(Error->localizedDescription()));
        
        // We need to pass a failure up the chain, so we'll clean up here.
        if(Pipeline->RenderPipelineState.get() == nullptr)
        {
			delete Pipeline;
            return nil;
        }

#if METAL_DEBUG_OPTIONS
		// Reset the pipeline descriptor to release MTLFunction/MTLLibrary references it retains.
		Pipeline->MeshRenderDesc.reset();

        Pipeline->MeshSource = MeshShader->GetSourceCode();
        Pipeline->ObjectSource = AmplificationShader ? AmplificationShader->GetSourceCode() : nil;
        Pipeline->FragmentSource = PixelShader ? PixelShader->GetSourceCode() : nil;
#endif
    }
#endif
    else
    {
        checkNoEntry();
    }

	if (Pipeline && Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
	{
		Pipeline->InitResourceMask(Init);
	}
	
    return !bSync ? nullptr : FMetalShaderPipelinePtr(Pipeline);
}

FMetalShaderPipelinePtr GetMTLRenderPipeline(bool const bSync, FMetalGraphicsPipelineState const* State, const FGraphicsPipelineStateInitializer& Init)
{
	return FMetalShaderPipelineCache::Get().GetRenderPipeline(bSync, State, Init);
}

void ReleaseMTLRenderPipeline(FMetalShaderPipelinePtr Pipeline)
{
	FMetalShaderPipelineCache::Get().ReleaseRenderPipeline(Pipeline);
}

FMetalPipelineStateCacheManager::FMetalPipelineStateCacheManager(FMetalDevice& Device)
{
	FMetalShaderPipelineCache::Create(Device);
	
#if PLATFORM_IOS
	ApplicationWillDeactivateDelegateHandle = FCoreDelegates::ApplicationWillDeactivateDelegate.AddStatic(&FShaderPipelineCache::PauseBatching);
	ApplicationHasReactivatedDelegateHandle = FCoreDelegates::ApplicationHasReactivatedDelegate.AddStatic(&FShaderPipelineCache::ResumeBatching);
	
	OnShaderPipelineCachePreOpenDelegate = FShaderPipelineCache::GetCachePreOpenDelegate().AddRaw(this, &FMetalPipelineStateCacheManager::OnShaderPipelineCachePreOpen);
	OnShaderPipelineCacheOpenedDelegate = FShaderPipelineCache::GetCacheOpenedDelegate().AddRaw(this, &FMetalPipelineStateCacheManager::OnShaderPipelineCacheOpened);
	OnShaderPipelineCachePrecompilationCompleteDelegate = FShaderPipelineCache::GetPrecompilationCompleteDelegate().AddRaw(this, &FMetalPipelineStateCacheManager::OnShaderPipelineCachePrecompilationComplete);
#endif
}

FMetalPipelineStateCacheManager::~FMetalPipelineStateCacheManager()
{
	FCoreDelegates::ApplicationWillDeactivateDelegate.Remove(ApplicationWillDeactivateDelegateHandle);
	FCoreDelegates::ApplicationHasReactivatedDelegate.Remove(ApplicationHasReactivatedDelegateHandle);
	
	if (OnShaderPipelineCacheOpenedDelegate.IsValid())
	{
		FShaderPipelineCache::GetCacheOpenedDelegate().Remove(OnShaderPipelineCacheOpenedDelegate);
	}
	
	if (OnShaderPipelineCachePrecompilationCompleteDelegate.IsValid())
	{
		FShaderPipelineCache::GetPrecompilationCompleteDelegate().Remove(OnShaderPipelineCachePrecompilationCompleteDelegate);
	}
}

void FMetalPipelineStateCacheManager::OnShaderPipelineCachePreOpen(FString const& Name, EShaderPlatform Platform, bool& bReady)
{
	// only do this when haven't gotten a full pso cache already
	struct stat FileInfo;
	static FString PrivateWritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
	FString Result = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/functions.data", [NSBundle mainBundle].bundleIdentifier]);
	FString Result2 = PrivateWritePathBase + FString([NSString stringWithFormat:@"/Caches/%@/com.apple.metal/usecache.txt", [NSBundle mainBundle].bundleIdentifier]);
	if (stat(TCHAR_TO_UTF8(*Result), &FileInfo) != -1 && ((FileInfo.st_size / 1024 / 1024) > GMetalCacheMinSize) && stat(TCHAR_TO_UTF8(*Result2), &FileInfo) != -1)
	{
		bReady = false;
		FShaderPipelineCache::SetBatchMode(FShaderPipelineCache::BatchMode::Background);
	}
	else
	{
		bReady = true;
		FShaderPipelineCache::SetBatchMode(FShaderPipelineCache::BatchMode::Precompile);
	}
}

void FMetalPipelineStateCacheManager::OnShaderPipelineCacheOpened(FString const& Name, EShaderPlatform Platform, uint32 Count, const FGuid& VersionGuid, FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext)
{
	ShaderCachePrecompileContext.SetPrecompilationIsSlowTask();
}

void FMetalPipelineStateCacheManager::OnShaderPipelineCachePrecompilationComplete(uint32 Count, double Seconds, const FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext)
{
	// Want to ignore any subsequent Shader Pipeline Cache opening/closing, eg when loading modules
	FShaderPipelineCache::GetCachePreOpenDelegate().Remove(OnShaderPipelineCachePreOpenDelegate);
	FShaderPipelineCache::GetCacheOpenedDelegate().Remove(OnShaderPipelineCacheOpenedDelegate);
	FShaderPipelineCache::GetPrecompilationCompleteDelegate().Remove(OnShaderPipelineCachePrecompilationCompleteDelegate);
	OnShaderPipelineCachePreOpenDelegate.Reset();
	OnShaderPipelineCacheOpenedDelegate.Reset();
	OnShaderPipelineCachePrecompilationCompleteDelegate.Reset();
}
