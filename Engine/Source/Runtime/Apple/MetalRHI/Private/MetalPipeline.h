// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"
#include "ShaderPipelineCache.h"
#include "MetalShaderResources.h"

/**
 * The sampler, buffer and texture resource limits as defined here:
 * https://developer.apple.com/library/ios/documentation/Miscellaneous/Conceptual/MetalProgrammingGuide/Render-Ctx/Render-Ctx.html
 */
#if PLATFORM_IOS
#define METAL_MAX_TEXTURES 31
typedef uint32 FMetalTextureMask;
#elif PLATFORM_MAC
#define METAL_MAX_TEXTURES 128
typedef __uint128_t FMetalTextureMask;
#else
#error "Unsupported Platform!"
#endif

#define METAL_MAX_BUFFERS 31

typedef uint32 FMetalBufferMask;
typedef uint16 FMetalSamplerMask;

/** A structure for quick mask-testing of shader-stage resource bindings */
struct FMetalShaderResourceMask
{
    FMetalTextureMask TextureMask;
    FMetalBufferMask BufferMask;
    FMetalSamplerMask SamplerMask;
};

enum EMetalShaderFrequency
{
    EMetalShaderVertex = 0,
    EMetalShaderFragment = 1,
    EMetalShaderCompute = 2,
    EMetalShaderRenderNum = 2,
    EMetalShaderStagesNum = 3
};

enum EMetalLimits
{
    ML_MaxSamplers = 16, /** Maximum number of samplers */
    ML_MaxBuffers = METAL_MAX_BUFFERS, /** Maximum number of buffers */
    ML_MaxTextures = METAL_MAX_TEXTURES, /** Maximum number of textures - there are more textures available on Mac than iOS */
    ML_MaxViewports = 16 /** Technically this may be different at runtime, but this is the likely absolute upper-bound */
};

enum EMetalPipelineHashBits
{
	// Blend States
	NumBits_BlendState = 8, //(x8=64),
	
	// Raster States
	NumBits_RenderTargetFormat = 6, //(x8=48),
	NumBits_DepthFormat = 3, //(x1=3),
	NumBits_SampleCount = 3, //(x1=3),
	NumBits_PrimitiveTopology = 2, //(x1=2)
	NumBits_AlphaToCoverage = 1, //(x1=1)
};

enum EMetalPipelineHashOffsets
{
	Offset_BlendState0 = 0,
	Offset_BlendState1 = Offset_BlendState0 + NumBits_BlendState,
	Offset_BlendState2 = Offset_BlendState1 + NumBits_BlendState,
	Offset_BlendState3 = Offset_BlendState2 + NumBits_BlendState,
	Offset_BlendState4 = Offset_BlendState3 + NumBits_BlendState,
	Offset_BlendState5 = Offset_BlendState4 + NumBits_BlendState,
	Offset_BlendState6 = Offset_BlendState5 + NumBits_BlendState,
	Offset_BlendState7 = Offset_BlendState6 + NumBits_BlendState,
	Offset_BlendStateEnd = Offset_BlendState7 + NumBits_BlendState - 1, // 63

	Offset_RenderTargetFormat0 = 64,
	Offset_RenderTargetFormat1 = Offset_RenderTargetFormat0 + NumBits_RenderTargetFormat,
	Offset_RenderTargetFormat2 = Offset_RenderTargetFormat1 + NumBits_RenderTargetFormat,
	Offset_RenderTargetFormat3 = Offset_RenderTargetFormat2 + NumBits_RenderTargetFormat,
	Offset_RenderTargetFormat4 = Offset_RenderTargetFormat3 + NumBits_RenderTargetFormat,
	Offset_RenderTargetFormat5 = Offset_RenderTargetFormat4 + NumBits_RenderTargetFormat,
	Offset_RenderTargetFormat6 = Offset_RenderTargetFormat5 + NumBits_RenderTargetFormat,
	Offset_RenderTargetFormat7 = Offset_RenderTargetFormat6 + NumBits_RenderTargetFormat,
	Offset_DepthFormat = Offset_RenderTargetFormat7 + NumBits_RenderTargetFormat,
	Offset_SampleCount = Offset_DepthFormat + NumBits_DepthFormat,
	Offset_AlphaToCoverage = Offset_SampleCount + NumBits_SampleCount,
	Offset_PrimitiveTopology = Offset_AlphaToCoverage + NumBits_AlphaToCoverage,
	Offset_End = Offset_PrimitiveTopology + NumBits_PrimitiveTopology
};

class FMetalDevice;
class FMetalPipelineStateCacheManager
{
public:
	FMetalPipelineStateCacheManager(FMetalDevice& Device);
	~FMetalPipelineStateCacheManager();
	
private:
	FDelegateHandle ApplicationWillDeactivateDelegateHandle;
	FDelegateHandle ApplicationHasReactivatedDelegateHandle;
	FDelegateHandle OnShaderPipelineCachePreOpenDelegate;
	FDelegateHandle OnShaderPipelineCacheOpenedDelegate;
	FDelegateHandle OnShaderPipelineCachePrecompilationCompleteDelegate;
	
	/** Delegate handlers to track the ShaderPipelineCache precompile. */
	void OnShaderPipelineCachePreOpen(FString const& Name, EShaderPlatform Platform, bool& bReady);
	void OnShaderPipelineCacheOpened(FString const& Name, EShaderPlatform Platform, uint32 Count, const FGuid& VersionGuid, FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext);
	void OnShaderPipelineCachePrecompilationComplete(uint32 Count, double Seconds, const FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext);
};

class FMetalShaderPipeline
{
public:
    FMetalShaderPipeline(FMetalDevice& MetalDevice)
	 : Device(MetalDevice) {};
    ~FMetalShaderPipeline();
    
    void Init();
    void InitResourceMask(const FGraphicsPipelineStateInitializer& Init);
	void InitResourceMask(const FComputePipelineStateInitializer& Init);
    void InitResourceMask(EMetalShaderFrequency Frequency, uint8 SideTableBinding);
    
	FMetalDevice& Device;
	MTLRenderPipelineStatePtr RenderPipelineState;
    MTLComputePipelineStatePtr ComputePipelineState;
	TArray<uint32> BufferDataSizes[EMetalShaderStagesNum];
	TMap<uint8, uint8> TextureTypes[EMetalShaderStagesNum];
	FMetalShaderResourceMask ResourceMask[EMetalShaderStagesNum];
    MTLRenderPipelineReflectionPtr RenderPipelineReflection;
    MTLComputePipelineReflectionPtr ComputePipelineReflection;
#if METAL_DEBUG_OPTIONS
	NS::String* VertexSource = nullptr;
	NS::String* FragmentSource = nullptr;
	NS::String* ComputeSource = nullptr;
#if PLATFORM_SUPPORTS_MESH_SHADERS
    NS::String* MeshSource = nullptr;
    NS::String* ObjectSource = nullptr;
#endif
    MTLRenderPipelineDescriptorPtr RenderDesc;
	MTLMeshRenderPipelineDescriptorPtr MeshRenderDesc;
    MTLRenderPipelineDescriptorPtr StreamDesc;
    MTLComputePipelineDescriptorPtr ComputeDesc;
#endif
};

void ShutdownPipelineCache();

typedef TSharedPtr<FMetalShaderPipeline, ESPMode::ThreadSafe> FMetalShaderPipelinePtr;
