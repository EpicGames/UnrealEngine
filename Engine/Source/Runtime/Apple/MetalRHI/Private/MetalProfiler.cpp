// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalProfiler.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "EngineGlobals.h"
#include "StaticBoundShaderState.h"
#include "MetalCommandBuffer.h"
#include "MetalRHIContext.h"
#include "HAL/PlatformFramePacer.h"
#include "HAL/FileManager.h"

DEFINE_STAT(STAT_MetalUniformMemAlloc);
DEFINE_STAT(STAT_MetalUniformMemFreed);
DEFINE_STAT(STAT_MetalVertexMemAlloc);
DEFINE_STAT(STAT_MetalVertexMemFreed);
DEFINE_STAT(STAT_MetalIndexMemAlloc);
DEFINE_STAT(STAT_MetalIndexMemFreed);
DEFINE_STAT(STAT_MetalTextureMemUpdate);

DEFINE_STAT(STAT_MetalDrawCallTime);
DEFINE_STAT(STAT_MetalPipelineStateTime);
DEFINE_STAT(STAT_MetalPrepareDrawTime);
DEFINE_STAT(STAT_MetalSwitchToNoneTime);
DEFINE_STAT(STAT_MetalSwitchToRenderTime);
DEFINE_STAT(STAT_MetalSwitchToComputeTime);
DEFINE_STAT(STAT_MetalSwitchToBlitTime);
DEFINE_STAT(STAT_MetalPrepareToRenderTime);
DEFINE_STAT(STAT_MetalPrepareToDispatchTime);
DEFINE_STAT(STAT_MetalCommitRenderResourceTablesTime);
DEFINE_STAT(STAT_MetalSetRenderStateTime);
DEFINE_STAT(STAT_MetalSetRenderPipelineStateTime);

DEFINE_STAT(STAT_MetalMakeDrawableTime);
DEFINE_STAT(STAT_MetalBufferPageOffTime);
DEFINE_STAT(STAT_MetalTexturePageOnTime);
DEFINE_STAT(STAT_MetalTexturePageOffTime);
DEFINE_STAT(STAT_MetalCustomPresentTime);
DEFINE_STAT(STAT_MetalCommandBufferCreatedPerFrame);
DEFINE_STAT(STAT_MetalCommandBufferCommittedPerFrame);
DEFINE_STAT(STAT_MetalBufferMemory);
DEFINE_STAT(STAT_MetalTextureMemory);
DEFINE_STAT(STAT_MetalHeapMemory);
DEFINE_STAT(STAT_MetalBufferUnusedMemory);
DEFINE_STAT(STAT_MetalTextureUnusedMemory);
DEFINE_STAT(STAT_MetalBufferCount);
DEFINE_STAT(STAT_MetalTextureCount);
DEFINE_STAT(STAT_MetalHeapCount);
DEFINE_STAT(STAT_MetalFenceCount);

DEFINE_STAT(STAT_MetalUniformMemoryInFlight);
DEFINE_STAT(STAT_MetalUniformAllocatedMemory);
DEFINE_STAT(STAT_MetalUniformBytesPerFrame);

DEFINE_STAT(STAT_MetalTempAllocatorAllocatedMemory);

#if WITH_RHI_BREADCRUMBS
FMetalBreadcrumbProfiler* FMetalBreadcrumbProfiler::Instance = nullptr;
#endif

void WriteString(FArchive* OutputFile, const char* String)
{
	OutputFile->Serialize((void*)String, sizeof(ANSICHAR)*FCStringAnsi::Strlen(String));
}

#if WITH_RHI_BREADCRUMBS

void FMetalBreadcrumbProfiler::ResolveBreadcrumbTrackerStream(FMetalCommandBuffer* CmdBuffer)
{
	FScopeLock Lock(&Mutex);

	for(FMetalBreadcrumbTrackerObject& Tracker : CmdBuffer->BreadcrumbTrackerStream)
	{
		switch(Tracker.Type)
		{
			case EMetalBreadcrumbTrackerType::Begin:
			{
				OnBreadcrumbBegin(Tracker.Instance);
				break;
			}
			case EMetalBreadcrumbTrackerType::End:
			{
				OnBreadcrumbEnd(Tracker.Instance);
				break;
			}
			case EMetalBreadcrumbTrackerType::Encode:
			{
				AddSample(Tracker.CounterSample);
				break;
			}
			case EMetalBreadcrumbTrackerType::Invalid:
			default:
				// Out-of-range Type implies memory corruption, not a missed case.
				checkf(false, TEXT("FMetalBreadcrumbTrackerObject has invalid Type %u"), uint8(Tracker.Type));
				break;
		}
	}

	ResolveTopForCmdBuffer(CmdBuffer);
}

#endif
