// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"
#include "MetalCounterSampler.h"
#include "GPUProfiler.h"
#include "RHIBreadcrumbs.h"
#include "Templates/RefCounting.h"

DECLARE_DELEGATE_OneParam(FMetalCommandBufferCompletionHandler, MTL::CommandBuffer*);

// Stats
DECLARE_CYCLE_STAT_EXTERN(TEXT("MakeDrawable time"),STAT_MetalMakeDrawableTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Draw call time"),STAT_MetalDrawCallTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PrepareDraw time"),STAT_MetalPrepareDrawTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SwitchToNone time"),STAT_MetalSwitchToNoneTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SwitchToRender time"),STAT_MetalSwitchToRenderTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SwitchToCompute time"),STAT_MetalSwitchToComputeTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SwitchToBlit time"),STAT_MetalSwitchToBlitTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PrepareToRender time"),STAT_MetalPrepareToRenderTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PrepareToDispatch time"),STAT_MetalPrepareToDispatchTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CommitRenderResourceTables time"),STAT_MetalCommitRenderResourceTablesTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SetRenderState time"),STAT_MetalSetRenderStateTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SetRenderPipelineState time"),STAT_MetalSetRenderPipelineStateTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PipelineState time"),STAT_MetalPipelineStateTime,STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Buffer Page-Off time"), STAT_MetalBufferPageOffTime, STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Texture Page-Off time"), STAT_MetalTexturePageOffTime, STATGROUP_MetalRHI, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Uniform Memory Allocated Per-Frame"), STAT_MetalUniformMemAlloc, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Uniform Memory Freed Per-Frame"), STAT_MetalUniformMemFreed, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Vertex Memory Allocated Per-Frame"), STAT_MetalVertexMemAlloc, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Vertex Memory Freed Per-Frame"), STAT_MetalVertexMemFreed, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Index Memory Allocated Per-Frame"), STAT_MetalIndexMemAlloc, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Index Memory Freed Per-Frame"), STAT_MetalIndexMemFreed, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Texture Memory Updated Per-Frame"), STAT_MetalTextureMemUpdate, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Buffer Memory"), STAT_MetalBufferMemory, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture Memory"), STAT_MetalTextureMemory, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Heap Memory"), STAT_MetalHeapMemory, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Unused Buffer Memory"), STAT_MetalBufferUnusedMemory, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Unused Texture Memory"), STAT_MetalTextureUnusedMemory, STATGROUP_MetalRHI, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Uniform Memory In Flight"), STAT_MetalUniformMemoryInFlight, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Allocated Uniform Pool Memory"), STAT_MetalUniformAllocatedMemory, STATGROUP_MetalRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Uniform Memory Per Frame"), STAT_MetalUniformBytesPerFrame, STATGROUP_MetalRHI, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Allocated Frame Temp Memory"), STAT_MetalTempAllocatorAllocatedMemory, STATGROUP_MetalRHI, );

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Buffer Count"), STAT_MetalBufferCount, STATGROUP_MetalRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Texture Count"), STAT_MetalTextureCount, STATGROUP_MetalRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Heap Count"), STAT_MetalHeapCount, STATGROUP_MetalRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Fence Count"), STAT_MetalFenceCount, STATGROUP_MetalRHI, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("Texture Page-On time"), STAT_MetalTexturePageOnTime, STATGROUP_MetalRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CustomPresent time"), STAT_MetalCustomPresentTime, STATGROUP_MetalRHI, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Number Command Buffers Created Per-Frame"), STAT_MetalCommandBufferCreatedPerFrame, STATGROUP_MetalRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Number Command Buffers Committed Per-Frame"), STAT_MetalCommandBufferCommittedPerFrame, STATGROUP_MetalRHI, );

class FMetalRHICommandContext;
class FMetalSyncPoint;

#if WITH_RHI_BREADCRUMBS

enum class EMetalBreadcrumbTrackerType : uint8
{
	Invalid = 0, // sentinel: a default-constructed tracker asserts in the resolver
	Begin,
	End,
	Encode,
};

class FMetalCommandBuffer;

struct FMetalBreadcrumbInstance : public FRefCountedObject
{
	FRHIBreadcrumbNode* Node = nullptr;
	bool bWithinRenderPass = false;
	bool bTopResolved = false;
	uint64_t* TimestampTOP = nullptr;
	uint64_t* TimestampBOP = nullptr;
	TArray<FMetalCounterSamplePtr> Samples;
	FMetalCommandBuffer* BeginCmdBuffer = nullptr;
};

struct FMetalBreadcrumbTrackerObject
{
	EMetalBreadcrumbTrackerType Type = EMetalBreadcrumbTrackerType::Invalid;
	FMetalCommandBuffer* CmdBuffer = nullptr;
	FMetalCounterSamplePtr CounterSample;
	TRefCountPtr<FMetalBreadcrumbInstance> Instance;   // null for Encode
};

class FMetalBreadcrumbProfiler
{
public:
	inline static FMetalBreadcrumbProfiler* GetInstance()
	{
		if(!Instance)
		{
			Instance = new FMetalBreadcrumbProfiler;
		}
		return Instance;
	}

	~FMetalBreadcrumbProfiler()
	{
	}

	inline TRefCountPtr<FMetalBreadcrumbInstance> GetBreadcrumbEvent(FRHIBreadcrumbNode* Node, bool bWithinRenderPass)
	{
		FScopeLock Lock(&Mutex);
		TRefCountPtr<FMetalBreadcrumbInstance>& InstanceRef = CreatedBreadcrumbs.FindOrAdd(Node);
		if (!InstanceRef.IsValid())
		{
			InstanceRef = new FMetalBreadcrumbInstance;
			InstanceRef->Node = Node;
		}
		InstanceRef->bWithinRenderPass |= bWithinRenderPass;
		return InstanceRef;
	}

	inline void AddSample(FMetalCounterSamplePtr Sample)
	{
		for (const TRefCountPtr<FMetalBreadcrumbInstance>& ActiveInstance : ActiveBreadcrumbs)
		{
			ActiveInstance->Samples.Add(Sample);
		}
	}

	inline void OnBreadcrumbBegin(FMetalBreadcrumbInstance* InInstance)
	{
		ActiveBreadcrumbs.Add(InInstance);
	}

	inline void ResolveTopForCmdBuffer(FMetalCommandBuffer* CmdBuffer)
	{
		for (const TRefCountPtr<FMetalBreadcrumbInstance>& ActiveInstance : ActiveBreadcrumbs)
		{
			if (ActiveInstance->BeginCmdBuffer != CmdBuffer || ActiveInstance->bTopResolved)
			{
				continue;
			}

			if (uint64_t* Start = ActiveInstance->TimestampTOP)
			{
				if (ActiveInstance->bWithinRenderPass || ActiveInstance->Samples.Num() == 0)
				{
					*Start = 0;
				}
				else
				{
					uint64_t MinStart = 0;
					for (const FMetalCounterSamplePtr& Sample : ActiveInstance->Samples)
					{
						uint64_t StartTime, EndTime;
						Sample->ResolveStageCounters(StartTime, EndTime);

						MinStart = MinStart > 0 ? FMath::Min(StartTime, MinStart) : StartTime;
					}
					*Start = MinStart;
				}
			}
			ActiveInstance->bTopResolved = true;
		}
	}

	inline void OnBreadcrumbEnd(FMetalBreadcrumbInstance* InInstance)
	{
		uint64_t* Start = InInstance->bTopResolved ? nullptr : InInstance->TimestampTOP;
		uint64_t* End   = InInstance->TimestampBOP;

		if (InInstance->bWithinRenderPass || InInstance->Samples.Num() == 0)
		{
			if (Start) 
			{ 
				*Start = 0; 
			} 
			if (End)
			{
				*End = 0;
			}
		}
		else
		{
			for (const FMetalCounterSamplePtr& Sample : InInstance->Samples)
			{
				uint64_t StartTime, EndTime;
				Sample->ResolveStageCounters(StartTime, EndTime);
				if (Start) 
				{ 
					*Start = *Start > 0 ? FMath::Min(StartTime, *Start) : StartTime;
				}
				if (End)
				{ 
					*End = *End > 0 ? FMath::Max(EndTime, *End) : EndTime;
				}
			}
		}
		InInstance->bTopResolved = true;

		ActiveBreadcrumbs.Remove(InInstance);
		CreatedBreadcrumbs.Remove(InInstance->Node);
	}

	// Resolves all the breadcrumbs added to the command buffer
	void ResolveBreadcrumbTrackerStream(FMetalCommandBuffer* CmdBuffer);

private:
	TArray<TRefCountPtr<FMetalBreadcrumbInstance>> ActiveBreadcrumbs;

	TMap<FRHIBreadcrumbNode*, TRefCountPtr<FMetalBreadcrumbInstance>> CreatedBreadcrumbs;

	static FMetalBreadcrumbProfiler* Instance;
	FCriticalSection Mutex;
};

#endif
