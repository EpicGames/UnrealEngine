// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHI.cpp: Metal device RHI implementation.
=============================================================================*/

#include "MetalRHI.h"
#include "Apple/ScopeAutoreleasePool.h"

#include "MetalThirdParty.h"
#include "HAL/Platform.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
#include "metal_irconverter.h"
#define IR_RUNTIME_METALCPP 1
#include "metal_irconverter_runtime.h"
#endif

#include "MetalDynamicRHI.h"
#include "MetalRHIPrivate.h"
#include "MetalCommandBuffer.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "RenderUtils.h"
#if PLATFORM_IOS
#include "IOS/IOSAppDelegate.h"
#elif PLATFORM_MAC
#include "Mac/MacApplication.h"
#include "Mac/MacGPUDescriptor.h"
#include "Mac/MacPlatformMiscEx.h"
#include "HAL/PlatformApplicationMisc.h"
#include "GenericPlatform/GenericPlatformFile.h"
#endif
#include "HAL/FileManager.h"
#include "MetalProfiler.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "Math/UnrealMathUtility.h"
#include "MetalShaderResources.h"
#include "MetalLLM.h"
#include "Engine/RendererSettings.h"
#include "MetalTransitionData.h"
#include "EngineGlobals.h"
#include "MetalBindlessDescriptors.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MetalResourceCollection.h"
#include "RHICore.h"
 
#if METAL_RHI_RAYTRACING
#include "MetalRayTracing.h"
#endif

DEFINE_LOG_CATEGORY(LogMetal)

bool GIsMetalInitialized = false;

FMetalBufferFormat GMetalBufferFormats[PF_MAX];

static TAutoConsoleVariable<int32> CVarUseIOSRHIThread(
													TEXT("r.Metal.IOSRHIThread"),
													0,
													TEXT("Controls RHIThread usage for IOS:\n")
													TEXT("\t0: No RHIThread.\n")
													TEXT("\t1: Use RHIThread.\n")
													TEXT("Default is 0."),
													ECVF_Default | ECVF_RenderThreadSafe
													);

static TAutoConsoleVariable<int32> CVarMetalParallel(
													TEXT("r.Metal.Parallel"),
													PLATFORM_MAC,
													TEXT("Controls Parallel Translate support for MacOS/IOS:\n")
													TEXT("\t0: No Parallel support.\n")
													TEXT("\t1: Parallel enabled.\n")
													TEXT("Default is 1 on Mac, 0 on iOS/VisionOS."),
													ECVF_Default | ECVF_RenderThreadSafe
													);

// If precaching is active we should not need the file cache.
// however, precaching and filecache are compatible with each other, there maybe some scenarios in which both could be used.
static TAutoConsoleVariable<bool> CVarEnableMetalPSOFileCacheWhenPrecachingActive(
	TEXT("r.Metal.EnablePSOFileCacheWhenPrecachingActive"),
	false,
	TEXT("false: If precaching is available (r.PSOPrecaching=1, then disable the PSO filecache. (default)\n")
	TEXT("true: Allow both PSO file cache and precaching."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarEnableMetalDeferredDeleteLatency(
	TEXT("r.Metal.EnableMetalDeferredDeleteLatency"),
	false,
	TEXT("false: No added latency on deferred delete \n")
	TEXT("true: Extra latency on deferred delete"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

extern int32 GMetalResourcePurgeOnDelete;

static void ValidateTargetedRHIFeatureLevelExists(EShaderPlatform Platform)
{
	bool bSupportsShaderPlatform = false;
#if PLATFORM_MAC
	TArray<FString> TargetedShaderFormats;
	GConfig->GetArray(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("TargetedRHIs"), TargetedShaderFormats, GEngineIni);
	
	for (FString Name : TargetedShaderFormats)
	{
		FName ShaderFormatName(*Name);
		if (ShaderFormatToLegacyShaderPlatform(ShaderFormatName) == Platform)
		{
			bSupportsShaderPlatform = true;
			break;
		}
	}
#else
	if (Platform == SP_METAL_ES3_1_IOS || Platform == SP_METAL_ES3_1_TVOS || Platform == SP_METAL_SIM)
	{
		GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetal"), bSupportsShaderPlatform, GEngineIni);
	}
	else if (Platform == SP_METAL_SM5_IOS || Platform == SP_METAL_SM5_TVOS)
	{
		GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM5"), bSupportsShaderPlatform, GEngineIni);
	}
	else if (Platform == SP_METAL_SM6_IOS)
	{
		GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM6"), bSupportsShaderPlatform, GEngineIni);
	}
#endif
	
	if (!bSupportsShaderPlatform && !WITH_EDITOR)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ShaderPlatform"), FText::FromString(LegacyShaderPlatformToShaderFormat(Platform).ToString()));
		FText LocalizedMsg = FText::Format(NSLOCTEXT("MetalRHI", "ShaderPlatformUnavailable","Shader platform: {ShaderPlatform} was not cooked! Please enable this shader platform in the project's target settings."),Args);
		
		FText Title = NSLOCTEXT("MetalRHI", "ShaderPlatformUnavailableTitle","Shader Platform Unavailable");
		FMessageDialog::Open(EAppMsgType::Ok, LocalizedMsg, Title);
		FPlatformMisc::RequestExit(true);
		
		METAL_FATAL_ERROR(TEXT("Shader platform: %s was not cooked! Please enable this shader platform in the project's target settings."), *LegacyShaderPlatformToShaderFormat(Platform).ToString());
	}
}

#if PLATFORM_MAC && WITH_EDITOR
static void VerifyMetalCompiler()
{
	FString OutStdOut;
	FString OutStdErr;
	
	// Using xcrun or xcodebuild will fire xcode-select if xcode or command line tools are not installed
	// This will also issue a popup dialog which will attempt to install command line tools which we don't want from the Editor
	
	// xcode-select --print-path
	// Can print out /Applications/Xcode.app/Contents/Developer OR /Library/Developer/CommandLineTools
	// CommandLineTools is no good for us as the Metal compiler isn't included
	{
		int32 ReturnCode = -1;
		bool bFoundXCode = false;
		
		FPlatformProcess::ExecProcess(TEXT("/usr/bin/xcode-select"), TEXT("--print-path"), &ReturnCode, &OutStdOut, &OutStdErr);
		if(ReturnCode == 0 && OutStdOut.Len() > 0)
		{
			OutStdOut.RemoveAt(OutStdOut.Len() - 1);
			if (IFileManager::Get().DirectoryExists(*OutStdOut))
			{
				FString XcodeAppPath = OutStdOut.Left(OutStdOut.Find(TEXT(".app/")) + 4);
				NSBundle* XcodeBundle = [NSBundle bundleWithPath:XcodeAppPath.GetNSString()];
				if (XcodeBundle)
				{
					bFoundXCode = true;
				}
			}
		}
		
		if(!bFoundXCode)
		{
			FMessageDialog::Open(EAppMsgType::Ok, 
				NSLOCTEXT("MetalRHI", "XCodeMissingInstall", "Unreal Engine requires Xcode to compile shaders for Metal. To continue, install Xcode and open it to accept the license agreement. If you install Xcode to any location other than Applications/Xcode, also run the xcode-select command-line tool to specify its location."), 
				NSLOCTEXT("MetalRHI", "XCodeMissingInstallTitle", "Xcode Not Found"));
			FPlatformMisc::RequestExit(true);
			return;
		}
	}
	
	// xcodebuild -license check
	// -license check :returns 0 for accepted, otherwise 1 for command line tools or non zero for license not accepted
	// -checkFirstLaunchStatus | -runFirstLaunch : returns status and runs first launch not so useful from within the editor as sudo is required
	{
		int ReturnCode = -1;
		FPlatformProcess::ExecProcess(TEXT("/usr/bin/xcodebuild"), TEXT("-license check"), &ReturnCode, &OutStdOut, &OutStdErr);
		if(ReturnCode != 0)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("MetalRHI", "XCodeLicenseAgreement", "Xcode license agreement error: {0}"), FText::FromString(OutStdErr)));
			FPlatformMisc::RequestExit(true);
			return;
		}
	}
	
	
	// xcrun will return non zero if using command line tools
	// This can fail for license agreement as well or wrong command line tools set i.e set to /Library/Developer/CommandLineTools rather than Applications/Xcode.app/Contents/Developer
	{
		int ReturnCode = -1;
		FPlatformProcess::ExecProcess(TEXT("/usr/bin/xcrun"), TEXT("-sdk macosx metal -v"), &ReturnCode, &OutStdOut, &OutStdErr);
		if(ReturnCode != 0)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(NSLOCTEXT("MetalRHI", "XCodeMetalCompiler", "Xcode Metal Compiler error: {0}"), FText::FromString(OutStdErr)));
			FPlatformMisc::RequestExit(true);
			return;
		}
	}
}
#endif

FMetalDynamicRHI::FMetalDynamicRHI(ERHIFeatureLevel::Type RequestedFeatureLevel)
	: Device(FMetalDevice::CreateDevice())
{
	check(Singleton == nullptr);
	Singleton = this;

    MTL_SCOPED_AUTORELEASE_POOL;
    
	// This should be called once at the start 
	check( IsInGameThread() );
	check( !GIsThreadedRendering );
	
#if PLATFORM_MAC && WITH_EDITOR
	VerifyMetalCompiler();
#endif
	
	GRHISupportsMultithreading = true;
	GRHISupportsMultithreadedResources = true;
	GRHISupportsAsyncGetRenderQueryResult = true;
	GRHISupportsGPUTimestampBubblesRemoval = true;
	
	// we cannot render to a volume texture without geometry shader or vertex-shader-layer support, so initialise to false and enable based on platform feature availability
	GSupportsVolumeTextureRendering = false;
	
	// Metal always needs a render target to render with fragment shaders!
	GRHIRequiresRenderTargetForPixelShaderUAVs = true;

	GRHIAdapterName = NSStringToFString(Device->GetDevice()->name());
	GRHIVendorId = 1; // non-zero to avoid asserts

	bool const bRequestedFeatureLevel = (RequestedFeatureLevel != ERHIFeatureLevel::Num);
	bool bSupportsPointLights = false;
	
	// get the device to ask about capabilities?
	MTL::Device* MTLDevice = Device->GetDevice();
		
	// Use D24S8 if supported, else D32S8
	bool bSupportsD24S8 = false;
	
#if PLATFORM_MAC
	bSupportsD24S8 = MTLDevice->depth24Stencil8PixelFormatSupported();
#endif
	
	bool bSupportsSM6 = false;
	bool bSupportsBindless = false;
	
	// Only MacOS 15.0+ can use SM6 with MSC
	if (@available(macOS 15.0, iOS 18.0, *))
	{
		// A M2+ Mac Apple GPU is required for SM6 Apple8, on iOS we need Apple9 or above
#if PLATFORM_MAC
		bSupportsSM6 = MTLDevice->supportsFamily(MTL::GPUFamilyApple8);
#else
		bSupportsSM6 = MTLDevice->supportsFamily(MTL::GPUFamilyApple9);
#endif
		bSupportsBindless = MTLDevice->supportsFamily(MTL::GPUFamilyApple7); 
	}
	
#if PLATFORM_IOS
	if (!MTLDevice->supportsFamily(MTL::GPUFamilyApple3))
	{
		static const FText UnsupportedGPUFamilyApple9 = NSLOCTEXT("MetalRHI", "UnsupportedA8FeatureSet", "This device does not support the Apple family 3 GPU features (Apple A9+), which is required as the minimum for this engine version.");
		UE_LOGF(LogMetal, Log, "%ls", *UnsupportedGPUFamilyApple9.ToString());
		FMessageDialog::Open(EAppMsgType::Ok, UnsupportedGPUFamilyApple9); 
		FPlatformMisc::RequestExit(true, TEXT("FMetalDynamicRHI::FMetalDynamicRHI.UnsupportedA8FeatureSet"));
	}
        
	GRHISupportsDrawIndirect = true;
	GRHISupportsPixelShaderUAVs = true;
	GRHISupportsRWTextureBuffers = MTLDevice->supportsFamily(MTL::GPUFamilyApple4);

	const MTL::FeatureSet FeatureSets[] = {
        MTL::FeatureSet_iOS_GPUFamily1_v1,
        MTL::FeatureSet_iOS_GPUFamily2_v1,
        MTL::FeatureSet_iOS_GPUFamily3_v1,
        MTL::FeatureSet_iOS_GPUFamily4_v1
	};
		
	const uint8 FeatureSetVersions[][3] = {
		{8, 0, 0},
		{8, 3, 0},
		{10, 0, 0},
		{11, 0, 0}
	};
	
	GRHIDeviceId = 0;
	for (uint32 i = 0; i < 4; i++)
	{
		if (FPlatformMisc::IOSVersionCompare(FeatureSetVersions[i][0],FeatureSetVersions[i][1],FeatureSetVersions[i][2]) >= 0 && MTLDevice->supportsFeatureSet(FeatureSets[i]))
		{
			GRHIDeviceId++;
		}
	}
		
	GSupportsVolumeTextureRendering = Device->SupportsFeature(EMetalFeaturesLayeredRendering);
	bSupportsPointLights = GSupportsVolumeTextureRendering;
        
	bool bProjectSupportsMetalMobileSM5 = false;
	bool bProjectSupportsMetalMobileSM6 = false;
	bool bProjectSupportsMetal          = false;

	static const TCHAR* const IOSSettingsSection = TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings");
	GConfig->GetBool(IOSSettingsSection, TEXT("bSupportsMetalMobileSM5"), bProjectSupportsMetalMobileSM5, GEngineIni);
	GConfig->GetBool(IOSSettingsSection, TEXT("bSupportsMetalMobileSM6"), bProjectSupportsMetalMobileSM6, GEngineIni);
	GConfig->GetBool(IOSSettingsSection, TEXT("bSupportsMetal"),          bProjectSupportsMetal,          GEngineIni);

	const bool bRequestedSM6 = (RequestedFeatureLevel == ERHIFeatureLevel::SM6) || (!bRequestedFeatureLevel && FParse::Param(FCommandLine::Get(), TEXT("metalmobilesm6")));
	const bool bRequestedSM5 = (RequestedFeatureLevel == ERHIFeatureLevel::SM5) || (!bRequestedFeatureLevel && FParse::Param(FCommandLine::Get(), TEXT("metalmobilesm5")));
	const bool bForceES3_1   = FParse::Param(FCommandLine::Get(), TEXT("es31"));

	// Resolve the concrete shader platform for each feature level on this compile target.
	#if PLATFORM_TVOS
		constexpr EShaderPlatform SM6Platform  = SP_NumPlatforms;       // not supported on tvOS
		constexpr EShaderPlatform SM5Platform  = SP_METAL_SM5_TVOS;
		constexpr EShaderPlatform ES31Platform = SP_METAL_ES3_1_TVOS;
	#elif WITH_IOS_SIMULATOR
		constexpr EShaderPlatform SM6Platform  = SP_NumPlatforms;       // not supported on simulator
		constexpr EShaderPlatform SM5Platform  = SP_NumPlatforms;       // not supported on simulator
		constexpr EShaderPlatform ES31Platform = SP_METAL_SIM;
	#else
		constexpr EShaderPlatform SM6Platform  = SP_METAL_SM6_IOS;
		constexpr EShaderPlatform SM5Platform  = SP_METAL_SM5_IOS;
		constexpr EShaderPlatform ES31Platform = SP_METAL_ES3_1_IOS;
	#endif

	// Select the highest supported feature level, in priority order.
	if (SM6Platform != SP_NumPlatforms && bProjectSupportsMetalMobileSM6 && bRequestedSM6 && !bForceES3_1 && bSupportsSM6)
	{
		ValidateTargetedRHIFeatureLevelExists(SM6Platform);
		GMaxRHIShaderPlatform = SM6Platform;
		GMaxRHIFeatureLevel   = ERHIFeatureLevel::SM6;
	}
	else if (SM5Platform != SP_NumPlatforms && bProjectSupportsMetalMobileSM5 && bRequestedSM5 && !bForceES3_1
		&& (UE::RHICore::GetBindlessConfigurationOnStartup(SM5Platform) == ERHIBindlessConfiguration::Disabled || bSupportsBindless))
	{
		ValidateTargetedRHIFeatureLevelExists(SM5Platform);
		GMaxRHIShaderPlatform = SM5Platform;
		GMaxRHIFeatureLevel   = ERHIFeatureLevel::SM5;
	}
	else if (bProjectSupportsMetal
		&& (UE::RHICore::GetBindlessConfigurationOnStartup(ES31Platform) == ERHIBindlessConfiguration::Disabled || bSupportsBindless))
	{
		ValidateTargetedRHIFeatureLevelExists(ES31Platform);
		GMaxRHIShaderPlatform = ES31Platform;
		GMaxRHIFeatureLevel   = ERHIFeatureLevel::ES3_1;
	}
	else
	{
		static const FText NoCompatibleMetalFormats = NSLOCTEXT("MetalRHI", "NoCompatibleMetalFormats", "This project has been built with project settings that are unsupported by this device");
		UE_LOGF(LogMetal, Log, "%ls", *NoCompatibleMetalFormats.ToString());
		FMessageDialog::Open(EAppMsgType::Ok, NoCompatibleMetalFormats);
		FPlatformMisc::RequestExit(true, TEXT("FMetalDynamicRHI::FMetalDynamicRHI.NoCompatibleMetalFormats"));
	}

	#if USE_STATIC_SHADER_PLATFORM_ENUMS
		GMaxRHIShaderPlatform = UE_IOS_STATIC_SHADER_PLATFORM;
	#endif
	#if	USE_STATIC_FEATURE_LEVEL_ENUMS
		GMaxRHIFeatureLevel = UE_IOS_STATIC_FEATURE_LEVEL;
	#endif
		
	FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
		
	MemoryStats.DedicatedVideoMemory = 0;
	MemoryStats.TotalGraphicsMemory = Stats.AvailablePhysical;
	MemoryStats.DedicatedSystemMemory = 0;
	MemoryStats.SharedSystemMemory = Stats.AvailablePhysical;
	
#if PLATFORM_TVOS
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_METAL_ES3_1_TVOS;
#else
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
#if WITH_IOS_SIMULATOR
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_METAL_SIM;
#else
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_METAL_ES3_1_IOS;
#endif
#endif
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5) ? GMaxRHIShaderPlatform : SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6) ? GMaxRHIShaderPlatform : SP_NumPlatforms;

#else // PLATFORM_IOS
                	
	TArray<FMacPlatformMisc::FGPUDescriptor> const& GPUs = FPlatformMisc::GetGPUDescriptors();
	check(GPUs.Num());
	FMacPlatformMisc::FGPUDescriptor const& GPUDesc = GPUs[0];
	
	bool bSupportsD16 = false;
	
	GRHIAdapterName = NSStringToFString(MTLDevice->name());
	
	GSupportsTimestampRenderQueries = true;

	// At this point this should be valid because we checked it while creating the FMetalDevice
	check(MTLDevice->supportsFamily(MTL::GPUFamilyApple7));
	
	bSupportsPointLights = true;
	GRHIVendorId = (uint32)EGpuVendorId::Apple;
	GSupportsTimestampRenderQueries = true;
	GSupportsParallelOcclusionQueries = true;

    bool const bRequestedSM6 = RequestedFeatureLevel == ERHIFeatureLevel::SM6 ||
                               (!bRequestedFeatureLevel && (FParse::Param(FCommandLine::Get(),TEXT("metalsm6"))));
        
	bool const bRequestedSM5 = RequestedFeatureLevel == ERHIFeatureLevel::SM5 ||
                               (!bRequestedFeatureLevel && FParse::Param(FCommandLine::Get(),TEXT("metalsm5")));
                                
	if(bRequestedSM6 && !bSupportsSM6)
	{
		if(!GRHIAdapterName.Contains("M1"))
		{
			UE_LOGF(LogMetal, Warning, "SM6 is enabled but is not supported on this system, falling back to SM5");
		}
		else
		{
			UE_LOGF(LogMetal, Warning, "To use SM6 on this system, please ensure you are running Mac OS 15. Falling back to SM5");
		}
	}
	
    if(bSupportsSM6 && bRequestedSM6)
    {
        GMaxRHIFeatureLevel = ERHIFeatureLevel::SM6;
        GMaxRHIShaderPlatform = SP_METAL_SM6;
		
		GRHIGlobals.SupportsNative16BitOps = true;
    }
	else
	{
		GMaxRHIFeatureLevel = ERHIFeatureLevel::SM5;
		GMaxRHIShaderPlatform = SP_METAL_SM5;
	}
	
	// Metal always uses AIR. This is used to differentiate standard and preview shaders in the DDC.
	GRHIGlobals.PreferredPreviewShaderCodeFormat = TEXT("AIR");

#if PLATFORM_SUPPORTS_MESH_SHADERS
    GRHISupportsMeshShadersTier0 = RHISupportsMeshShadersTier0(GMaxRHIShaderPlatform);
    GRHISupportsMeshShadersTier1 = RHISupportsMeshShadersTier1(GMaxRHIShaderPlatform);
#endif

	ERHIFeatureLevel::Type PreviewFeatureLevel;
	if (RHIGetPreviewFeatureLevel(PreviewFeatureLevel))
	{
		check(PreviewFeatureLevel == ERHIFeatureLevel::ES3_1);

		// ES3.1 feature level emulation
		GMaxRHIFeatureLevel = PreviewFeatureLevel;
		if (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1)
		{
			GMaxRHIShaderPlatform = SP_METAL_ES3_1;
		}
	}
	
	ValidateTargetedRHIFeatureLevelExists(GMaxRHIShaderPlatform);
	
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = (GMaxRHIFeatureLevel >= ERHIFeatureLevel::ES3_1) ? SP_METAL_ES3_1 : SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5) ? GMaxRHIShaderPlatform : SP_NumPlatforms;
    GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM6) ? GMaxRHIShaderPlatform : SP_NumPlatforms;
	
	// Mac GPUs support layer indexing.
	GSupportsVolumeTextureRendering = true;
	bSupportsPointLights = true;
	
	// Make sure the vendors match - the assumption that order in IORegistry is the order in Metal may not hold up forever.
	if(GPUDesc.GPUVendorId == GRHIVendorId)
	{
		GRHIDeviceId = GPUDesc.GPUDeviceId;
		MemoryStats.DedicatedVideoMemory = (int64)GPUDesc.GPUMemoryMB * 1024 * 1024;
		MemoryStats.TotalGraphicsMemory = (int64)GPUDesc.GPUMemoryMB * 1024 * 1024;
		MemoryStats.DedicatedSystemMemory = 0;
		MemoryStats.SharedSystemMemory = 0;
	}	
#endif
		
	GRHIGlobals.bSupportsBindless = UE::RHICore::GetBindlessConfigurationOnStartup(GMaxRHIShaderPlatform) != ERHIBindlessConfiguration::Disabled;
	GRHIGlobals.MaxSimultaneousUAVs = FMath::Min(METAL_MAX_TEXTURES, METAL_MAX_BUFFERS);
	GRHIGlobals.MaxTextureSamplers = GRHIGlobals.bSupportsBindless ? MTLDevice->maxArgumentBufferSamplerCount() : 16;
	
	GRHIMinimumWaveSize = 32;
	GRHIMaximumWaveSize = 32;
	
	if (GRHIGlobals.bSupportsBindless)
	{
		GRHISupportsWaveOperations = true;	
		Device->ForceHeapSupport();
	}
	
	// Must call post init after we have set flags on device
	Device->PostInit();
	
	if (bSupportsSM6)
	{
		// Int64 atomic support was introduced with M2 devices.
		GRHISupportsAtomicUInt64 = true;
		GRHIPersistentThreadGroupCount = 1024;
	
		bSupportsPointLights = true;
		
		// Disable persistent threads on Apple Silicon (as it doesn't support forward progress guarantee).
		IConsoleVariable* NanitePersistentThreadCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.PersistentThreadsCulling"));
		if (NanitePersistentThreadCVar != nullptr && NanitePersistentThreadCVar->GetInt() == 1)
		{
			NanitePersistentThreadCVar->Set(0);
		}
	}
	else
	{
		IConsoleVariable* LumenLightingFormatCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Lumen.LightingDataFormat"));
		if (LumenLightingFormatCVar != nullptr)
		{
			// If we are not running on an SM6 compatible device we need to use an RGBA16 format for Lumen
			LumenLightingFormatCVar->Set(1);
		}
	}
	
	GRHISupportsDynamicResolution = true;
	GRHISupportsFrameCyclesBubblesRemoval = true;

	GPoolSizeVRAMPercentage = 0;
	GTexturePoolSize = 0;
	GConfig->GetInt(TEXT("TextureStreaming"), TEXT("PoolSizeVRAMPercentage"), GPoolSizeVRAMPercentage, GEngineIni);
	if ( GPoolSizeVRAMPercentage > 0 && MemoryStats.TotalGraphicsMemory > 0 )
	{
		float PoolSize = float(GPoolSizeVRAMPercentage) * 0.01f * float(MemoryStats.TotalGraphicsMemory);
		
		// Truncate GTexturePoolSize to MB (but still counted in bytes)
		GTexturePoolSize = int64(FGenericPlatformMath::TruncToFloat(PoolSize / 1024.0f / 1024.0f)) * 1024 * 1024;
		
		UE_LOGF(LogRHI,Log,"Texture pool is %llu MB (%d%% of %llu MB)",
			   GTexturePoolSize / 1024 / 1024,
			   GPoolSizeVRAMPercentage,
			   MemoryStats.TotalGraphicsMemory / 1024 / 1024);
	}
	else
	{
		static const auto CVarStreamingTexturePoolSize = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Streaming.PoolSize"));
		GTexturePoolSize = (int64)CVarStreamingTexturePoolSize->GetValueOnAnyThread() * 1024 * 1024;

		UE_LOGF(LogRHI,Log,"Texture pool is %llu MB (of %llu MB total graphics mem)",
			   GTexturePoolSize / 1024 / 1024,
			   MemoryStats.TotalGraphicsMemory / 1024 / 1024);
	}

	GRHITransitionPrivateData_SizeInBytes = sizeof(FMetalTransitionData);
	GRHITransitionPrivateData_AlignInBytes = alignof(FMetalTransitionData);

	GRHISupportsRHIThread = false;
	if (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5)
	{
		GRHISupportsRHIThread = true;
	}
	else
	{
		GRHISupportsRHIThread = FParse::Param(FCommandLine::Get(),TEXT("rhithread")) || (CVarUseIOSRHIThread.GetValueOnAnyThread() > 0);
	}
	
	bool bSupportsParallel = CVarMetalParallel.GetValueOnAnyThread() || FParse::Param(FCommandLine::Get(),TEXT("rhiparallel"));
	
	GRHISupportsParallelRHIExecute = bSupportsParallel;
	GRHIGlobals.ParallelRenderPasses.Supported = bSupportsParallel;
	GRHIGlobals.ParallelRenderPasses.ParentWaitForChildTranslate = true;
	GRHIGlobals.ParallelRenderPasses.ChildWaitForParentBegin = true;
	GRHIGlobals.ParallelRenderPasses.UsesCustomContexts = true;
	
	if (FPlatformMisc::IsDebuggerPresent() && UE_BUILD_DEBUG)
	{
#if PLATFORM_IOS // @todo zebra : needs a RENDER_API or whatever
		// Enable debug markers if we're running in Xcode
		extern int32 GEmitMeshDrawEvent;
		GEmitMeshDrawEvent = 1;
#endif
		SetEmitDrawEvents(true);
	}
	
	// Force disable vertex-shader-layer point light rendering on GPUs that don't support it properly yet.
	if(!bSupportsPointLights && !FParse::Param(FCommandLine::Get(),TEXT("metalpointlights")))
	{
		// Disable point light cubemap shadows on Mac Metal as currently they aren't supported.
		static auto CVarCubemapShadows = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AllowPointLightCubemapShadows"));
		if(CVarCubemapShadows && CVarCubemapShadows->GetInt() != 0)
		{
			CVarCubemapShadows->Set(0);
		}
	}
	
	if (!GSupportsVolumeTextureRendering && !FParse::Param(FCommandLine::Get(),TEXT("metaltlv")))
	{
		// Disable point light cubemap shadows on Mac Metal as currently they aren't supported.
		static auto CVarTranslucencyLightingVolume = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TranslucencyLightingVolume"));
		if(CVarTranslucencyLightingVolume && CVarTranslucencyLightingVolume->GetInt() != 0)
		{
			CVarTranslucencyLightingVolume->Set(0);
		}
	}

#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	// we don't want to auto-enable draw events in Test
	SetEmitDrawEvents(GetEmitDrawEvents() | ENABLE_METAL_GPUEVENTS);
#endif

	GSupportsShaderFramebufferFetch = !PLATFORM_MAC && GMaxRHIShaderPlatform != SP_METAL_SM5_IOS && GMaxRHIShaderPlatform != SP_METAL_SM6_IOS && GMaxRHIShaderPlatform != SP_METAL_SM5_TVOS;
	GSupportsShaderMRTFramebufferFetch = GSupportsShaderFramebufferFetch;
	GHardwareHiddenSurfaceRemoval = true;
	GSupportsRenderTargetFormat_PF_G8 = false;
	GRHISupportsTextureStreaming = true;
	GSupportsWideMRT = true;
	GSupportsSeparateRenderTargetBlendState = (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5);

	GRHISupportsPSOPrecaching = true;
	GRHISupportsPipelineFileCache = !GRHISupportsPSOPrecaching || CVarEnableMetalPSOFileCacheWhenPrecachingActive.GetValueOnAnyThread();
	GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer = MTLDevice->maxBufferLength();
	GRHIGlobals.MaxViewDimensionForTypedBuffer = 1 << 28;

#if PLATFORM_MAC
	GRHISupportsBaseVertexIndex = true;
	GRHISupportsFirstInstance = true; // Supported on macOS & iOS but not tvOS.
	GMaxTextureDimensions = 16384;
	GMaxCubeTextureDimensions = 16384;
	GMaxTextureArrayLayers = 2048;
	GMaxShadowDepthBufferSizeX = GMaxTextureDimensions;
	GMaxShadowDepthBufferSizeY = GMaxTextureDimensions;
    bSupportsD16 = true;
    GRHISupportsHDROutput = true;
	GRHIHDRDisplayOutputFormat = (GRHISupportsHDROutput) ? PF_PLATFORM_HDR_0 : PF_B8G8R8A8;
	// Based on the spec below, the maxTotalThreadsPerThreadgroup is not a fixed number but calculated according to the device current ability, so the available threads could less than the maximum number.
	// For safety and keep the consistency for all platform, reduce the maximum number to half of the device based.
	// https://developer.apple.com/documentation/metal/mtlcomputepipelinedescriptor/2966560-maxtotalthreadsperthreadgroup?language=objc
	GMaxWorkGroupInvocations = 512;
#else
	//@todo investigate gpufam4
	GMaxComputeSharedMemory = 1 << 14;
#if PLATFORM_TVOS
	GRHISupportsBaseVertexIndex = false;
	GRHISupportsFirstInstance = false; // Supported on macOS & iOS but not tvOS.
	GRHISupportsHDROutput = false;
	GRHIHDRDisplayOutputFormat = PF_B8G8R8A8; // must have a default value for non-hdr, just like mac or ios
#elif PLATFORM_VISIONOS
	GRHISupportsBaseVertexIndex = true;
	GRHISupportsFirstInstance = GRHISupportsBaseVertexIndex;
	GRHISupportsHDROutput = true;
	GRHIHDRDisplayOutputFormat = (GRHISupportsHDROutput) ? PF_PLATFORM_HDR_0 : PF_B8G8R8A8;
	GMaxWorkGroupInvocations = 512;
#else
	GRHISupportsBaseVertexIndex = true;
	GRHISupportsFirstInstance = true;
	
	// TODO: Move this into IOSPlatform
    {
        MTL_SCOPED_AUTORELEASE_POOL;
        UIScreen* mainScreen = [UIScreen mainScreen];
        UIDisplayGamut gamut = mainScreen.traitCollection.displayGamut;
        GRHISupportsHDROutput = FPlatformMisc::IOSVersionCompare(10, 0, 0) && gamut == UIDisplayGamutP3;
    }
	
	GRHIHDRDisplayOutputFormat = (GRHISupportsHDROutput) ? PF_PLATFORM_HDR_0 : PF_B8G8R8A8;
	// Based on the spec below, the maxTotalThreadsPerThreadgroup is not a fixed number but calculated according to the device current ability, so the available threads could less than the maximum number.
	// For safety and keep the consistency for all platform, reduce the maximum number to half of the device based.
	// https://developer.apple.com/documentation/metal/mtlcomputepipelinedescriptor/2966560-maxtotalthreadsperthreadgroup?language=objc
	GMaxWorkGroupInvocations = MTLDevice->supportsFeatureSet(MTL::FeatureSet_iOS_GPUFamily4_v1) ? 512 : 256;
#endif
	GMaxTextureDimensions = 8192;
	GMaxCubeTextureDimensions = 8192;
	GMaxTextureArrayLayers = 2048;
	GMaxShadowDepthBufferSizeX = GMaxTextureDimensions;
	GMaxShadowDepthBufferSizeY = GMaxTextureDimensions;
#endif

    if(MTLDevice->supportsFamily(MTL::GPUFamilyApple6) ||
	   MTLDevice->supportsFamily(MTL::GPUFamilyMac2))
    {
        GRHISupportsArrayIndexFromAnyShader = true;
    }
            
	GRHIMaxDispatchThreadGroupsPerDimension.X = MAX_int32;
	GRHIMaxDispatchThreadGroupsPerDimension.Y = MAX_int32;
	GRHIMaxDispatchThreadGroupsPerDimension.Z = MAX_int32;

	GMaxTextureMipCount = FPlatformMath::CeilLogTwo( GMaxTextureDimensions ) + 1;
	GMaxTextureMipCount = FPlatformMath::Min<int32>( MAX_TEXTURE_MIP_COUNT, GMaxTextureMipCount );

	// Initialize the buffer format map - in such a way as to be able to validate it in non-shipping...
#if METAL_DEBUG_OPTIONS
	FMemory::Memset(GMetalBufferFormats, 255);
#endif
	GMetalBufferFormats[PF_Unknown              ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_A32B32G32R32F        ] = { MTL::PixelFormatRGBA32Float, (uint8)EMetalBufferFormat::RGBA32Float };
	GMetalBufferFormats[PF_B8G8R8A8             ] = { MTL::PixelFormatRGBA8Unorm, (uint8)EMetalBufferFormat::RGBA8Unorm }; // MTL::PixelFormatBGRA8Unorm/EMetalBufferFormat::BGRA8Unorm,  < We don't support this as a vertex-format so we have code to swizzle in the shader
	GMetalBufferFormats[PF_G8                   ] = { MTL::PixelFormatR8Unorm, (uint8)EMetalBufferFormat::R8Unorm };
	GMetalBufferFormats[PF_G16                  ] = { MTL::PixelFormatR16Unorm, (uint8)EMetalBufferFormat::R16Unorm };
	GMetalBufferFormats[PF_DXT1                 ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_DXT3                 ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_DXT5                 ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_UYVY                 ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_FloatRGB        	] = { MTL::PixelFormatRG11B10Float, (uint8)EMetalBufferFormat::RG11B10Half };
	GMetalBufferFormats[PF_FloatRGBA            ] = { MTL::PixelFormatRGBA16Float, (uint8)EMetalBufferFormat::RGBA16Half };
	GMetalBufferFormats[PF_DepthStencil         ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ShadowDepth          ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R32_FLOAT            ] = { MTL::PixelFormatR32Float, (uint8)EMetalBufferFormat::R32Float };
	GMetalBufferFormats[PF_G16R16               ] = { MTL::PixelFormatRG16Unorm, (uint8)EMetalBufferFormat::RG16Unorm };
	GMetalBufferFormats[PF_G16R16F              ] = { MTL::PixelFormatRG16Float, (uint8)EMetalBufferFormat::RG16Half };
	GMetalBufferFormats[PF_G16R16F_FILTER       ] = { MTL::PixelFormatRG16Float, (uint8)EMetalBufferFormat::RG16Half };
	GMetalBufferFormats[PF_G32R32F              ] = { MTL::PixelFormatRG32Float, (uint8)EMetalBufferFormat::RG32Float };
	GMetalBufferFormats[PF_A2B10G10R10          ] = { MTL::PixelFormatRGB10A2Unorm, (uint8)EMetalBufferFormat::RGB10A2Unorm };
	GMetalBufferFormats[PF_A16B16G16R16         ] = { MTL::PixelFormatRGBA16Unorm, (uint8)EMetalBufferFormat::RGBA16Half };
	GMetalBufferFormats[PF_D24                  ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R16F                 ] = { MTL::PixelFormatR16Float, (uint8)EMetalBufferFormat::RG16Half };
	GMetalBufferFormats[PF_R16F_FILTER          ] = { MTL::PixelFormatR16Float, (uint8)EMetalBufferFormat::RG16Half };
	GMetalBufferFormats[PF_BC5                  ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_V8U8                 ] = { MTL::PixelFormatRG8Snorm, (uint8)EMetalBufferFormat::RG8Unorm };
	GMetalBufferFormats[PF_A1                   ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_FloatR11G11B10       ] = { MTL::PixelFormatRG11B10Float, (uint8)EMetalBufferFormat::RG11B10Half }; // < May not work on tvOS
	GMetalBufferFormats[PF_A8                   ] = { MTL::PixelFormatA8Unorm, (uint8)EMetalBufferFormat::R8Unorm };
	GMetalBufferFormats[PF_R32_UINT             ] = { MTL::PixelFormatR32Uint, (uint8)EMetalBufferFormat::R32Uint };
	GMetalBufferFormats[PF_R32_SINT             ] = { MTL::PixelFormatR32Sint, (uint8)EMetalBufferFormat::R32Sint };
	GMetalBufferFormats[PF_PVRTC2               ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_PVRTC4               ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R16_UINT             ] = { MTL::PixelFormatR16Uint, (uint8)EMetalBufferFormat::R16Uint };
	GMetalBufferFormats[PF_R16_SINT             ] = { MTL::PixelFormatR16Sint, (uint8)EMetalBufferFormat::R16Sint };
	GMetalBufferFormats[PF_R16G16B16A16_UINT    ] = { MTL::PixelFormatRGBA16Uint, (uint8)EMetalBufferFormat::RGBA16Uint };
	GMetalBufferFormats[PF_R16G16B16A16_SINT    ] = { MTL::PixelFormatRGBA16Sint, (uint8)EMetalBufferFormat::RGBA16Sint };
	GMetalBufferFormats[PF_R5G6B5_UNORM         ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::R5G6B5Unorm };
	GMetalBufferFormats[PF_B5G5R5A1_UNORM       ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::B5G5R5A1Unorm };
	GMetalBufferFormats[PF_R8G8B8A8             ] = { MTL::PixelFormatRGBA8Unorm, (uint8)EMetalBufferFormat::RGBA8Unorm };
	GMetalBufferFormats[PF_A8R8G8B8				] = { MTL::PixelFormatRGBA8Unorm, (uint8)EMetalBufferFormat::RGBA8Unorm }; // MTL::PixelFormatBGRA8Unorm/EMetalBufferFormat::BGRA8Unorm,  < We don't support this as a vertex-format so we have code to swizzle in the shader
	GMetalBufferFormats[PF_BC4					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R8G8                 ] = { MTL::PixelFormatRG8Unorm, (uint8)EMetalBufferFormat::RG8Unorm };
	GMetalBufferFormats[PF_ATC_RGB				] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ATC_RGBA_E			] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ATC_RGBA_I			] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_X24_G8				] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ETC1					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ETC2_RGB				] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ETC2_RGBA			] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R32G32B32A32_UINT	] = { MTL::PixelFormatRGBA32Uint, (uint8)EMetalBufferFormat::RGBA32Uint };
	GMetalBufferFormats[PF_R16G16_UINT			] = { MTL::PixelFormatRG16Uint, (uint8)EMetalBufferFormat::RG16Uint };
	GMetalBufferFormats[PF_R16G16_SINT			] = { MTL::PixelFormatRG16Sint, (uint8)EMetalBufferFormat::RG16Sint };
	GMetalBufferFormats[PF_R32G32_UINT			] = { MTL::PixelFormatRG32Uint, (uint8)EMetalBufferFormat::RG32Uint };
	GMetalBufferFormats[PF_ASTC_4x4             ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_6x6             ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_8x8             ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_10x10           ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_12x12           ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_4x4_HDR         ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_6x6_HDR         ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_8x8_HDR         ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_10x10_HDR       ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_12x12_HDR       ] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_BC6H					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_BC7					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R8_UINT				] = { MTL::PixelFormatR8Uint, (uint8)EMetalBufferFormat::R8Uint };
	GMetalBufferFormats[PF_R8					] = { MTL::PixelFormatR8Unorm, (uint8)EMetalBufferFormat::R8Unorm };
	GMetalBufferFormats[PF_L8					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::R8Unorm };
	GMetalBufferFormats[PF_XGXR8				] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R8G8B8A8_UINT		] = { MTL::PixelFormatRGBA8Uint, (uint8)EMetalBufferFormat::RGBA8Uint };
	GMetalBufferFormats[PF_R8G8B8A8_SNORM		] = { MTL::PixelFormatRGBA8Snorm, (uint8)EMetalBufferFormat::RGBA8Snorm };
	GMetalBufferFormats[PF_R16G16B16A16_UNORM	] = { MTL::PixelFormatRGBA16Unorm, (uint8)EMetalBufferFormat::RGBA16Unorm };
	GMetalBufferFormats[PF_R16G16B16A16_SNORM	] = { MTL::PixelFormatRGBA16Snorm, (uint8)EMetalBufferFormat::RGBA16Snorm };
	GMetalBufferFormats[PF_PLATFORM_HDR_0		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_PLATFORM_HDR_1		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_PLATFORM_HDR_2		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_NV12					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	
	GMetalBufferFormats[PF_ETC2_R11_EAC			] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ETC2_RG11_EAC		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
		
	GMetalBufferFormats[PF_G16R16_SNORM			] = { MTL::PixelFormatRG16Snorm, (uint8)EMetalBufferFormat::RG16Snorm };
	GMetalBufferFormats[PF_R8G8_UINT			] = { MTL::PixelFormatRG8Uint, (uint8)EMetalBufferFormat::RG8Uint };
	GMetalBufferFormats[PF_R32G32B32_UINT		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R32G32B32_SINT		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R32G32B32F			] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R8_SINT				] = { MTL::PixelFormatR8Sint, (uint8)EMetalBufferFormat::R8Sint };
	GMetalBufferFormats[PF_R64_UINT				] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R9G9B9EXP5			] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_P010					] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_4x4_NORM_RG		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_6x6_NORM_RG		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_8x8_NORM_RG		] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_10x10_NORM_RG	] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_ASTC_12x12_NORM_RG	] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	GMetalBufferFormats[PF_R8G8B8				] = { MTL::PixelFormatInvalid, (uint8)EMetalBufferFormat::Unknown };
	static_assert(PF_MAX == 94, "Please setup GMetalBufferFormats properly for the new pixel format");

	// Initialize the platform pixel format map.
	GPixelFormats[PF_Unknown			].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_A32B32G32R32F		].PlatformFormat	= (uint32)MTL::PixelFormatRGBA32Float;
	GPixelFormats[PF_B8G8R8A8			].PlatformFormat	= (uint32)MTL::PixelFormatBGRA8Unorm;
	GPixelFormats[PF_G8					].PlatformFormat	= (uint32)MTL::PixelFormatR8Unorm;
	GPixelFormats[PF_G16				].PlatformFormat	= (uint32)MTL::PixelFormatR16Unorm;
	GPixelFormats[PF_R32G32B32A32_UINT	].PlatformFormat	= (uint32)MTL::PixelFormatRGBA32Uint;
	GPixelFormats[PF_R16G16_UINT		].PlatformFormat	= (uint32)MTL::PixelFormatRG16Uint;
	GPixelFormats[PF_R16G16_SINT		].PlatformFormat	= (uint32)MTL::PixelFormatRG16Sint;
	GPixelFormats[PF_R32G32_UINT		].PlatformFormat	= (uint32)MTL::PixelFormatRG32Uint;

	// Use Depth28_Stencil8 when it is available for consistency
	if(bSupportsD24S8)
	{
		GPixelFormats[PF_DepthStencil].PlatformFormat				= (uint32)MTL::PixelFormatDepth24Unorm_Stencil8;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil 	= true;
		GPixelFormats[PF_DepthStencil].BlockBytes					= 4;
		GPixelFormats[PF_X24_G8].PlatformFormat						= (uint32)MTL::PixelFormatX24_Stencil8;
		GPixelFormats[PF_X24_G8].BlockBytes							= 4;
	}
	else
	{
		GPixelFormats[PF_DepthStencil].PlatformFormat				= (uint32)MTL::PixelFormatDepth32Float_Stencil8;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil 	= true;
		GPixelFormats[PF_DepthStencil].BlockBytes					= 5;
		GPixelFormats[PF_X24_G8].PlatformFormat						= (uint32)MTL::PixelFormatX32_Stencil8;
		GPixelFormats[PF_X24_G8].BlockBytes							= 5;
	}

	GPixelFormats[PF_DepthStencil].Supported = true;
	GPixelFormats[PF_X24_G8].Supported = true;

#if PLATFORM_IOS
    GPixelFormats[PF_DXT1				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_DXT1				].Supported			= false;
    GPixelFormats[PF_DXT3				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_DXT3				].Supported			= false;
    GPixelFormats[PF_DXT5				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_DXT5				].Supported			= false;
	GPixelFormats[PF_BC4				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_BC4				].Supported			= false;
	GPixelFormats[PF_BC5				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_BC5				].Supported			= false;
	GPixelFormats[PF_BC6H				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_BC6H				].Supported			= false;
	GPixelFormats[PF_BC7				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_BC7				].Supported			= false;
	GPixelFormats[PF_PVRTC2				].PlatformFormat	= (uint32)MTL::PixelFormatPVRTC_RGBA_2BPP;
	GPixelFormats[PF_PVRTC2				].Supported			= true;
	GPixelFormats[PF_PVRTC4				].PlatformFormat	= (uint32)MTL::PixelFormatPVRTC_RGBA_4BPP;
	GPixelFormats[PF_PVRTC4				].Supported			= true;
	GPixelFormats[PF_PVRTC4				].PlatformFormat	= (uint32)MTL::PixelFormatPVRTC_RGBA_4BPP;
	GPixelFormats[PF_PVRTC4				].Supported			= true;
	GPixelFormats[PF_ASTC_4x4			].PlatformFormat	= (uint32)MTL::PixelFormatASTC_4x4_LDR;
	GPixelFormats[PF_ASTC_4x4			].Supported			= true;
	GPixelFormats[PF_ASTC_6x6			].PlatformFormat	= (uint32)MTL::PixelFormatASTC_6x6_LDR;
	GPixelFormats[PF_ASTC_6x6			].Supported			= true;
	GPixelFormats[PF_ASTC_8x8			].PlatformFormat	= (uint32)MTL::PixelFormatASTC_8x8_LDR;
	GPixelFormats[PF_ASTC_8x8			].Supported			= true;
	GPixelFormats[PF_ASTC_10x10			].PlatformFormat	= (uint32)MTL::PixelFormatASTC_10x10_LDR;
	GPixelFormats[PF_ASTC_10x10			].Supported			= true;
	GPixelFormats[PF_ASTC_12x12			].PlatformFormat	= (uint32)MTL::PixelFormatASTC_12x12_LDR;
	GPixelFormats[PF_ASTC_12x12			].Supported			= true;

#if !PLATFORM_TVOS
	if(MTLDevice->supportsFamily(MTL::GPUFamilyApple6))
	{
		GPixelFormats[PF_ASTC_4x4_HDR].PlatformFormat = (uint32)MTL::PixelFormatASTC_4x4_HDR;
		GPixelFormats[PF_ASTC_4x4_HDR].Supported = true;
		GPixelFormats[PF_ASTC_6x6_HDR].PlatformFormat = (uint32)MTL::PixelFormatASTC_6x6_HDR;
		GPixelFormats[PF_ASTC_6x6_HDR].Supported = true;
		GPixelFormats[PF_ASTC_8x8_HDR].PlatformFormat = (uint32)MTL::PixelFormatASTC_8x8_HDR;
		GPixelFormats[PF_ASTC_8x8_HDR].Supported = true;
		GPixelFormats[PF_ASTC_10x10_HDR].PlatformFormat = (uint32)MTL::PixelFormatASTC_10x10_HDR;
		GPixelFormats[PF_ASTC_10x10_HDR].Supported = true;
		GPixelFormats[PF_ASTC_12x12_HDR].PlatformFormat = (uint32)MTL::PixelFormatASTC_12x12_HDR;
		GPixelFormats[PF_ASTC_12x12_HDR].Supported = true;
	}
#endif
	// used with virtual textures
	GPixelFormats[PF_ETC2_RGB	  		].PlatformFormat	= (uint32)MTL::PixelFormatETC2_RGB8;
	GPixelFormats[PF_ETC2_RGB			].Supported			= true;
	GPixelFormats[PF_ETC2_RGBA	  		].PlatformFormat	= (uint32)MTL::PixelFormatEAC_RGBA8;
	GPixelFormats[PF_ETC2_RGBA			].Supported			= true;
	GPixelFormats[PF_ETC2_R11_EAC	  	].PlatformFormat	= (uint32)MTL::PixelFormatEAC_R11Unorm;
	GPixelFormats[PF_ETC2_R11_EAC		].Supported			= true;
	GPixelFormats[PF_ETC2_RG11_EAC		].PlatformFormat	= (uint32)MTL::PixelFormatEAC_RG11Unorm;
	GPixelFormats[PF_ETC2_RG11_EAC		].Supported			= true;

	// IOS HDR format is BGR10_XR (32bits, 3 components)
	GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeX		= 1;
	GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeY		= 1;
	GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeZ		= 1;
	GPixelFormats[PF_PLATFORM_HDR_0		].BlockBytes		= 4;
	GPixelFormats[PF_PLATFORM_HDR_0		].NumComponents		= 3;
	GPixelFormats[PF_PLATFORM_HDR_0		].PlatformFormat	= (uint32)MTL::PixelFormatBGR10_XR_sRGB;
	GPixelFormats[PF_PLATFORM_HDR_0		].Supported			= GRHISupportsHDROutput;
		
	GPixelFormats[PF_FloatRGB			].PlatformFormat	= (uint32)MTL::PixelFormatRG11B10Float;
	GPixelFormats[PF_FloatRGB			].BlockBytes		= 4;
	GPixelFormats[PF_FloatR11G11B10		].PlatformFormat	= (uint32)MTL::PixelFormatRG11B10Float;
	GPixelFormats[PF_FloatR11G11B10		].BlockBytes		= 4;
	GPixelFormats[PF_FloatR11G11B10		].Supported			= true;
	
	GPixelFormats[PF_ShadowDepth		].PlatformFormat	= (uint32)MTL::PixelFormatDepth16Unorm;
	GPixelFormats[PF_ShadowDepth		].BlockBytes		= 2;
	GPixelFormats[PF_ShadowDepth		].Supported			= true;
	GPixelFormats[PF_D24				].PlatformFormat	= (uint32)MTL::PixelFormatDepth32Float;
	GPixelFormats[PF_D24				].Supported			= true;
		
	GPixelFormats[PF_BC5				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_R5G6B5_UNORM		].PlatformFormat	= (uint32)MTL::PixelFormatB5G6R5Unorm;
	GPixelFormats[PF_R5G6B5_UNORM       ].Supported         = true;
	GPixelFormats[PF_B5G5R5A1_UNORM     ].PlatformFormat    = (uint32)MTL::PixelFormatBGR5A1Unorm;
	GPixelFormats[PF_B5G5R5A1_UNORM     ].Supported         = true;
#else
    GPixelFormats[PF_DXT1				].PlatformFormat	= (uint32)MTL::PixelFormatBC1_RGBA;
    GPixelFormats[PF_DXT3				].PlatformFormat	= (uint32)MTL::PixelFormatBC2_RGBA;
    GPixelFormats[PF_DXT5				].PlatformFormat	= (uint32)MTL::PixelFormatBC3_RGBA;
	
    GPixelFormats[PF_FloatRGB		].PlatformFormat	= (uint32)MTL::PixelFormatRG11B10Float;
    GPixelFormats[PF_FloatRGB		].BlockBytes		= 4;

	
	GPixelFormats[PF_FloatR11G11B10		].PlatformFormat	= (uint32)MTL::PixelFormatRG11B10Float;
	GPixelFormats[PF_FloatR11G11B10		].BlockBytes		= 4;
	GPixelFormats[PF_FloatR11G11B10		].Supported			= true;
	
	if(GRHIVendorId == (uint32)EGpuVendorId::Apple)
	{
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeX		= 1;
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeY		= 1;
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeZ		= 1;
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockBytes		= 4;
		GPixelFormats[PF_PLATFORM_HDR_0		].NumComponents		= 3;
		GPixelFormats[PF_PLATFORM_HDR_0		].PlatformFormat	= (uint32)MTL::PixelFormatBGR10_XR_sRGB;
		GPixelFormats[PF_PLATFORM_HDR_0		].Supported			= GRHISupportsHDROutput;
	}
	else
	{
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeX		= 1;
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeY		= 1;
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockSizeZ		= 1;
		GPixelFormats[PF_PLATFORM_HDR_0		].BlockBytes		= 8;
		GPixelFormats[PF_PLATFORM_HDR_0		].NumComponents		= 4;
		GPixelFormats[PF_PLATFORM_HDR_0		].PlatformFormat	= (uint32)MTL::PixelFormatRGBA16Float;
		GPixelFormats[PF_PLATFORM_HDR_0		].Supported			= GRHISupportsHDROutput;
	}

	if (bSupportsD16)
	{
		GPixelFormats[PF_ShadowDepth		].PlatformFormat	= (uint32)MTL::PixelFormatDepth16Unorm;
		GPixelFormats[PF_ShadowDepth		].BlockBytes		= 2;
	}
	else
	{
		GPixelFormats[PF_ShadowDepth		].PlatformFormat	= (uint32)MTL::PixelFormatDepth32Float;
		GPixelFormats[PF_ShadowDepth		].BlockBytes		= 4;
	}
	GPixelFormats[PF_ShadowDepth		].Supported			= true;
	if(bSupportsD24S8)
	{
		GPixelFormats[PF_D24			].PlatformFormat	= (uint32)MTL::PixelFormatDepth24Unorm_Stencil8;
	}
	else
	{
		GPixelFormats[PF_D24			].PlatformFormat	= (uint32)MTL::PixelFormatDepth32Float;
	}
	GPixelFormats[PF_D24				].Supported			= true;
	GPixelFormats[PF_BC4				].Supported			= true;
	GPixelFormats[PF_BC4				].PlatformFormat	= (uint32)MTL::PixelFormatBC4_RUnorm;
	GPixelFormats[PF_BC5				].Supported			= true;
	GPixelFormats[PF_BC5				].PlatformFormat	= (uint32)MTL::PixelFormatBC5_RGUnorm;
	GPixelFormats[PF_BC6H				].Supported			= true;
	GPixelFormats[PF_BC6H               ].PlatformFormat	= (uint32)MTL::PixelFormatBC6H_RGBUfloat;
	GPixelFormats[PF_BC7				].Supported			= true;
	GPixelFormats[PF_BC7				].PlatformFormat	= (uint32)MTL::PixelFormatBC7_RGBAUnorm;
	GPixelFormats[PF_R5G6B5_UNORM		].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_B5G5R5A1_UNORM		].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
#endif
	GPixelFormats[PF_UYVY				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_FloatRGBA			].PlatformFormat	= (uint32)MTL::PixelFormatRGBA16Float;
	GPixelFormats[PF_FloatRGBA			].BlockBytes		= 8;
	
    GPixelFormats[PF_R32_FLOAT			].PlatformFormat	= (uint32)MTL::PixelFormatR32Float;
	
    if (MTLDevice->supportsFamily(MTL::GPUFamilyApple9))
    {
        EnumAddFlags(GPixelFormats[PF_R32_FLOAT].Capabilities, EPixelFormatCapabilities::TextureFilterable);
    }
        
	GPixelFormats[PF_G16R16				].PlatformFormat	= (uint32)MTL::PixelFormatRG16Unorm;
	GPixelFormats[PF_G16R16				].Supported			= true;
	
	EnumAddFlags(GPixelFormats[PF_G16R16].Capabilities, EPixelFormatCapabilities::TextureFilterable);
        
    GPixelFormats[PF_G16R16F			].PlatformFormat	= (uint32)MTL::PixelFormatRG16Float;
	GPixelFormats[PF_G16R16F_FILTER		].PlatformFormat	= (uint32)MTL::PixelFormatRG16Float;
	GPixelFormats[PF_G32R32F			].PlatformFormat	= (uint32)MTL::PixelFormatRG32Float;
	GPixelFormats[PF_A2B10G10R10		].PlatformFormat    = (uint32)MTL::PixelFormatRGB10A2Unorm;
	GPixelFormats[PF_A16B16G16R16		].PlatformFormat    = (uint32)MTL::PixelFormatRGBA16Unorm;
	GPixelFormats[PF_R16F				].PlatformFormat	= (uint32)MTL::PixelFormatR16Float;
	GPixelFormats[PF_R16F_FILTER		].PlatformFormat	= (uint32)MTL::PixelFormatR16Float;
	GPixelFormats[PF_V8U8				].PlatformFormat	= (uint32)MTL::PixelFormatRG8Snorm;
	GPixelFormats[PF_A1					].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	// A8 does not allow writes in Metal. So we will fake it with R8.
	// If you change this you must also change the swizzle pattern in Platform.ush
	// See Texture2DSample_A8 in Common.ush and A8_SAMPLE_MASK in Platform.ush
	GPixelFormats[PF_A8					].PlatformFormat	= (uint32)MTL::PixelFormatR8Unorm;
	GPixelFormats[PF_R32_UINT			].PlatformFormat	= (uint32)MTL::PixelFormatR32Uint;
	GPixelFormats[PF_R32_SINT			].PlatformFormat	= (uint32)MTL::PixelFormatR32Sint;
	GPixelFormats[PF_R16G16B16A16_UINT	].PlatformFormat	= (uint32)MTL::PixelFormatRGBA16Uint;
	GPixelFormats[PF_R16G16B16A16_SINT	].PlatformFormat	= (uint32)MTL::PixelFormatRGBA16Sint;
	GPixelFormats[PF_R8G8B8A8			].PlatformFormat	= (uint32)MTL::PixelFormatRGBA8Unorm;
    GPixelFormats[PF_A8R8G8B8           ].PlatformFormat    = (uint32)MTL::PixelFormatRGBA8Unorm;
	GPixelFormats[PF_R8G8B8A8_UINT		].PlatformFormat	= (uint32)MTL::PixelFormatRGBA8Uint;
	GPixelFormats[PF_R8G8B8A8_SNORM		].PlatformFormat	= (uint32)MTL::PixelFormatRGBA8Snorm;
	GPixelFormats[PF_R8G8				].PlatformFormat	= (uint32)MTL::PixelFormatRG8Unorm;
	GPixelFormats[PF_R16_SINT			].PlatformFormat	= (uint32)MTL::PixelFormatR16Sint;
	GPixelFormats[PF_R16_UINT			].PlatformFormat	= (uint32)MTL::PixelFormatR16Uint;
	GPixelFormats[PF_R8_UINT			].PlatformFormat	= (uint32)MTL::PixelFormatR8Uint;
	GPixelFormats[PF_R8					].PlatformFormat	= (uint32)MTL::PixelFormatR8Unorm;

	GPixelFormats[PF_R16G16B16A16_UNORM ].PlatformFormat	= (uint32)MTL::PixelFormatRGBA16Unorm;
	GPixelFormats[PF_R16G16B16A16_SNORM ].PlatformFormat	= (uint32)MTL::PixelFormatRGBA16Snorm;

	GPixelFormats[PF_NV12				].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_NV12				].Supported			= false;
	
	GPixelFormats[PF_G16R16_SNORM		].PlatformFormat	= (uint32)MTL::PixelFormatRG16Snorm;
	GPixelFormats[PF_R8G8_UINT			].PlatformFormat	= (uint32)MTL::PixelFormatRG8Uint;
	GPixelFormats[PF_R32G32B32_UINT		].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_R32G32B32_UINT		].Supported			= false;
	GPixelFormats[PF_R32G32B32_SINT		].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_R32G32B32_SINT		].Supported			= false;
	GPixelFormats[PF_R32G32B32F			].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_R32G32B32F			].Supported			= false;
	GPixelFormats[PF_R8_SINT			].PlatformFormat	= (uint32)MTL::PixelFormatR8Sint;
	GPixelFormats[PF_R64_UINT			].PlatformFormat	= (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_R64_UINT			].Supported			= false;
	GPixelFormats[PF_R9G9B9EXP5		    ].PlatformFormat    = (uint32)MTL::PixelFormatInvalid;
	GPixelFormats[PF_R9G9B9EXP5		    ].Supported			= false;

#if METAL_DEBUG_OPTIONS
	for (uint32 i = 0; i < PF_MAX; i++)
	{
		checkf((NS::UInteger)GMetalBufferFormats[i].LinearTextureFormat != NS::UIntegerMax, TEXT("Metal linear texture format for pixel-format %s (%d) is not configured!"), GPixelFormats[i].Name, i);
		checkf(GMetalBufferFormats[i].DataFormat != 255, TEXT("Metal data buffer format for pixel-format %s (%d) is not configured!"), GPixelFormats[i].Name, i);
	}
#endif

	RHIInitDefaultPixelFormatCapabilities();

	if (MTLDevice->supportsFamily(MTL::GPUFamilyApple2) && !MTLDevice->supportsFamily(MTL::GPUFamilyApple3))
	{
		UE_LOGF(LogMetal, Display, "Detected GPUFamilyApple2 device.");
		EnumRemoveFlags(GPixelFormats[PF_A2B10G10R10].Capabilities, EPixelFormatCapabilities::UAV);
		EnumRemoveFlags(GPixelFormats[PF_FloatRGB].Capabilities, EPixelFormatCapabilities::UAV);
		EnumRemoveFlags(GPixelFormats[PF_FloatR11G11B10].Capabilities, EPixelFormatCapabilities::UAV);
		EnumRemoveFlags(GPixelFormats[PF_R9G9B9EXP5].Capabilities, EPixelFormatCapabilities::UAV);
	}

	auto AddTypedUAVSupport = [](EPixelFormat InPixelFormat)
	{
		EnumAddFlags(GPixelFormats[InPixelFormat].Capabilities, EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore);
	};

	switch (MTLDevice->readWriteTextureSupport())
	{
	case MTL::ReadWriteTextureTier2:
		AddTypedUAVSupport(PF_A32B32G32R32F);
		AddTypedUAVSupport(PF_R32G32B32A32_UINT);
		AddTypedUAVSupport(PF_FloatRGBA);
		AddTypedUAVSupport(PF_R16G16B16A16_UINT);
		AddTypedUAVSupport(PF_R16G16B16A16_SINT);
		AddTypedUAVSupport(PF_R8G8B8A8);
		AddTypedUAVSupport(PF_R8G8B8A8_UINT);
		AddTypedUAVSupport(PF_R16F);
		AddTypedUAVSupport(PF_R16_UINT);
		AddTypedUAVSupport(PF_R16_SINT);
		AddTypedUAVSupport(PF_R8);
		AddTypedUAVSupport(PF_R8_UINT);
		
		// If we have Atomic64 support then we have Apple Silicon which has undocumented support for R11G11B10
		if(GRHISupportsAtomicUInt64)
		{
			AddTypedUAVSupport(PF_FloatR11G11B10);
		}
		// Fall through

	case MTL::ReadWriteTextureTier1:
		AddTypedUAVSupport(PF_R32_FLOAT);
		AddTypedUAVSupport(PF_R32_UINT);
		AddTypedUAVSupport(PF_R32_SINT);
		// Fall through

	case MTL::ReadWriteTextureTierNone:
		break;
	};

#if PLATFORM_MAC
	if(GPUDesc.GPUVendorId == GRHIVendorId)
	{
		UE_LOGF(LogMetal, Display,  "      Vendor ID: %d", GPUDesc.GPUVendorId);
		UE_LOGF(LogMetal, Display,  "      Device ID: %d", GPUDesc.GPUDeviceId);
		UE_LOGF(LogMetal, Display,  "      VRAM (MB): %d", GPUDesc.GPUMemoryMB);
	}
	else
	{
		UE_LOGF(LogMetal, Warning,  "GPU descriptor (%ls) from IORegistry failed to match Metal (%ls)", *FString(GPUDesc.GPUName), *GRHIAdapterName);
	}
#endif

#if PLATFORM_MAC
	if (!FPlatformProcess::IsSandboxedApplication())
	{
		// Cleanup local BinaryPSOs folder as it's not used anymore.
		const FString BinaryPSOsDir = FPaths::ProjectSavedDir() / TEXT("BinaryPSOs");
		IPlatformFile::GetPlatformPhysical().DeleteDirectoryRecursively(*BinaryPSOsDir);
	}
#endif

#if METAL_RHI_RAYTRACING
	if(Device->SupportsFeature(EMetalFeaturesRayTracing))
	{
		if (!FParse::Param(FCommandLine::Get(), TEXT("noraytracing")))
		{
			GRHISupportsRayTracing = RHISupportsRayTracing(GMaxRHIShaderPlatform);
			GRHISupportsRayTracingShaders = RHISupportsRayTracingShaders(GMaxRHIShaderPlatform) && UE::RHICore::AreRayTracingShadersEnabled();

			GRHISupportsRayTracingPSOAdditions = false;
			GRHISupportsRayTracingAMDHitToken = false;

			GRHISupportsInlineRayTracing = GRHISupportsRayTracing && RHISupportsInlineRayTracing(GMaxRHIShaderPlatform);
		}
		else
		{
			GRHISupportsRayTracing = false;
		}

		GRHISupportsInlineRayTracing = GRHISupportsRayTracing && RHISupportsInlineRayTracing(GMaxRHIShaderPlatform);
		GRHISupportsRayTracingDispatchIndirect = true;
		
		GRHIRayTracingAccelerationStructureAlignment = Device->GetDevice()->heapAccelerationStructureSizeAndAlign(NS::UInteger(0)).align;
		GRHIRayTracingScratchBufferAlignment = 256;
		GRHIRayTracingInstanceDescriptorSize = uint32(sizeof(MTL::IndirectAccelerationStructureInstanceDescriptor));
		GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer = true;
		
		check(sizeof(MTLIndirectAccelerationStructureInstanceDescriptor) == sizeof(MTL::IndirectAccelerationStructureInstanceDescriptor));

		static auto CVarRayTracingAllowCompaction = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Metal.RayTracing.AllowCompaction"));
		GRHIGlobals.RayTracing.SupportsAccelerationStructureCompaction = CVarRayTracingAllowCompaction->GetInt() != 0;
		GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT = true;
	}
#endif
		
	GDynamicRHI = this;
	
	// Start the submission and interrupt handler threads
	InitializeSubmissionPipe();
	
	GIsMetalInitialized = true;

#if METAL_USE_METAL_SHADER_CONVERTER
	CompilerInstance = IRCompilerCreate();
#endif

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (GRHIGlobals.bSupportsBindless)
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device->GetBindlessDescriptorManager();
		BindlessDescriptorManager->Init();
	}
#endif
}

FMetalDynamicRHI::~FMetalDynamicRHI()
{
	check(IsInGameThread() && IsInRenderingThread());
	
	RHIBlockUntilGPUIdle();
	ShutdownSubmissionPipe();
	
	GIsMetalInitialized = false;
	GIsRHIInitialized = false;

	// Ask all initialized FRenderResources to release their RHI resources.
	FRenderResource::ReleaseRHIForAllResources();	
	
#if METAL_USE_METAL_SHADER_CONVERTER
    IRCompilerDestroy(CompilerInstance);
#endif
}

FDynamicRHI::FRHICalcTextureSizeResult FMetalDynamicRHI::RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex)
{
	FDynamicRHI::FRHICalcTextureSizeResult Result;
	Result.Size = Desc.CalcMemorySizeEstimate(FirstMipIndex);
	Result.Align = 0;
	return Result;
}

uint64 FMetalDynamicRHI::RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format)
{
	return Device->GetDevice()->minimumLinearTextureAlignmentForPixelFormat((MTL::PixelFormat)GMetalBufferFormats[Format].LinearTextureFormat);
}

void FMetalDynamicRHI::Init()
{
	FRenderResource::InitPreRHIResources();
	GIsRHIInitialized = true;
}

void FMetalDynamicRHI::RHIEndFrame_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	RHICmdList.EnqueueLambdaMultiPipe(ERHIPipeline::Graphics, FRHICommandListBase::EThreadFence::Enabled, TEXT("Metal EndFrame"),
		[this](FMetalContextArray const& Contexts)
	{
		MTL_SCOPED_AUTORELEASE_POOL;

#if METAL_RHI_RAYTRACING
		Device->UpdateRayTracing(*Contexts[ERHIPipeline::Graphics]);
#endif // METAL_RHI_RAYTRACING
	});

	FDynamicRHI::RHIEndFrame_RenderThread(RHICmdList);
}

void FMetalDynamicRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	// increment the internal frame counter
	Device->IncrementFrameRHIThread();
    Device->GarbageCollect();

	// Close the previous frame's timing and start a new one
	auto Lambda = [this, OldTiming = MoveTemp(CurrentTimingPerQueue)]()
	{
		TArray<UE::RHI::GPUProfiler::FEventStream, TInlineAllocator<GMetalMaxNumQueues>> Streams;
		for (auto const& Timing : OldTiming)
		{
			Streams.Add(MoveTemp(Timing->EventStream));
		}
		
		UE::RHI::GPUProfiler::ProcessEvents(Streams);
	};
	
	EnqueueEndOfPipeTask(MoveTemp(Lambda), [&](FMetalPayload& Payload)
	{
		// Modify the payloads the EOP task will submit to include
		// a new timing struct and a frame boundary event.
		Payload.Timing = CurrentTimingPerQueue.CreateNew(Payload.Queue);
		
		ERHIPipeline Pipeline = ERHIPipeline::Graphics;
		
		Payload.EndFrameEvent = UE::RHI::GPUProfiler::FEvent::FFrameBoundary(Pipeline, Args, 0);
	});
	
	// Pump the interrupt queue to gather completed events
	// (required if we're not using an interrupt thread).
	ProcessInterruptQueueUntil(nullptr);
}

#if WITH_RHI_BREADCRUMBS
void FMetalRHICommandContext::RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
{
#if ENABLE_METAL_GPUEVENTS
	if (ShouldEmitBreadcrumbs())
	{
		MTL_SCOPED_AUTORELEASE_POOL;
		{
			// @todo dev-pr avoid TCHAR -> ANSI conversion
			FRHIBreadcrumb::FBuffer Buffer;
			TCHAR const* Name = Breadcrumb->GetTCHAR(Buffer);

			CurrentEncoder.PushDebugGroup(NS::String::string(TCHAR_TO_UTF8(Name), NS::UTF8StringEncoding));
		}
	}
#endif

#if ENABLE_METAL_GPUPROFILE
	if (!CurrentEncoder.IsParallelEncoding())
	{
		if (!CurrentEncoder.GetCommandBuffer())
		{
			StartCommandBuffer();
		}
		
		FMetalCommandBuffer* CmdBuffer = CurrentEncoder.GetCommandBuffer();
		
		// Can't process breadcrumbs if we are within a render pass
		if (Device.SupportsFeature(EMetalFeaturesStageCounterSampling))
		{
			TRefCountPtr<FMetalBreadcrumbInstance> Instance = FMetalBreadcrumbProfiler::GetInstance()->GetBreadcrumbEvent(Breadcrumb, bWithinRenderPass);
			Instance->TimestampTOP = &CmdBuffer->EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginBreadcrumb>(Breadcrumb).GPUTimestampTOP;
			*Instance->TimestampTOP = 0;
			Instance->BeginCmdBuffer = CmdBuffer;
			CmdBuffer->BeginBreadcrumb(Instance);
		}
	}
#endif
}

void FMetalRHICommandContext::RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
{
#if ENABLE_METAL_GPUPROFILE
	if (!CurrentEncoder.IsParallelEncoding())
	{
		if(!CurrentEncoder.GetCommandBuffer())
		{
			StartCommandBuffer();	
		}
		
		FMetalCommandBuffer* CmdBuffer = CurrentEncoder.GetCommandBuffer();
		
		if (Device.SupportsFeature(EMetalFeaturesStageCounterSampling))
		{
			TRefCountPtr<FMetalBreadcrumbInstance> Instance = FMetalBreadcrumbProfiler::GetInstance()->GetBreadcrumbEvent(Breadcrumb, bWithinRenderPass);
			Instance->TimestampBOP = &CmdBuffer->EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndBreadcrumb>(Breadcrumb).GPUTimestampBOP;
			*Instance->TimestampBOP = 0;
			CmdBuffer->EndBreadcrumb(Instance);
		}
	}
#endif

#if ENABLE_METAL_GPUEVENTS
	if (ShouldEmitBreadcrumbs())
	{
		MTL_SCOPED_AUTORELEASE_POOL;
		{
			CurrentEncoder.PopDebugGroup();
		}
	}
#endif
}
#endif // WITH_RHI_BREADCRUMBS

void FMetalDynamicRHI::RHIGetSupportedResolution( uint32 &Width, uint32 &Height )
{
#if PLATFORM_MAC
	CGDisplayModeRef DisplayMode = FMacPlatformMiscEx::GetSupportedDisplayMode(kCGDirectMainDisplay, Width, Height);
	if (DisplayMode)
	{
		Width = CGDisplayModeGetWidth(DisplayMode);
		Height = CGDisplayModeGetHeight(DisplayMode);
		CGDisplayModeRelease(DisplayMode);
	}
#else
	UE_LOGF(LogMetal, Warning,  "RHIGetSupportedResolution unimplemented!");
#endif
}

bool FMetalDynamicRHI::RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
#if PLATFORM_MAC
	const int32 MinAllowableResolutionX = 0;
	const int32 MinAllowableResolutionY = 0;
	const int32 MaxAllowableResolutionX = 10480;
	const int32 MaxAllowableResolutionY = 10480;
	const int32 MinAllowableRefreshRate = 0;
	const int32 MaxAllowableRefreshRate = 10480;
	
	CFArrayRef AllModes = CGDisplayCopyAllDisplayModes(kCGDirectMainDisplay, NULL);
	if (AllModes)
	{
		const int32 NumModes = CFArrayGetCount(AllModes);
		const int32 Scale = (int32)FMacApplication::GetPrimaryScreenBackingScaleFactor();
		
		for (int32 Index = 0; Index < NumModes; Index++)
		{
			const CGDisplayModeRef Mode = (const CGDisplayModeRef)CFArrayGetValueAtIndex(AllModes, Index);
			const int32 Width = (int32)CGDisplayModeGetWidth(Mode) / Scale;
			const int32 Height = (int32)CGDisplayModeGetHeight(Mode) / Scale;
			const int32 RefreshRate = (int32)CGDisplayModeGetRefreshRate(Mode);
			
			if (Width >= MinAllowableResolutionX && Width <= MaxAllowableResolutionX && Height >= MinAllowableResolutionY && Height <= MaxAllowableResolutionY)
			{
				bool bAddIt = true;
				if (bIgnoreRefreshRate == false)
				{
					if (RefreshRate < MinAllowableRefreshRate || RefreshRate > MaxAllowableRefreshRate)
					{
						continue;
					}
				}
				else
				{
					// See if it is in the list already
					for (int32 CheckIndex = 0; CheckIndex < Resolutions.Num(); CheckIndex++)
					{
						FScreenResolutionRHI& CheckResolution = Resolutions[CheckIndex];
                        if ((CheckResolution.Width == Width) &&
                            (CheckResolution.Height == Height))
                        {
                            // Already in the list...
                            bAddIt = false;
                            break;
                        }
                        // Filter out unusable resolutions on notched Macs
                        else if ((CheckResolution.Width == Width) &&
                                 (CheckResolution.Height != Height))
                        {
                            bAddIt = false;
                            if (Height < CheckResolution.Height)
                            {
                                // Only use the shorter (below notch and padding) version
                                CheckResolution.Height = Height;
                            }
                            break;
                        }
					}
				}
				
				if (bAddIt)
				{
					// Add the mode to the list
					const int32 Temp2Index = Resolutions.AddZeroed();
					FScreenResolutionRHI& ScreenResolution = Resolutions[Temp2Index];
					
					ScreenResolution.Width = Width;
					ScreenResolution.Height = Height;
					ScreenResolution.RefreshRate = RefreshRate;
				}
			}
		}
		
		CFRelease(AllModes);
	}
	
	return true;
#else
	UE_LOGF(LogMetal, Warning,  "RHIGetAvailableResolutions unimplemented!");
	return false;
#endif
}

void* FMetalDynamicRHI::RHIGetNativeDevice()
{
	return (void*)Device->GetDevice();
}

MTL::Device* FMetalDynamicRHI::RHIGetDevice()
{
	return Device->GetDevice();
}

void* FMetalDynamicRHI::RHIGetNativeGraphicsQueue()
{
	return nullptr;
}

void* FMetalDynamicRHI::RHIGetNativeComputeQueue()
{
	return nullptr;
}

uint16 FMetalDynamicRHI::RHIGetPlatformTextureMaxSampleCount()
{
	TArray<ECompositingSampleCount::Type> SamplesArray{ ECompositingSampleCount::Type::One, ECompositingSampleCount::Type::Two, ECompositingSampleCount::Type::Four, ECompositingSampleCount::Type::Eight };

	uint16 PlatformMaxSampleCount = ECompositingSampleCount::Type::One;
	for (auto sampleIt = SamplesArray.CreateConstIterator(); sampleIt; ++sampleIt)
	{
		int sample = *sampleIt;

#if PLATFORM_IOS || PLATFORM_MAC
		if (!Device->GetDevice()->supportsTextureSampleCount(sample))
		{
			break;
		}
		PlatformMaxSampleCount = sample;
#endif
	}
	return PlatformMaxSampleCount;
}

void FMetalDynamicRHI::RHIBlockUntilGPUIdle()
{
	// Submit a new sync point to each queue
	TArray<FMetalPayload*> Payloads;
	Payloads.Reserve(int32(EMetalQueueType::Count));

	TArray<FMetalSyncPointRef, TInlineAllocator<(uint32)EMetalQueueType::Count>> SyncPoints;

	for (uint32 QueueTypeIndex = 0; QueueTypeIndex < (uint32)EMetalQueueType::Count; ++QueueTypeIndex)
	{
		FMetalSyncPointRef SyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);

		FMetalPayload* Payload = new FMetalPayload(Device->GetCommandQueue((EMetalQueueType)QueueTypeIndex));
		Payload->SyncPointsToSignal.Add(SyncPoint);
		Payload->bAlwaysSignal = true;

		Payloads.Add(Payload);
		SyncPoints.Add(SyncPoint);
	}

	SubmitPayloads(MoveTemp(Payloads));

	// Block this thread until the sync points have signaled.
	for (FMetalSyncPointRef& SyncPoint : SyncPoints)
	{
		SyncPoint->Wait();
	}
}

IRHICommandContext* FMetalDynamicRHI::RHIGetDefaultContext()
{
	// There is no "default"/"immediate" context in Metal now, just return a regular (i.e. parallel) RHI context.
	// Forward this call through the GDynamicRHI pointer so the validation layer can still do its per-context setup.
	return static_cast<IRHICommandContext*>(&GDynamicRHI->RHIGetCommandContext(ERHIPipeline::Graphics)->GetLowestLevelContext());
}

IRHIComputeContext* FMetalDynamicRHI::RHIGetCommandContext(ERHIPipeline Pipeline)
{
	FMetalRHICommandContext* Context = MetalCommandContextPool.Pop();
	if (!Context)
	{
		Context = new FMetalRHICommandContext(*Device);
	}
	
	Context->ResetContext();
	
	return static_cast<IRHIComputeContext*>(Context);
}

IRHICommandContext* FMetalDynamicRHI::RHIGetParallelCommandContext(FRHIParallelRenderPassInfo const& ParallelRenderPass)
{
	check(GRHISupportsParallelRHIExecute);
	
	FMetalRHICommandContext* Context = MetalCommandContextPool.Pop();
	if (!Context)
	{
		Context = new FMetalRHICommandContext(*Device);
	}
	
	Context->ResetContext();
	Context->SetupParallelContext(&ParallelRenderPass);
	
	return Context;
}

class FMetalPlatformCommandList final : public IRHIPlatformCommandList
{
public:
	~FMetalPlatformCommandList() {};
	TArray<FMetalCommandBuffer*> CommandBuffers;
};

void FMetalDynamicRHI::RHIProcessDeleteQueue()
{
	FScopeLock Lock(&ObjectsToDeleteCS);
	TArray<FMetalDeferredDeleteObject> Objects = MoveTemp(ObjectsToDelete);
	
	if(!Objects.Num())
	{
		return;
	}
	
	EnqueueEndOfPipeTask([this, LocalDeleteObjects = MoveTemp(Objects)]() mutable
	{
		for(FMetalDeferredDeleteObject& Object : LocalDeleteObjects)
		{	
			switch (Object.Storage.GetIndex())
			{
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<NS::Object*>():
				{
					Object.Storage.Get<NS::Object*>()->release();
					break;
				}
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<FMetalBufferPtr>():
				{
					FMetalBufferPtr Buffer = Object.Storage.Get<FMetalBufferPtr>();	
					Buffer->MarkDeleted();
				
					break;
				}
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<MTLTexturePtr>():
				{
					MTLTexturePtr Texture = Object.Storage.Get<MTLTexturePtr>();
					
					if (!Texture->buffer() && !Texture->parentTexture())
					{
						Device->GetResourceHeap().ReleaseTexture(nullptr, Texture);
					}
					break;
				}
	#if METAL_RHI_RAYTRACING
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<FMetalAccelerationStructure*>():
				{
					FMetalAccelerationStructure* AS = Object.Storage.Get<FMetalAccelerationStructure*>();
					Device->GetResourceHeap().ReleaseAccelerationStructure(AS);
					
					break;
				}
	#endif
	#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<FRHIDescriptorHandle>():
				{
					FRHIDescriptorHandle Handle = Object.Storage.Get<FRHIDescriptorHandle>();
					
					FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device->GetBindlessDescriptorManager();
					check(BindlessDescriptorManager);

					BindlessDescriptorManager->FreeDescriptor(Handle);
					
					break;
				}
	#endif
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<FMetalFence*>():
				{
					FMetalFence* Fence = Object.Storage.Get<FMetalFence*>();
					FMetalFencePool::Get().ReleaseFence(Fence);
					
					break;
				}
				case FMetalDeferredDeleteObject::TObjectStorage::IndexOfType<TUniqueFunction<void()>*>():
				{
					TUniqueFunction<void()>* Func = Object.Storage.Get<TUniqueFunction<void()>*>();
					(*Func)();
					delete Func;
					
					break;
				}
				default:
				{
					checkNoEntry();
				}
			}
		}
		
		Device->DrainHeap();
	});
}

void FMetalDynamicRHI::EnqueueEndOfPipeTask(TUniqueFunction<void()> TaskFunc, TUniqueFunction<void(FMetalPayload&)> ModifyPayloadCallback)
{
	FGraphEventArray Prereqs;
	Prereqs.Reserve(GMetalMaxNumQueues + 1);
	if (EopTask)
	{
		Prereqs.Add(EopTask);
	}
	
	TArray<FMetalPayload*> Payloads;
	Payloads.Reserve(GMetalMaxNumQueues);
	
	ForEachQueue([&](FMetalCommandQueue& Queue)
	{
		FMetalPayload* Payload = new FMetalPayload(Queue);
		
		FMetalSyncPointRef SyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);
		Payload->SyncPointsToSignal.Emplace(SyncPoint);
		Prereqs.Add(SyncPoint->GetGraphEvent());
		
		if (ModifyPayloadCallback)
			ModifyPayloadCallback(*Payload);
		
		Payloads.Add(Payload);
	});
	
	SubmitPayloads(MoveTemp(Payloads));
	
	EopTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
					 MoveTemp(TaskFunc),
					 QUICK_USE_CYCLE_STAT(FExecuteRHIThreadTask, STATGROUP_TaskGraphTasks),
					 &Prereqs
					 );
}

void FMetalDynamicRHI::RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos)
{
	RHICmdList.EnqueueLambda(TEXT("FMetalDynamicRHI::RHIReplaceResources"),
		[ReplaceInfos = MoveTemp(ReplaceInfos)](FRHICommandListBase& InRHICmdList)
		{
			MTL_SCOPED_AUTORELEASE_POOL;

			for (FRHIResourceReplaceInfo const& Info : ReplaceInfos)
			{
				switch (Info.GetType())
				{
				default:
					checkNoEntry();
					break;

				case FRHIResourceReplaceInfo::EType::Buffer:
					{
						FMetalRHIBuffer* Dst = ResourceCast(Info.GetBuffer().Dst);
						FMetalRHIBuffer* Src = ResourceCast(Info.GetBuffer().Src);

						if (Src)
						{
							// The source buffer should not have any associated views.
							check(!Src->HasLinkedViews());

							Dst->TakeOwnership(*Src);
						}
						else
						{
							Dst->ReleaseOwnership();
						}

						Dst->UpdateLinkedViews(&FMetalRHICommandContext::Get(InRHICmdList));
					}
					break;

#if METAL_RHI_RAYTRACING
				case FRHIResourceReplaceInfo::EType::RTGeometry:
					{
						FMetalRayTracingGeometry* Dst = ResourceCast(Info.GetRTGeometry().Dst);
						FMetalRayTracingGeometry* Src = ResourceCast(Info.GetRTGeometry().Src);

						if (!Src)
						{
							Dst->ReleaseUnderlyingResource();
						}
						else
						{
							Dst->Swap(*Src);
						}
					}
					break;
#endif // METAL_RHI_RAYTRACING
				}
			}
		}
	);

	RHICmdList.RHIThreadFence(true);
}

void FMetalDynamicRHI::RHIWriteResourceGPUAddresses(FRHICommandListBase& RHICmdList, uint32 GPUIndex, TConstArrayView<FRHIResource*> Resources, uint64* AddressesPtr)
{
	RHICmdList.EnqueueLambda([AddressesPtr, Resources](FRHICommandListBase&)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RHIWriteResourceGPUAddresses);

			for (int32 ResourceIndex = 0; ResourceIndex < Resources.Num(); ++ResourceIndex)
			{
				const FRHIResource* Resource = Resources[ResourceIndex];

				if (Resource->GetType() == RRT_Buffer)
				{
					const FMetalRHIBuffer* Buffer = ResourceCast(static_cast<const FRHIBuffer*>(Resource));
					AddressesPtr[ResourceIndex] = Buffer->GetCurrentBufferOrNull()->GetGPUAddress();
				}
				else if (Resource->GetType() == RRT_Texture)
				{
					unimplemented();
				}
#if METAL_RHI_RAYTRACING
				else if (Resource->GetType() == RRT_RayTracingGeometry)
				{
					const FMetalRayTracingGeometry* RayTracingGeometry = ResourceCast(static_cast<const FRHIRayTracingGeometry*>(Resource));
					AddressesPtr[ResourceIndex] = RayTracingGeometry->GetGPUAddress();
				}
#endif
				else
				{
					checkNoEntry();
				}
			}
		});
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
FRHIResourceCollectionRef FMetalDynamicRHI::RHICreateResourceCollection(FRHICommandListBase& RHICmdList, TConstArrayView<FRHIResourceCollectionMember> InMembers)
{
	return UE::RHICore::CreateGenericResourceCollection(RHICmdList, InMembers);
}

void FMetalDynamicRHI::RHIUpdateResourceCollection(FRHICommandListBase& RHICmdList, FRHIResourceCollection* InResourceCollection, uint32 InStartIndex, TConstArrayView<FRHIResourceCollectionMember> InMemberUpdates)
{
	UE::RHICore::UpdateGenericResourceCollection(RHICmdList, ResourceCast(InResourceCollection), InStartIndex, InMemberUpdates);
}
#endif

void FMetalDynamicRHI::ForEachQueue(TFunctionRef<void(FMetalCommandQueue&)> Callback)
{
	// TODO - Carl: Multiple Queues
	Callback(Device->GetCommandQueue(EMetalQueueType::Direct));
}
