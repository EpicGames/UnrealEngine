// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalVertexBuffer.cpp: Metal vertex buffer RHI implementation.
=============================================================================*/

#include "MetalRHIPrivate.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalProfiler.h"
#include "MetalCommandBuffer.h"
#include "MetalCommandQueue.h"
#include "MetalDynamicRHI.h"
#include "Containers/ResourceArray.h"
#include "RenderUtils.h"
#include "MetalLLM.h"
#include "RHICoreBufferInitializer.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include <objc/runtime.h>

#define METAL_POOL_BUFFER_BACKING 1

#if STATS
#define METAL_INC_DWORD_STAT_BY(Name, Size, Usage) do { if (EnumHasAnyFlags(Usage, BUF_IndexBuffer)) { INC_DWORD_STAT_BY(STAT_MetalIndex##Name, Size); } else { INC_DWORD_STAT_BY(STAT_MetalVertex##Name, Size); } } while (false)
#else
#define METAL_INC_DWORD_STAT_BY(Name, Size, Usage)
#endif

FMetalBufferData::~FMetalBufferData()
{
    if (Data)
    {
        FMemory::Free(Data);
        Data = nullptr;
        Len = 0;
    }
}

void FMetalBufferData::InitWithSize(uint32 Size)
{
    Data = (uint8*)FMemory::Malloc(Size);
    Len = Size;
    check(Data);
}

enum class EMetalBufferUsage
{
	None = 0,
	GPUOnly = 1 << 0,
	LinearTex = 1 << 1,
};
ENUM_CLASS_FLAGS(EMetalBufferUsage);

static EMetalBufferUsage GetMetalBufferUsage(EBufferUsageFlags InUsage)
{
	EMetalBufferUsage Usage = EMetalBufferUsage::None;

	if (EnumHasAnyFlags(InUsage, BUF_VertexBuffer))
	{
		Usage |= EMetalBufferUsage::LinearTex;
	}

	if (EnumHasAnyFlags(InUsage, BUF_IndexBuffer))
	{
		Usage |= (EMetalBufferUsage::GPUOnly | EMetalBufferUsage::LinearTex);
	}

	if (EnumHasAnyFlags(InUsage, BUF_StructuredBuffer))
	{
		Usage |= EMetalBufferUsage::GPUOnly;
	}

	return Usage;
}

bool FMetalRHIBuffer::UsePrivateMemory() const
{
	if(EnumHasAnyFlags(GetUsage(), BUF_KeepCPUAccessible) && FMetalCommandQueue::IsUMASystem())
	{
		return false;
	}
	
	return Device.SupportsFeature(EMetalFeaturesEfficientBufferBlits);
}

FMetalRHIBuffer::FMetalRHIBuffer(FRHICommandListBase& RHICmdList, FMetalDevice& MetalDevice, const FRHIBufferCreateDesc& CreateDesc, FResourceArrayUploadInterface* InResourceArray)
	: FRHIBuffer(CreateDesc)
	, Device(MetalDevice)
	, Size(CreateDesc.Size)
	, Mode(BUFFER_STORAGE_MODE)
{
#if METAL_RHI_RAYTRACING
	if (EnumHasAnyFlags(CreateDesc.Usage, BUF_AccelerationStructure))
	{
		AccelerationStructure = Device.GetResourceHeap().CreateAccelerationStructure(Size);
		return;
	}
#endif

	const EMetalBufferUsage MetalUsage = GetMetalBufferUsage(CreateDesc.Usage);
	
	const bool bIsStatic  	= EnumHasAnyFlags(CreateDesc.Usage, BUF_Static);
	const bool bIsDynamic 	= EnumHasAnyFlags(CreateDesc.Usage, BUF_Dynamic);
	const bool bIsVolatile	= EnumHasAnyFlags(CreateDesc.Usage, BUF_Volatile);
	const bool bIsNull		= EnumHasAnyFlags(CreateDesc.Usage, BUF_NullResource);
	const bool bWantsView 	= EnumHasAnyFlags(CreateDesc.Usage, BUF_ShaderResource | BUF_UnorderedAccess);
	
	uint32_t ValidateTypeCount = (uint32_t)bIsStatic + (uint32_t)bIsDynamic +
								(uint32_t)bIsVolatile + (uint32_t)bIsNull;
	
	check(ValidateTypeCount == 1);

	Mode = UsePrivateMemory() ? MTL::StorageModePrivate : BUFFER_STORAGE_MODE;
	
	if (CreateDesc.Size)
	{
		checkf(CreateDesc.Size <= Device.GetDevice()->maxBufferLength(), TEXT("Requested buffer size larger than supported by device."));
			
#if PLATFORM_MAC
		// Buffer can be blit encoder copied on lock/unlock, we need to know that the buffer size is large enough for copy operations that are in multiples of
		// 4 bytes on macOS, iOS can be 1 byte.  Update size to know we have at least this much buffer memory, it will be larger in the end.
		Size = Align(CreateDesc.Size, 4);
#endif
		
		AllocateBuffer();
	}

	if (InResourceArray && CreateDesc.Size > 0)
	{
		check(CreateDesc.Size == InResourceArray->GetResourceDataSize());

		if (Mode == MTL::StorageModePrivate)
		{
			if (RHICmdList.IsBottomOfPipe())
			{
				void* Backing = this->Lock(RHICmdList, RLM_WriteOnly, 0, CreateDesc.Size);
				FMemory::Memcpy(Backing, InResourceArray->GetResourceData(), CreateDesc.Size);
				this->Unlock(RHICmdList);
			}
			else
			{
				void* Result = FMemory::Malloc(CreateDesc.Size, 16);
				FMemory::Memcpy(Result, InResourceArray->GetResourceData(), CreateDesc.Size);

				RHICmdList.EnqueueLambda(
					[this, Result, InSize = CreateDesc.Size](FRHICommandListBase& RHICmdList)
					{
						void* Backing = this->Lock(RHICmdList, RLM_WriteOnly, 0, InSize);
						FMemory::Memcpy(Backing, Result, InSize);
						this->Unlock(RHICmdList);
						FMemory::Free(Result);
					});
			}
		}
		else
		{
			FMetalBufferPtr TheBuffer = GetCurrentBuffer();
			MTL::Buffer* MTLBuffer = TheBuffer->GetMTLBuffer();
			FMemory::Memcpy(TheBuffer->Contents(), InResourceArray->GetResourceData(), CreateDesc.Size);
#if PLATFORM_MAC 
			if (Mode == MTL::StorageModeManaged)
			{
				NS::Range ModifyRange = NS::Range(TheBuffer->GetOffset(), TheBuffer->GetLength());
				MTLBuffer->didModifyRange(ModifyRange);
			}
#endif
		}

		// Discard the resource array's contents.
		InResourceArray->Discard();
	}
}

FMetalRHIBuffer::~FMetalRHIBuffer()
{
    ReleaseOwnership();
}

void FMetalRHIBuffer::SwitchBuffer(FRHICommandListBase& RHICmdList)
{
	AllocateBuffer();
	UpdateLinkedViews(&FMetalRHICommandContext::Get(RHICmdList));
}

void FMetalRHIBuffer::AllocateBuffer()
{
	if(CurrentBuffer)
	{
		ReleaseBuffer();
	}
	
	uint32 AllocSize = Size;
	
	const bool bWantsView = EnumHasAnyFlags(GetDesc().Usage, BUF_ShaderResource | BUF_UnorderedAccess);
	
	// These allocations will not go into the pool.
	uint32 RequestedBufferOffsetAlignment = BufferOffsetAlignment;
	if(bWantsView)
	{
		// Buffer backed linear textures have specific align requirements
		// We don't know upfront the pixel format that may be requested for an SRV so we can't use minimumLinearTextureAlignmentForPixelFormat:
		RequestedBufferOffsetAlignment = BufferBackedLinearTextureOffsetAlignment;
	}
	
	const EMetalBufferUsage MetalUsage = GetMetalBufferUsage(GetDesc().Usage);
	
	if (EnumHasAnyFlags(MetalUsage, EMetalBufferUsage::LinearTex))
	{
		if (EnumHasAnyFlags(GetDesc().Usage, BUF_UnorderedAccess))
		{
			// Padding for write flushing when not using linear texture bindings for buffers
			AllocSize = Align(AllocSize + 512, 1024);
		}
		
		if (bWantsView)
		{
			uint32 NumElements = AllocSize;
			uint32 SizeX = NumElements;
			uint32 SizeY = 1;
			uint32 Dimension = GMaxTextureDimensions;
			while (SizeX > GMaxTextureDimensions)
			{
				while((NumElements % Dimension) != 0)
				{
					check(Dimension >= 1);
					Dimension = (Dimension >> 1);
				}
				SizeX = Dimension;
				SizeY = NumElements / Dimension;
				if(SizeY > GMaxTextureDimensions)
				{
					Dimension <<= 1;
					checkf(SizeX <= GMaxTextureDimensions, TEXT("Calculated width %u is greater than maximum permitted %d when converting buffer of size %u to a 2D texture."), Dimension, (int32)GMaxTextureDimensions, AllocSize);
					if(Dimension <= GMaxTextureDimensions)
					{
						AllocSize = Align(Size, Dimension);
						NumElements = AllocSize;
						SizeX = NumElements;
					}
					else
					{
						// We don't know the Pixel Format and so the bytes per element for the potential linear texture
						// Use max texture dimension as the align to be a worst case rather than crashing
						AllocSize = Align(Size, GMaxTextureDimensions);
						break;
					}
				}
			}
			
			AllocSize = Align(AllocSize, 1024);
		}
	}
	AllocSize = Align(AllocSize, RequestedBufferOffsetAlignment);
	
	FMetalBufferPtr Buffer = nullptr;

#if METAL_POOL_BUFFER_BACKING
	FMetalPooledBufferArgs ArgsCPU(&Device, AllocSize, GetDesc().Usage, Mode);
	Buffer = Device.CreatePooledBuffer(ArgsCPU);
#else
	NS::UInteger Options = (((NS::UInteger) Mode) << MTL::ResourceStorageModeShift);

	// Allocate one.
	MTL::Buffer* BufferPtr = Device.GetDevice()->newBuffer(AllocSize, (MTL::ResourceOptions) Options);
	Buffer = FMetalBufferPtr(new FMetalBuffer(BufferPtr, FMetalBuffer::FreePolicy::Owner));
	
	METAL_FATAL_ASSERT(Buffer, TEXT("Failed to create buffer of size %u and resource options %u"), Size, (uint32)Options);
	
	const bool bIsStatic = EnumHasAnyFlags(GetDesc().Usage, BUF_Static);
	if(bIsStatic)
	{
		FString Label = FString::Printf(TEXT("Static on frame %u"), Device.GetFrameNumberRHIThread());
		BufferPtr->setLabel(FStringToNSString(Label));
	}
	else
	{
		FString Label = FString::Printf(TEXT("Buffer on frame %u"), Device.GetFrameNumberRHIThread());
		BufferPtr->setLabel(FStringToNSString(Label));
	}
#endif
	
	MetalBufferStats::UpdateBufferStats(GetDesc(), Buffer->GetLength(), true);
	CurrentBuffer = Buffer;
	
	check(Buffer);
	check(AllocSize <= Buffer->GetLength());
	check(Buffer->GetMTLBuffer()->storageMode() == Mode);
}

void FMetalRHIBuffer::ReleaseBuffer()
{
	if(CurrentBuffer)
	{	
		MetalBufferStats::UpdateBufferStats(GetDesc(), CurrentBuffer->GetLength(), false);
		METAL_INC_DWORD_STAT_BY(MemFreed, CurrentBuffer->GetLength(), GetUsage());
		FMetalDynamicRHI::Get().DeferredDelete(CurrentBuffer);
		CurrentBuffer.Reset();
	}
} 

void FMetalRHIBuffer::AllocTransferBuffer(bool bOnRHIThread, uint32 InSize, EResourceLockMode LockMode)
{
	check(!TransferBuffer);
	FMetalPooledBufferArgs ArgsCPU(&Device, InSize, BUF_Dynamic, MTL::StorageModeShared);
	TransferBuffer = Device.CreatePooledBuffer(ArgsCPU);
	check(TransferBuffer);
	METAL_INC_DWORD_STAT_BY(MemAlloc, InSize, GetUsage());
	METAL_FATAL_ASSERT(TransferBuffer, TEXT("Failed to create buffer of size %u and storage mode %u"), InSize, (uint32)MTL::StorageModeShared);
}

bool FMetalRHIBuffer::RequiresTransferBuffer()
{
	const bool bIsStatic = EnumHasAnyFlags(GetUsage(), BUF_Static);
	return (Mode == MTL::StorageModePrivate || (Mode == MTL::StorageModeShared && bIsStatic));
}

static uint32 AdjustLockSize(uint32 Size, uint32 Offset)
{
#if PLATFORM_MAC
	// Blit encoder validation error, lock size and subsequent blit copy unlock operations need to be in 4 byte multiples on macOS
	return FMath::Min(Align(Size, 4), Size - Offset);
#else
	return Size;
#endif
}

void* FMetalRHIBuffer::Lock(FRHICommandListBase& RHICmdList, EResourceLockMode InLockMode, uint32 Offset, uint32 InSize)
{
	check(CurrentLockMode == RLM_Num);
	check(LockSize == 0 && LockOffset == 0);	
	check(!TransferBuffer);
	
	InSize = AdjustLockSize(InSize, Offset);

	const bool bWriteLock = InLockMode == RLM_WriteOnly;
	const bool bIsStatic = EnumHasAnyFlags(GetUsage(), EBufferUsageFlags::Static);

	void* ReturnPointer = nullptr;

	uint32 Len = GetCurrentBuffer()->GetLength(); // all buffers should have the same length or we are in trouble.
	check(Len >= InSize);

	if (bWriteLock)
	{
		const bool bUseTransferBuffer = RequiresTransferBuffer();

		// If we are locking for the first time then use the current buffer
		bool bValidFirstLock = bIsFirstLock && CurrentBuffer;

		if (!bValidFirstLock && (!bIsStatic || bUseTransferBuffer))
		{
			SwitchBuffer(RHICmdList);
		}

		bIsFirstLock = false;

		if(bUseTransferBuffer)
		{			
			TransferBuffer = Device.GetTransferAllocator()->Allocate(Len);
			ReturnPointer = TransferBuffer->Contents();
			check(ReturnPointer != nullptr);
		}
		else
		{
			check(GetCurrentBuffer());
			ReturnPointer = GetCurrentBuffer()->Contents();
			check(ReturnPointer != nullptr);
		}
	}
	else
	{
		check(InLockMode == EResourceLockMode::RLM_ReadOnly);
		// assumes offset is 0 for reads.
		check(Offset == 0);

		if (Mode == MTL::StorageModePrivate)
		{
			check(!TransferBuffer);
			SCOPE_CYCLE_COUNTER(STAT_MetalBufferPageOffTime);
			AllocTransferBuffer(true, Len, RLM_WriteOnly);
			check(TransferBuffer->GetLength() >= InSize);
			
			FRHICommandListImmediate& ImmediateCmdList = RHICmdList.GetAsImmediate();
			
			ImmediateCmdList.EnqueueLambda([this, InTransferBuffer=TransferBuffer](FRHICommandListBase& ExecutingCmdList)
			{
				FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(ExecutingCmdList);
				Context.CopyFromBufferToBuffer(GetCurrentBuffer(), 0, InTransferBuffer, 0, GetCurrentBuffer()->GetLength());
			});
			
			//kick the current command buffer.
			ImmediateCmdList.SubmitAndBlockUntilGPUIdle();
			
			ReturnPointer = TransferBuffer->Contents();
		}
#if PLATFORM_MAC
		else if(Mode == MTL::StorageModeManaged)
		{
			SCOPE_CYCLE_COUNTER(STAT_MetalBufferPageOffTime);
			
			FRHICommandListImmediate& ImmediateCmdList = RHICmdList.GetAsImmediate();
			
			ImmediateCmdList.EnqueueLambda([this](FRHICommandListBase& ExecutingCmdList)
			{
				FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(ExecutingCmdList);
				// Synchronise the buffer with the CPU
				Context.SynchronizeResource(GetCurrentBuffer()->GetMTLBuffer());
			});
			
			//kick the current command buffer.
			ImmediateCmdList.SubmitAndBlockUntilGPUIdle();
			
			ReturnPointer = GetCurrentBuffer()->Contents();
		}
#endif
		else
		{
			// Shared
            ReturnPointer = GetCurrentBuffer()->Contents();
		}

		check(ReturnPointer);
	} // Read Path

	check(GetCurrentBuffer());
	check((!GetCurrentBuffer()->GetMTLBuffer()->heap() && !GetCurrentBuffer()->GetMTLBuffer()->isAliasable()) || GetCurrentBuffer()->GetMTLBuffer()->heap() != nullptr);
	
	LockOffset = Offset;
	LockSize = InSize;
	CurrentLockMode = InLockMode;
	
	if(InSize == 0)
	{
		LockSize = Len;
	}
	
	ReturnPointer = ((uint8*) (ReturnPointer)) + Offset;
	return ReturnPointer;
}

void FMetalRHIBuffer::Unlock(FRHICommandListBase& RHICmdList)
{
	FMetalBufferPtr CurrentBufferToUnlock = GetCurrentBuffer();
	
	check(CurrentBufferToUnlock);
	check(LockSize > 0);
	const bool bWriteLock = CurrentLockMode == RLM_WriteOnly;
	const bool bIsStatic = EnumHasAnyFlags(GetUsage(), BUF_Static);
	
	if (bWriteLock)
	{
		check(LockOffset == 0);
		check(LockSize <= CurrentBufferToUnlock->GetLength());

		// Use transfer buffer for writing into 'Static' buffers as they could be in use by GPU atm
		// Initialization of 'Static' buffers still uses direct copy when possible
		const bool bUseTransferBuffer = RequiresTransferBuffer();
		
		if (bUseTransferBuffer)
		{
			FMetalRHIUploadContext& UploadContext = static_cast<FMetalRHIUploadContext&>(RHICmdList.GetUploadContext());
			
			UploadContext.EnqueueFunction([&InDevice=Device, Size=LockSize, Dest=CurrentBufferToUnlock, InTransferBuffer=TransferBuffer](FMetalRHICommandContext* Context)
			{
				Context->CopyFromBufferToBuffer(InTransferBuffer, 0, Dest, 0, Size);
				FMetalDynamicRHI::Get().DeferredDelete(InTransferBuffer);
			});
			
			TransferBuffer = nullptr;
		}
#if PLATFORM_MAC
		else if (Mode == MTL::StorageModeManaged)
		{
			CurrentBufferToUnlock->GetMTLBuffer()->didModifyRange(NS::Range(LockOffset + CurrentBufferToUnlock->GetOffset(), LockSize));
		}
#endif //PLATFORM_MAC
		else
		{
			// shared buffers are always mapped so nothing happens
			check(Mode == MTL::StorageModeShared);
		}
	}
	else
	{
		check(CurrentLockMode == RLM_ReadOnly);
		if(TransferBuffer)
		{
			check(Mode == MTL::StorageModePrivate);
			FMetalDynamicRHI::Get().DeferredDelete(TransferBuffer);
			TransferBuffer = nullptr;
		}
	}

	check(!TransferBuffer);
	CurrentLockMode = RLM_Num;
	LockSize = 0;
	LockOffset = 0;
}

void FMetalRHIBuffer::UploadTransferBuffer(FRHICommandListBase& RHICmdList, FMetalBufferPtr&& InTransferBuffer, uint32 UploadSize)
{
	check(InTransferBuffer);
	check(UploadSize);
	
	RHICmdList.EnqueueLambda(
		[this, InTransferBuffer = MoveTemp(InTransferBuffer), UploadSize](FRHICommandListBase& RHICmdList) mutable
		{
			if (!(bIsFirstLock && CurrentBuffer))
			{
				SwitchBuffer(RHICmdList);
			}
			
			bIsFirstLock = false;
				
			FMetalRHIUploadContext& UploadContext = static_cast<FMetalRHIUploadContext&>(RHICmdList.GetUploadContext());

			UploadContext.EnqueueFunction(
				[InTransferBuffer = MoveTemp(InTransferBuffer), CurrentBuffer=CurrentBuffer, UploadSize](FMetalRHICommandContext* Context)
				{
					Context->CopyFromBufferToBuffer(InTransferBuffer, 0, CurrentBuffer, 0, UploadSize);
					FMetalDynamicRHI::Get().DeferredDelete(InTransferBuffer);
				});
		});

	RHICmdList.RHIThreadFence(true);
}

void FMetalRHIBuffer::TakeOwnership(FMetalRHIBuffer& Other)
{
    check(Other.CurrentLockMode == RLM_Num);

    // Clean up any resource this buffer already owns
    ReleaseOwnership();

    // Transfer ownership of Other's resources to this instance
    FRHIBuffer::TakeOwnership(Other);

	CurrentBuffer = Other.CurrentBuffer;
    TransferBuffer = Other.TransferBuffer;
    CurrentLockMode = Other.CurrentLockMode;
    LockOffset = Other.LockOffset;
    LockSize = Other.LockSize;
    Size = Other.Size;
    Mode = Other.Mode;
    
	Other.CurrentBuffer.Reset();
    Other.TransferBuffer = nullptr;
    Other.CurrentLockMode = RLM_Num;
    Other.LockOffset = 0;
    Other.LockSize = 0;
    Other.Size = 0;
}

void FMetalRHIBuffer::ReleaseOwnership()
{
    if(TransferBuffer)
    {
        METAL_INC_DWORD_STAT_BY(MemFreed, TransferBuffer->GetLength(), GetUsage());
		FMetalDynamicRHI::Get().DeferredDelete(TransferBuffer);
    }
    
	ReleaseBuffer();

#if METAL_RHI_RAYTRACING
    if (EnumHasAnyFlags(GetUsage(), BUF_AccelerationStructure))
    {
		FMetalDynamicRHI::Get().DeferredDelete(AccelerationStructure);
        AccelerationStructure = nullptr;
    }
#endif // METAL_RHI_RAYTRACING
	
	FRHIBuffer::ReleaseOwnership();
}

static FRHIBufferCreateDesc MetalModifyBufferCreateDesc(const FRHIBufferCreateDesc& InCreateDesc)
{
	FRHIBufferCreateDesc CreateDesc(InCreateDesc);

	// No life-time usage information? Enforce Dynamic.
	if (!EnumHasAnyFlags(CreateDesc.Usage, EBufferUsageFlags::Static | EBufferUsageFlags::Dynamic | EBufferUsageFlags::Volatile | EBufferUsageFlags::NullResource))
	{
		CreateDesc.AddUsage(EBufferUsageFlags::Dynamic);
	}

	return CreateDesc;
}

FRHIBufferInitializer FMetalDynamicRHI::RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& InCreateDesc)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	const FRHIBufferCreateDesc CreateDesc = MetalModifyBufferCreateDesc(InCreateDesc);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	// TODO: Can Metal use UE::RHICore::CreateUnifiedMemoryBufferInitializer ?

	FMetalRHIBuffer* Buffer = new FMetalRHIBuffer(RHICmdList, *Device, CreateDesc, CreateDesc.InitialData);

	if (CreateDesc.IsNull() || CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray || CreateDesc.InitAction == ERHIBufferInitAction::Default)
	{
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Zeroed)
	{
		void* WritableData = RHICmdList.LockBuffer(Buffer, 0, CreateDesc.Size, RLM_WriteOnly);
		FMemory::Memzero(WritableData, CreateDesc.Size);
		RHICmdList.UnlockBuffer(Buffer);

		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
	{
		// Use LockBuffer + UnlockBuffer to allow the caller to write initial buffer data
		return UE::RHICore::FLockBufferInitializer(RHICmdList, Buffer);
	}

	return UE::RHICore::HandleUnknownBufferInitializerInitAction(RHICmdList, CreateDesc);
}

struct FMetalRHILockBuffer
{
	FMetalRHILockBuffer(FMetalBufferPtr&& InBuffer)
		: Buffer(MoveTemp(InBuffer))
	{
	}
	
	FMetalBufferPtr Buffer;
};

static FLockTracker GBufferLockTracker;

void* FMetalDynamicRHI::RHILockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	FMetalRHIBuffer* Buffer = ResourceCast(BufferRHI);

	if (RHICmdList.IsTopOfPipe())
	{
		void* Result = nullptr;
		if (LockMode != RLM_WriteOnly)
		{
			Result = Buffer->Lock(RHICmdList, LockMode, Offset, SizeRHI);

			GBufferLockTracker.Lock(Buffer, nullptr, Offset, SizeRHI, LockMode);
		}
		else
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_LockBuffer_Malloc);
			
			if (Buffer->RequiresTransferBuffer())
			{
				FMetalBufferPtr TempBuffer = Device->GetResourceHeap().CreateBuffer(SizeRHI, BufferBackedLinearTextureOffsetAlignment, BUF_Dynamic, MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared, true);

				Result = TempBuffer->Contents();

				GBufferLockTracker.Lock(Buffer, new FMetalRHILockBuffer(MoveTemp(TempBuffer)), Offset, SizeRHI, LockMode);
			}
			else
			{
				Result = FMemory::Malloc(SizeRHI, 16);

				GBufferLockTracker.Lock(Buffer, Result, Offset, SizeRHI, LockMode);
			}
		}

		return Result;
	}
	
	return Buffer->Lock(RHICmdList, LockMode, Offset, SizeRHI);
}

void FMetalDynamicRHI::RHIUnlockBuffer(class FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FDynamicRHI_UnlockBuffer_RenderThread);

	FMetalRHIBuffer* Buffer = ResourceCast(BufferRHI);

	if (RHICmdList.IsTopOfPipe())
	{
		FLockTracker::FLockParams Params = GBufferLockTracker.Unlock(Buffer);

		if (Params.LockMode != RLM_WriteOnly)
		{
			Buffer->Unlock(RHICmdList);
		}
		else
		{
			if (Buffer->RequiresTransferBuffer())
			{
				FMetalRHILockBuffer* LockBuffer = reinterpret_cast<FMetalRHILockBuffer*>(Params.Buffer);

				const uint32 UploadSize = AdjustLockSize(Params.BufferSize, Params.Offset);
				Buffer->UploadTransferBuffer(RHICmdList, MoveTemp(LockBuffer->Buffer), UploadSize);

				delete LockBuffer;
			}
			else
			{
				check(Params.Buffer);
				
				RHICmdList.EnqueueLambda([Buffer, Params](FRHICommandListBase& RHICmdList)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateBuffer_Execute);

					void* LockData = Buffer->Lock(RHICmdList, Params.LockMode, Params.Offset, Params.BufferSize);

					FMemory::Memcpy(LockData, Params.Buffer, Params.BufferSize);

					Buffer->Unlock(RHICmdList);

					FMemory::Free(Params.Buffer);
				});
				RHICmdList.RHIThreadFence(true);
			}
		}
	}
	else
	{
		Buffer->Unlock(RHICmdList);
	}
}

void* FMetalDynamicRHI::LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalRHIBuffer* Buffer = ResourceCast(BufferRHI);

	// default to buffer memory
	return Buffer->Lock(RHICmdList, LockMode, Offset, Size);
}

void FMetalDynamicRHI::UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    FMetalRHIBuffer* Buffer = ResourceCast(BufferRHI);
	Buffer->Unlock(RHICmdList);
}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
void FMetalRHIBuffer::UpdateAllocationTags()
{
	if(CurrentBuffer)
	{	
		MetalLLM::LogFreeBufferNative(CurrentBuffer->GetMTLBuffer());
		MetalLLM::LogAllocBufferNative(CurrentBuffer->GetMTLBuffer());
	}
} 

void FMetalDynamicRHI::RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer)
{
	check(RHICmdList.IsBottomOfPipe());
	ResourceCast(Buffer)->UpdateAllocationTags();
}
#endif

