// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalRHIContext.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalRHIPrivate.h"
#include "MetalRHIRenderQuery.h"
#include "MetalRHIVisionOSBridge.h"
#include "MetalBindlessDescriptors.h"
#include "MetalDevice.h"
#include "MetalCommandBuffer.h"
#include "MetalDynamicRHI.h"

#if PLATFORM_VISIONOS
#import <CompositorServices/CompositorServices.h>
#endif

static int32 GMetalSampleBlitEncoderTimings = 0;
static FAutoConsoleVariableRef CVarMetalSampleBlitEncoderTimings(
	TEXT("rhi.Metal.SampleBlitEncoderTimings"),
	GMetalSampleBlitEncoderTimings,
	TEXT("Whether to include Blit Encoder Timings in GPU Stats, can cause poor perf due to splitting encoders: off"));

static int32 GMetalSampleComputeEncoderTimings = 0;
static FAutoConsoleVariableRef CVarMetalSampleComputeEncoderTimings(
	TEXT("rhi.Metal.SampleComputeEncoderTimings"),
	GMetalSampleComputeEncoderTimings,
	TEXT("Whether to include Compute Encoder Timings in GPU Stats, can cause poor perf due to splitting encoders: on"));

static int32 GMetalConcurrentDispatch = 1;
static FAutoConsoleVariableRef CVarMetalConcurrentDispatch(
	TEXT("rhi.Metal.ConcurrentDispatch"),
	GMetalConcurrentDispatch,
	TEXT("Whether concurrent dispatch is on or off: default on"));

void METALRHI_API SafeReleaseMetalObject(NS::Object* Object)
{
	if(GIsMetalInitialized && GDynamicRHI && Object)
	{
		if(!IsRunningRHIInSeparateThread())
		{
			FMetalDynamicRHI::Get().DeferredDelete(Object);
		}
		else
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(
			   [Object]()
			   {
				   FMetalDynamicRHI::Get().DeferredDelete(Object);
			   },
			   QUICK_USE_CYCLE_STAT(FExecuteRHIThreadTask, STATGROUP_TaskGraphTasks), nullptr, ENamedThreads::RHIThread);
		}
		
		return;
	}
	Object->release();
}

FMetalRHICommandContext::FMetalRHICommandContext(FMetalDevice& MetalDevice)
	: Device(MetalDevice)
	, CommandQueue(Device.GetCommandQueue(EMetalQueueType::Direct))
	, CurrentEncoder(MetalDevice, CommandQueue)
	, StateCache(MetalDevice)
	, QueryBuffer(new FMetalQueryBufferPool(MetalDevice))
	, RenderPassDesc(nullptr)
	, bWithinRenderPass(false)
{
	StaticUniformBuffers.AddZeroed(FUniformBufferStaticSlotRegistry::Get().GetSlotCount());
}

FMetalRHICommandContext::~FMetalRHICommandContext()
{
	CurrentEncoder.Release();
}

void FMetalRHICommandContext::ResetContext()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	check(!PendingDescriptorUpdates.Num());
#endif
	
	// Reset cached state in the encoder
	StateCache.Reset();
	
	// Reset the current encoder
	CurrentEncoder.Reset();
	
	// Reallocate if necessary to ensure >= 80% usage, otherwise we're just too wasteful
	CurrentEncoder.GetRingBuffer().Shrink();
	
	// make sure first SetRenderTarget goes through
	StateCache.InvalidateRenderTargets();	
	
	ShaderBindingLayout = nullptr;
	FMemory::Memzero(StaticUniformBuffers.GetData(), StaticUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));
	
	bIsParallelContext = false;
}

void FMetalRHICommandContext::SetupParallelContext(const FRHIParallelRenderPassInfo* InRenderPassInfo)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	FMetalParallelRenderPassInfo* ParallelInfo = static_cast<FMetalParallelRenderPassInfo*>(InRenderPassInfo->RHIPlatformData);
	
	CurrentEncoder.BeginRenderCommandEncoding(ParallelInfo->RenderPassDesc, ParallelInfo->ParallelEncoder);
	
	RenderPassInfo = *static_cast<const FRHIRenderPassInfo*>(InRenderPassInfo);
	RenderPassDesc = ParallelInfo->RenderPassDesc;
	
	StateCache.StartRenderPass(RenderPassInfo, nullptr, RenderPassDesc, true);

	StateCache.SetRenderTargetsActive(true);
	StateCache.SetRenderStoreActions(CurrentEncoder, false);
	
	bWithinRenderPass = true;
	bIsParallelContext = true;
	
	// Set the viewport to the full size of render target 0.
	if (RenderPassInfo.ColorRenderTargets[0].RenderTarget)
	{
		const FRHIRenderPassInfo::FColorEntry& RenderTargetView = RenderPassInfo.ColorRenderTargets[0];
		FMetalSurface* RenderTarget = GetMetalSurfaceFromRHITexture(RenderTargetView.RenderTarget);

		uint32 Width = FMath::Max((uint32)(RenderTarget->Texture->width() >> RenderTargetView.MipIndex), (uint32)1);
		uint32 Height = FMath::Max((uint32)(RenderTarget->Texture->height() >> RenderTargetView.MipIndex), (uint32)1);

		RHISetViewport(0.0f, 0.0f, 0.0f, (float)Width, (float)Height, 1.0f);
	}
}

static uint32_t MAX_COLOR_RENDER_TARGETS_PER_DESC = 8;

void FMetalRHICommandContext::BeginComputeEncoder()
{
	MTL_SCOPED_AUTORELEASE_POOL;

	SCOPE_CYCLE_COUNTER(STAT_MetalSwitchToComputeTime);
	
	check(!bWithinRenderPass);
	check(IsInParallelRenderingThread());
	
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	StateCache.SetStateDirty();
	
	if(!CurrentEncoder.IsComputeCommandEncoderActive())
	{
		StateCache.ClearPreviousComputeState();
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
			StateCache.ResetBindings();
		}
		bool bUseStageCounterSamples = Device.SupportsFeature(EMetalFeaturesStageCounterSampling);
		
		MTL::DispatchType DispatchType = GMetalConcurrentDispatch ? MTL::DispatchTypeConcurrent : MTL::DispatchTypeSerial;
		CurrentEncoder.BeginComputeCommandEncoding(DispatchType,
												   bUseStageCounterSamples ? Device.GetCounterSampler() : nullptr);
	}
	
	if (CurrentEncoderFence)
	{
		CurrentEncoder.WaitForFence(CurrentEncoderFence);
		CurrentEncoderFence = nullptr;
	}
	
	check(CurrentEncoder.IsComputeCommandEncoderActive());
}

void FMetalRHICommandContext::EndComputeEncoder()
{
	check(CurrentEncoder.IsComputeCommandEncoderActive());
	
	// If we are using breadcrumbs then end the encoding here so that our stat tracking is correct
#if WITH_PROFILEGPU
	if(Device.SupportsFeature(EMetalFeaturesStageCounterSampling) && GMetalSampleComputeEncoderTimings)
	{
		CurrentEncoderFence = CurrentEncoder.EndEncoding();
		StateCache.ResetBindings();
	}
#endif
}


void FMetalRHICommandContext::BeginBlitEncoder()
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	SCOPE_CYCLE_COUNTER(STAT_MetalSwitchToBlitTime);
	check(!bWithinRenderPass);
	
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	if(!CurrentEncoder.IsBlitCommandEncoderActive())
	{
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
			StateCache.ResetBindings();
		}

		bool bUseStageCounterSamples = Device.SupportsFeature(EMetalFeaturesStageCounterSampling);
		CurrentEncoder.BeginBlitCommandEncoding(bUseStageCounterSamples ? Device.GetCounterSampler() : nullptr);
	}
	
	if (CurrentEncoderFence)
	{
		CurrentEncoder.WaitForFence(CurrentEncoderFence);
		CurrentEncoderFence = nullptr;
	}
	
	check(CurrentEncoder.IsBlitCommandEncoderActive());
}

void FMetalRHICommandContext::EndBlitEncoder()
{
	check(CurrentEncoder.IsBlitCommandEncoderActive());
	
#if WITH_PROFILEGPU
	if(Device.SupportsFeature(EMetalFeaturesStageCounterSampling) && GMetalSampleBlitEncoderTimings)
	{
		CurrentEncoderFence = CurrentEncoder.EndEncoding();
		StateCache.ResetBindings();
	}
#endif
}

void FMetalRHICommandContext::PushDescriptorUpdates()
{
	MTL_SCOPED_AUTORELEASE_POOL;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (PendingDescriptorUpdates.Num())
	{
		check(!bWithinRenderPass);

		Device.GetBindlessDescriptorManager()->FlushPendingDescriptorUpdates(*this, PendingDescriptorUpdates);
		PendingDescriptorUpdates.Empty();
	}
#endif
}

void FMetalRHICommandContext::RHIBeginParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> InInfo, const TCHAR* InName)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	PushDescriptorUpdates();
	
	RenderPassInfo = *StaticCastSharedPtr<FRHIRenderPassInfo>(InInfo);
	
	ParallelRenderPassInfo = new FMetalParallelRenderPassInfo; 
	InInfo->RHIPlatformData = ParallelRenderPassInfo;
	
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	check(CurrentEncoder.GetCommandBuffer());
	
	StateCache.SetStateDirty();
	StateCache.SetRenderTargetsActive(true);
	
	FMetalQueryBufferRef VisBuffer = nullptr;
	if(RenderPassInfo.NumOcclusionQueries > 0)
	{
		VisBuffer = QueryBuffer->AcquireQueryBuffer(RenderPassInfo.NumOcclusionQueries);
	}
		
	StateCache.StartRenderPass(RenderPassInfo, VisBuffer, nullptr, false);
	RenderPassDesc = StateCache.GetRenderPassDescriptor();
	
	if(CurrentEncoder.IsAnyCommandEncoderActive())
	{
		CurrentEncoder.EndEncoding();
	}
	
	check(IsInParallelRenderingThread());
	
	CurrentEncoder.SetRenderPassDescriptor(RenderPassDesc);
	bool bUseStageCounterSamples = Device.SupportsFeature(EMetalFeaturesStageCounterSampling);
	MTLParallelRenderCommandEncoderPtr Encoder = CurrentEncoder.BeginParallelRenderCommandEncoding(bUseStageCounterSamples ? Device.GetCounterSampler() : nullptr);
	
	ParallelRenderPassInfo->ParallelEncoder = Encoder;
	ParallelRenderPassInfo->RenderPassDesc = RenderPassDesc;
	
	if (CurrentEncoderFence)
	{
		CurrentEncoder.WaitForFence(CurrentEncoderFence);
		CurrentEncoderFence = nullptr;
	}

	StateCache.SetRenderStoreActions(CurrentEncoder);
	
	bWithinRenderPass = true;
	
	// Set the viewport to the full size of render target 0.
	if (RenderPassInfo.ColorRenderTargets[0].RenderTarget)
	{
		const FRHIRenderPassInfo::FColorEntry& RenderTargetView = RenderPassInfo.ColorRenderTargets[0];
		FMetalSurface* RenderTarget = GetMetalSurfaceFromRHITexture(RenderTargetView.RenderTarget);

		uint32 Width = FMath::Max((uint32)(RenderTarget->Texture->width() >> RenderTargetView.MipIndex), (uint32)1);
		uint32 Height = FMath::Max((uint32)(RenderTarget->Texture->height() >> RenderTargetView.MipIndex), (uint32)1);

		RHISetViewport(0.0f, 0.0f, 0.0f, (float)Width, (float)Height, 1.0f);
	}
}

void FMetalRHICommandContext::RHIEndParallelRenderPass()
{
	check(bWithinRenderPass);

	StateCache.FlushVisibilityResults(CurrentEncoder);
	
	CurrentEncoder.EndEncoding();
	StateCache.ResetBindings();
	
	bWithinRenderPass = false;
	
	// Uses a Blit encoder so need to run after end encoding 
	UE::RHICore::ResolveRenderPassTargets(RenderPassInfo, [this](UE::RHICore::FResolveTextureInfo Info)
	{
		ResolveTexture(Info);
	});
	
	RenderPassDesc = nullptr;
	
	delete ParallelRenderPassInfo;
}

void FMetalRHICommandContext::RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
{	
    MTL_SCOPED_AUTORELEASE_POOL;

	PushDescriptorUpdates();
    
	RenderPassInfo = InInfo;
	
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	StateCache.SetStateDirty();
	StateCache.SetRenderTargetsActive(true);
	
	FMetalQueryBufferRef VisBuffer = nullptr;
	if(InInfo.NumOcclusionQueries > 0)
	{
		VisBuffer = QueryBuffer->AcquireQueryBuffer(InInfo.NumOcclusionQueries);
	}
	
	StateCache.StartRenderPass(InInfo, VisBuffer, nullptr, false);
	
	RenderPassDesc = StateCache.GetRenderPassDescriptor();
	
	if(!CurrentEncoder.IsRenderCommandEncoderActive())
	{
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
			StateCache.ResetBindings();
		}
		CurrentEncoder.SetRenderPassDescriptor(RenderPassDesc);
		bool bUseStageCounterSamples = Device.SupportsFeature(EMetalFeaturesStageCounterSampling);
		CurrentEncoder.BeginRenderCommandEncoding(bUseStageCounterSamples ? Device.GetCounterSampler() : nullptr);
	}
	
	if (CurrentEncoderFence)
	{
		CurrentEncoder.WaitForFence(CurrentEncoderFence);
		CurrentEncoderFence = nullptr;
	}
	StateCache.SetRenderStoreActions(CurrentEncoder, false);
	check(CurrentEncoder.IsRenderCommandEncoderActive());

	bWithinRenderPass = true;
	
	// Set the viewport to the full size of render target 0.
	if (InInfo.ColorRenderTargets[0].RenderTarget)
	{
		const FRHIRenderPassInfo::FColorEntry& RenderTargetView = InInfo.ColorRenderTargets[0];
		FMetalSurface* RenderTarget = GetMetalSurfaceFromRHITexture(RenderTargetView.RenderTarget);

		uint32 Width = FMath::Max((uint32)(RenderTarget->Texture->width() >> RenderTargetView.MipIndex), (uint32)1);
		uint32 Height = FMath::Max((uint32)(RenderTarget->Texture->height() >> RenderTargetView.MipIndex), (uint32)1);

		RHISetViewport(0.0f, 0.0f, 0.0f, (float)Width, (float)Height, 1.0f);
	}
}

void FMetalRHICommandContext::RHIEndRenderPass()
{
	check(bWithinRenderPass);
	check(CurrentEncoder.IsRenderCommandEncoderActive());
	
	StateCache.FlushVisibilityResults(CurrentEncoder);
	
	CurrentEncoderFence = CurrentEncoder.EndEncoding();
	
	bWithinRenderPass = false;
	
	// Uses a Blit encoder so need to run after end encoding 
	UE::RHICore::ResolveRenderPassTargets(RenderPassInfo, [this](UE::RHICore::FResolveTextureInfo Info)
	{
		ResolveTexture(Info);
	});
	
	StateCache.EndRenderPass();
	StateCache.ResetBindings();
	RenderPassDesc = nullptr;
}

void FMetalRHICommandContext::ResolveTexture(UE::RHICore::FResolveTextureInfo Info)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalSurface* Source = GetMetalSurfaceFromRHITexture(Info.SourceTexture);
	FMetalSurface* Destination = GetMetalSurfaceFromRHITexture(Info.DestTexture);

	const FRHITextureDesc& SourceDesc = Source->GetDesc();
	const FRHITextureDesc& DestinationDesc = Destination->GetDesc();

	const bool bDepthStencil = SourceDesc.Format == PF_DepthStencil;
	const bool bSupportsMSAADepthResolve = Device.SupportsFeature(EMetalFeaturesMSAADepthResolve);
	const bool bSupportsMSAAStoreAndResolve = Device.SupportsFeature(EMetalFeaturesMSAAStoreAndResolve);
	// Resolve required - Device must support this - Using Shader for resolve not supported amd NumSamples should be 1
	check((!bDepthStencil && bSupportsMSAAStoreAndResolve) || (bDepthStencil && bSupportsMSAADepthResolve));

	MTL::Origin Origin(0, 0, 0);
    MTL::Size Size(0, 0, 1);

	if (Info.ResolveRect.IsValid())
	{
		Origin.x    = Info.ResolveRect.X1;
		Origin.y    = Info.ResolveRect.Y1;
		Size.width  = Info.ResolveRect.X2 - Info.ResolveRect.X1;
		Size.height = Info.ResolveRect.Y2 - Info.ResolveRect.Y1;
	}
	else
	{
		Size.width  = FMath::Max<uint32>(1, SourceDesc.Extent.X >> Info.MipLevel);
		Size.height = FMath::Max<uint32>(1, SourceDesc.Extent.Y >> Info.MipLevel);
	}

	int32 ArraySliceBegin = Info.ArraySlice;
	int32 ArraySliceEnd   = Info.ArraySlice + 1;

	if (Info.ArraySlice < 0)
	{
		ArraySliceBegin = 0;
		ArraySliceEnd   = SourceDesc.ArraySize;
	}

	BeginBlitEncoder();
	
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	
	check(Encoder);
	
	for (int32 ArraySlice = ArraySliceBegin; ArraySlice < ArraySliceEnd; ArraySlice++)
	{
		Encoder->copyFromTexture(Source->MSAAResolveTexture.get(), ArraySlice, Info.MipLevel, Origin, Size, Destination->Texture.get(), ArraySlice, Info.MipLevel, Origin);
	}
	
	EndBlitEncoder();
}

void FMetalRHICommandContext::RHINextSubpass()
{
#if PLATFORM_MAC
	if (RenderPassInfo.SubpassHint == ESubpassHint::DepthReadSubpass)
	{
		if (CurrentEncoder.IsRenderCommandEncoderActive())
		{
			MTL::RenderCommandEncoder* RenderEncoder = CurrentEncoder.GetRenderCommandEncoder();
			check(RenderEncoder);
			RenderEncoder->memoryBarrier(MTL::BarrierScopeRenderTargets, MTL::RenderStageFragment, MTL::RenderStageVertex);
		}
	}
#endif
}

void FMetalRHICommandContext::FillBuffer(MTL::Buffer* Buffer, NS::Range Range, uint8 Value)
{
	check(Buffer);
	BeginBlitEncoder();

	MTL::BlitCommandEncoder* TargetEncoder = CurrentEncoder.GetBlitCommandEncoder();
	check(TargetEncoder);
	
	TargetEncoder->fillBuffer(Buffer, Range, Value);
	
	EndBlitEncoder();
}

void FMetalRHICommandContext::CopyFromTextureToBuffer(MTL::Texture* Texture, uint32 sourceSlice, uint32 sourceLevel, MTL::Origin sourceOrigin, MTL::Size sourceSize, FMetalBufferPtr toBuffer, uint32 destinationOffset, uint32 destinationBytesPerRow, uint32 destinationBytesPerImage, MTL::BlitOption options)
{
	BeginBlitEncoder();
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	check(Encoder);

	if (Texture)
	{
		Encoder->copyFromTexture(
			Texture,
			sourceSlice,
			sourceLevel,
			sourceOrigin,
			sourceSize,
			toBuffer->GetMTLBuffer(),
			destinationOffset + toBuffer->GetOffset(),
			destinationBytesPerRow,
			destinationBytesPerImage,
			options
		);
	}

	EndBlitEncoder();
}

void FMetalRHICommandContext::CopyFromBufferToTexture(FMetalBufferPtr Buffer, uint32 sourceOffset, uint32 sourceBytesPerRow, uint32 sourceBytesPerImage, MTL::Size sourceSize, MTL::Texture* toTexture, uint32 destinationSlice, uint32 destinationLevel, MTL::Origin destinationOrigin, MTL::BlitOption options)
{
	BeginBlitEncoder();
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	check(Encoder);

	if (options == MTL::BlitOptionNone)
	{
		Encoder->copyFromBuffer(Buffer->GetMTLBuffer(), sourceOffset + Buffer->GetOffset(), sourceBytesPerRow, sourceBytesPerImage, sourceSize,
								toTexture, destinationSlice, destinationLevel, destinationOrigin);
	}
	else
	{
		Encoder->copyFromBuffer(Buffer->GetMTLBuffer(), sourceOffset + Buffer->GetOffset(), sourceBytesPerRow, sourceBytesPerImage, sourceSize,
								toTexture, destinationSlice, destinationLevel, destinationOrigin, options);
	}
	
	EndBlitEncoder();
}

void FMetalRHICommandContext::CopyFromTextureToTexture(MTL::Texture* Texture, uint32 sourceSlice, uint32 sourceLevel, MTL::Origin sourceOrigin, MTL::Size sourceSize, MTL::Texture* toTexture, uint32 destinationSlice, uint32 destinationLevel, MTL::Origin destinationOrigin)
{
	BeginBlitEncoder();
	
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	check(Encoder);

	Encoder->copyFromTexture(Texture, sourceSlice, sourceLevel, sourceOrigin, sourceSize, toTexture, destinationSlice, destinationLevel, destinationOrigin);
	
	EndBlitEncoder();
}

void FMetalRHICommandContext::CopyFromBufferToBuffer(FMetalBufferPtr SourceBuffer, NS::UInteger SourceOffset, FMetalBufferPtr DestinationBuffer, NS::UInteger DestinationOffset, NS::UInteger Size)
{
	BeginBlitEncoder();
	
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	check(Encoder);
	
	Encoder->copyFromBuffer(
		SourceBuffer->GetMTLBuffer(),
		SourceOffset + SourceBuffer->GetOffset(),
		DestinationBuffer->GetMTLBuffer(),
		DestinationOffset + DestinationBuffer->GetOffset(),
		Size
	);
	
	EndBlitEncoder();
}

void FMetalRHICommandContext::Finalize(TArray<FMetalPayload*>& OutPayloads)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	if(CurrentEncoder.IsAnyCommandEncoderActive())
	{
		if(CurrentEncoder.IsRenderCommandEncoderActive())
		{
			RHIEndRenderPass();
		}
		else
		{
			CurrentEncoder.EndEncoding();
		}
	}
	
	PushDescriptorUpdates();
	
	// No command buffer if we are running parallel
	if (CurrentEncoder.GetCommandBuffer())
	{
		check(!bIsParallelContext);
		EndCommandBuffer();
	}
	
	// Collect the context's batch of sync points to wait/signal
	if (BatchedSyncPoints.ToWait.Num())
	{
		FMetalPayload* Payload = Payloads.Num()
			? Payloads[0]
			: GetPayload(EPhase::Wait);

		Payload->SyncPointsToWait.Append(BatchedSyncPoints.ToWait);
		BatchedSyncPoints.ToWait.Reset();
	}

	if (BatchedSyncPoints.ToSignal.Num())
	{
		GetPayload(EPhase::Signal)->SyncPointsToSignal.Append(BatchedSyncPoints.ToSignal);
		BatchedSyncPoints.ToSignal.Reset();
	}
	
	ContextSyncPoint = nullptr;
	
	OutPayloads.Append(MoveTemp(Payloads));
}

void FMetalRHICommandContext::SignalSyncPoint(FMetalSyncPoint* SyncPoint)
{
	if(CurrentEncoder.GetCommandBuffer())
	{
		EndCommandBuffer();
	}
 
	GetPayload(EPhase::Signal)->SyncPointsToSignal.Add(SyncPoint);
}

void FMetalRHICommandContext::WaitSyncPoint(FMetalSyncPoint* SyncPoint)
{
	if(CurrentEncoder.GetCommandBuffer())
	{
		EndCommandBuffer();
	}
	
	GetPayload(EPhase::Wait)->SyncPointsToWait.Add(SyncPoint);
}

void FMetalRHICommandContext::SignalEvent(MTLEventPtr Event, uint32_t SignalCount)
{
	if(!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
	}
	CurrentEncoder.SignalEvent(Event, SignalCount);
}

void FMetalRHICommandContext::WaitForEvent(MTLEventPtr Event, uint32_t SignalCount)
{
	if(!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
	}
	CurrentEncoder.WaitForEvent(Event, SignalCount);
}

void FMetalRHICommandContext::SignalSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount)
{
	if(!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
	}
	CurrentEncoder.SignalEvent(Event, SignalCount);
}

void FMetalRHICommandContext::WaitForSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount)
{
	if(!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
	}
	CurrentEncoder.WaitForEvent(Event, SignalCount);
}

void FMetalRHICommandContext::StartCommandBuffer()
{
	check(!CurrentEncoder.GetCommandBuffer());
	
	CurrentEncoder.StartCommandBuffer();
	
	auto& Event = CurrentEncoder.GetCommandBuffer()->EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(0);
	Event.GPUTimestampTOP = 0;
	CurrentEncoder.GetCommandBuffer()->SetBeginWorkTimestamp(&Event.GPUTimestampTOP);
	
	// Add new command buffer to payload
	GetPayload(EPhase::Execute)->CommandBuffersToExecute.Add(CurrentEncoder.GetCommandBuffer());
}

void FMetalRHICommandContext::EndCommandBuffer()
{
	check(CurrentEncoder.GetCommandBuffer());
	check(!bWithinRenderPass);
	
	auto& Event = CurrentEncoder.GetCommandBuffer()->EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>();
	Event.GPUTimestampBOP = 0;
	CurrentEncoder.GetCommandBuffer()->SetEndWorkTimestamp(&Event.GPUTimestampBOP);
	
	if(CurrentEncoder.IsAnyCommandEncoderActive())
	{
		CurrentEncoder.EndEncoding();
	}
	CurrentEncoder.EndCommandBuffer(this);
}

void FMetalRHICommandContext::SynchronizeResource(MTL::Resource* Resource)
{
	check(Resource);
#if PLATFORM_MAC
	BeginBlitEncoder();
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	check(Encoder);

	Encoder->synchronizeResource(Resource);
	EndBlitEncoder();
#endif
}

void FMetalRHICommandContext::SynchronizeTexture(MTL::Texture* Texture, uint32 Slice, uint32 Level)
{
	check(Texture);
#if PLATFORM_MAC
	BeginBlitEncoder();
	MTL::BlitCommandEncoder* Encoder = CurrentEncoder.GetBlitCommandEncoder();
	check(Encoder);

	Encoder->synchronizeTexture(Texture, Slice, Level);
	EndBlitEncoder();
#endif
}

FMetalCommandBuffer* FMetalRHICommandContext::GetCurrentCommandBuffer()
{
	FMetalCommandBuffer* CommandBuffer = CurrentEncoder.GetCommandBuffer();
	if(!CommandBuffer)
	{
		StartCommandBuffer();
	}
	return CurrentEncoder.GetCommandBuffer();
}

void FMetalRHICommandContext::FlushProfilerStats()
{
	// Flush accumulated draw stats
	if (StatEvent)
	{
		CurrentEncoder.GetCommandBuffer()->EmplaceProfilerEvent<UE::RHI::GPUProfiler::FEvent::FStats>() = StatEvent;
		StatEvent = {};
	}
}

void FMetalRHICommandContext::FlushCommands(EMetalFlushFlags FlushFlags)
{
	FMetalSyncPointRef SyncPoint;
	if (EnumHasAnyFlags(FlushFlags, EMetalFlushFlags::WaitForCompletion))
	{
		SyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);
		SignalSyncPoint(SyncPoint);
	}

	FGraphEventRef SubmissionEvent;
	if (EnumHasAnyFlags(FlushFlags, EMetalFlushFlags::WaitForSubmission))
	{
		SubmissionEvent = FGraphEvent::CreateGraphEvent();
		GetPayload(EPhase::Signal)->SubmissionEvent = SubmissionEvent;
	}

	FMetalFinalizedCommands* FinalizedPayloads = new FMetalFinalizedCommands;
	Finalize(*FinalizedPayloads);

	FDynamicRHI::FRHISubmitCommandListsArgs Args;
	Args.CommandLists.Add(FinalizedPayloads);
	FMetalDynamicRHI::Get().RHISubmitCommandLists(MoveTemp(Args));

	if (SyncPoint)
	{
		SyncPoint->Wait();
	}

	if (SubmissionEvent && !SubmissionEvent->IsComplete())
	{
		SCOPED_NAMED_EVENT_TEXT("Submission_Wait", FColor::Turquoise);
		SubmissionEvent->Wait();
	}
}

FMetalRHIUploadContext::FMetalRHIUploadContext(FMetalDevice& Device)
{
	UploadContext = new FMetalRHICommandContext(Device);
	UploadContext->ResetContext();
}

FMetalRHIUploadContext::~FMetalRHIUploadContext()
{
	delete UploadContext;
}

void FMetalRHIUploadContext::Finalize(TArray<FMetalPayload*>& OutPayloads)
{
	for(auto& Function : UploadFunctions)
	{
		Function(UploadContext);
	}
	
	UploadContext->Finalize(OutPayloads);
	
	UploadFunctions.Reset();
	UploadContext->ResetContext();
}

FMetalContextArray::FMetalContextArray(FRHIContextArray const& Contexts)
	: TRHIPipelineArray(InPlace, nullptr)
{
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		IRHIComputeContext* Context = Contexts[Pipeline];

		switch (Pipeline)
		{
		default:
			checkNoEntry();
			break;

		case ERHIPipeline::Graphics:
			(*this)[Pipeline] = Context ? static_cast<FMetalRHICommandContext*>(&Context->GetLowestLevelContext()) : nullptr;
			break;

		case ERHIPipeline::AsyncCompute:
			(*this)[Pipeline] = Context ? static_cast<FMetalRHICommandContext*>(&Context->GetLowestLevelContext()) : nullptr;
			break;
		}
	}
}
