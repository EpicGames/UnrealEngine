// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHIPrivate.h: Private Metal RHI definitions.
=============================================================================*/

#pragma once

#include "MetalRHI.h"
#include "MetalThirdParty.h"
#include "PixelFormat.h"
#include "RHICommandList.h"
#include "RHIGlobals.h"

class FMetalDevice;
class FMetalSurface;

DECLARE_LOG_CATEGORY_EXTERN(LogMetal, Display, All);

DECLARE_DELEGATE_OneParam(FMetalCommandBufferCompletionHandler, MTL::CommandBuffer*);

// Whether the Metal RHI is initialized sufficiently to handle resources
extern bool GIsMetalInitialized;

// Requirement for vertex buffer offset field
#if PLATFORM_MAC
const uint32 BufferOffsetAlignment = 256;
const uint32 BufferBackedLinearTextureOffsetAlignment = 1024;
#else
const uint32 BufferOffsetAlignment = 16;
const uint32 BufferBackedLinearTextureOffsetAlignment = 64;
#endif

// The maximum buffer page size that can be uploaded in a set*Bytes call
const uint32 MetalBufferPageSize = 4096;

// The buffer size that is more efficiently uploaded in a set*Bytes call - defined in terms of BufferOffsetAlignment
#if PLATFORM_MAC
const uint32 MetalBufferBytesSize = BufferOffsetAlignment * 2;
#else
const uint32 MetalBufferBytesSize = BufferOffsetAlignment * 32;
#endif

#ifndef METAL_USE_METAL_SHADER_CONVERTER
#define METAL_USE_METAL_SHADER_CONVERTER 0
#endif

#define METAL_RHI_RAYTRACING (METAL_USE_METAL_SHADER_CONVERTER)

// Metal Shader Converter
#if METAL_USE_METAL_SHADER_CONVERTER
THIRD_PARTY_INCLUDES_START
    #include "metal_irconverter.h"
    #define IR_RUNTIME_METALCPP 1
    #include "metal_irconverter_runtime.h"
THIRD_PARTY_INCLUDES_END

constexpr uint64_t kIRStandardHeapBindPoint                = 0;

#endif

#define BUFFER_CACHE_MODE MTL::ResourceCPUCacheModeDefaultCache

#if PLATFORM_MAC
#define BUFFER_MANAGED_MEM MTL::ResourceStorageModeManaged
#define BUFFER_STORAGE_MODE MTL::StorageModeShared
#define BUFFER_RESOURCE_STORAGE_MANAGED MTL::ResourceStorageModeManaged
#define BUFFER_DYNAMIC_REALLOC BUF_AnyDynamic
// How many possible vertex streams are allowed
const uint32 MaxMetalStreams = 31;
#else
#define BUFFER_MANAGED_MEM 0

#if WITH_IOS_SIMULATOR
#define BUFFER_STORAGE_MODE MTL::StorageModePrivate
#define BUFFER_RESOURCE_STORAGE_MANAGED MTL::ResourceStorageModePrivate
#else
#define BUFFER_STORAGE_MODE MTL::StorageModeShared
#define BUFFER_RESOURCE_STORAGE_MANAGED MTL::ResourceStorageModeShared
#endif

#define BUFFER_DYNAMIC_REALLOC BUF_AnyDynamic
// How many possible vertex streams are allowed
const uint32 MaxMetalStreams = 30;
#endif

// Unavailable on iOS, but dealing with this clutters the code.
enum EMTLTextureType
{
	EMTLTextureTypeCubeArray = 6
};

// This is the right VERSION check, see Availability.h in the SDK
#define METAL_SUPPORTS_INDIRECT_ARGUMENT_BUFFERS 1
#define METAL_SUPPORTS_CAPTURE_MANAGER 1
#define METAL_SUPPORTS_TILE_SHADERS 1

// In addition to compile-time SDK checks we also need a way to check if these are available on runtime
extern bool GMetalSupportsCaptureManager;

struct FMetalBufferFormat
{
	// Valid linear texture pixel formats - potentially different than the actual texture formats
	MTL::PixelFormat LinearTextureFormat;
	// Metal buffer data types for manual ALU format conversions
	uint8 DataFormat;
};

extern FMetalBufferFormat GMetalBufferFormats[PF_MAX];

#define METAL_DEBUG_OPTIONS !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#if METAL_DEBUG_OPTIONS
    #define METAL_DEBUG_OPTION(Code) Code
#else
    #define METAL_DEBUG_OPTION(Code)
#endif

extern bool GMetalCommandBufferDebuggingEnabled;

/** Set to 1 to enable GPU events in Xcode frame debugger */
#ifndef ENABLE_METAL_GPUEVENTS_IN_TEST
    #define ENABLE_METAL_GPUEVENTS_IN_TEST 0
#endif
#define ENABLE_METAL_GPUEVENTS     (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || (UE_BUILD_TEST && ENABLE_METAL_GPUEVENTS_IN_TEST))
#define ENABLE_METAL_GPUPROFILE    (ENABLE_METAL_GPUEVENTS && 1)

#define UNREAL_TO_METAL_BUFFER_INDEX(Index) ((MaxMetalStreams - 1) - Index)
#define METAL_TO_UNREAL_BUFFER_INDEX(Index) ((MaxMetalStreams - 1) - Index)

#define METAL_NEW_NONNULL_DECL (__clang_major__ >= 9)

#if PLATFORM_IOS
#define METAL_FATAL_ERROR(Format, ...)  { UE_LOG(LogMetal, Warning, Format, __VA_ARGS__); FIOSPlatformMisc::MetalAssert(); }
#else
#define METAL_FATAL_ERROR(Format, ...)	UE_LOG(LogMetal, Fatal, Format, __VA_ARGS__)
#endif
#define METAL_FATAL_ASSERT(Condition, Format, ...) if (!(Condition)) { METAL_FATAL_ERROR(Format, __VA_ARGS__); }

#if !defined(METAL_IGNORED)
	#define METAL_IGNORED(Func)
#endif

FORCEINLINE bool IsMetalBindlessEnabled()
{
	return GRHIGlobals.bSupportsBindless;
}

// Access the underlying surface object from any kind of texture
FMetalSurface* GetMetalSurfaceFromRHITexture(FRHITexture* Texture);

#define NOT_SUPPORTED(Func) UE_LOGF(LogMetal, Fatal, "'%ls' is not supported", TEXT(Func));

FORCEINLINE bool MetalIsSafeToUseRHIThreadResources()
{
	// we can use RHI thread resources if we are on the RHIThread or on RenderingThread when there's no RHI thread, or the RHI thread is stalled or inactive
	return (GIsMetalInitialized && !GIsRHIInitialized)
		|| IsInRHIThread()
		|| (IsInRenderingThread() && (!IsRunningRHIInSeparateThread() || !FRHICommandListExecutor::AreRHITasksActive() || FRHICommandListImmediate::IsStalled()));
}

FORCEINLINE int32 GetMetalCubeFace(ECubeFace Face)
{
	// According to Metal docs these should match now: https://developer.apple.com/library/prerelease/ios/documentation/Metal/Reference/MTLTexture_Ref/index.html#//apple_ref/c/tdef/MTLTextureType
	switch (Face)
	{
		case CubeFace_PosX:;
		default:			return 0;
		case CubeFace_NegX:	return 1;
		case CubeFace_PosY:	return 2;
		case CubeFace_NegY:	return 3;
		case CubeFace_PosZ:	return 4;
		case CubeFace_NegZ:	return 5;
	}
}

FORCEINLINE MTL::LoadAction GetMetalRTLoadAction(ERenderTargetLoadAction LoadAction)
{
	switch(LoadAction)
	{
		case ERenderTargetLoadAction::ENoAction: return MTL::LoadActionDontCare;
		case ERenderTargetLoadAction::ELoad: return MTL::LoadActionLoad;
		case ERenderTargetLoadAction::EClear: return MTL::LoadActionClear;
		default: return MTL::LoadActionDontCare;
	}
}

MTL::PrimitiveType TranslatePrimitiveType(uint32 PrimitiveType);
MTL::PrimitiveTopologyClass TranslatePrimitiveTopology(uint32 PrimitiveType);

MTL::PixelFormat UEToMetalFormat(FMetalDevice& Device, EPixelFormat UEFormat, bool bSRGB);

uint8 GetMetalPixelFormatKey(MTL::PixelFormat Format);

template<class T>
struct TMetalResourceTraits
{
};

template<typename TRHIType>
static FORCEINLINE typename TMetalResourceTraits<TRHIType>::TConcreteType* ResourceCast(TRHIType* Resource)
{
	return static_cast<typename TMetalResourceTraits<TRHIType>::TConcreteType*>(Resource);
}

template<typename TRHIType>
static FORCEINLINE const typename TMetalResourceTraits<TRHIType>::TConcreteType* ResourceCast(const TRHIType* Resource)
{
	return static_cast<const typename TMetalResourceTraits<TRHIType>::TConcreteType*>(Resource);
}

static FORCEINLINE FMetalSurface* ResourceCast(FRHITexture* Texture)
{
	return GetMetalSurfaceFromRHITexture(Texture);
}

static FORCEINLINE const FMetalSurface* ResourceCast(const FRHITexture* Texture)
{
	unimplemented();
	return nullptr;
}

MTL::LanguageVersion ValidateVersion(uint32 Version);

// Needs to be the same as EShaderFrequency when all stages are supported, but unlike EShaderFrequency you can compile out stages.
enum EMetalShaderStages
{
	Vertex,
	Pixel,
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	Geometry,
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	Mesh,
	Amplification,
#endif
	Compute,
	
	Num,
};

FORCEINLINE EShaderFrequency GetRHIShaderFrequency(EMetalShaderStages Stage)
{
	switch (Stage)
	{
		case EMetalShaderStages::Vertex:
			return SF_Vertex;
		case EMetalShaderStages::Pixel:
			return SF_Pixel;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		case EMetalShaderStages::Geometry:
			return SF_Geometry;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case EMetalShaderStages::Mesh:
			return SF_Mesh;
		case EMetalShaderStages::Amplification:
			return SF_Amplification;
#endif
		case EMetalShaderStages::Compute:
			return SF_Compute;
		default:
			return SF_NumFrequencies;
	}
}

FORCEINLINE EMetalShaderStages GetMetalShaderFrequency(EShaderFrequency Stage)
{
	switch (Stage)
	{
		case SF_Vertex:
			return EMetalShaderStages::Vertex;
		case SF_Pixel:
			return EMetalShaderStages::Pixel;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		case SF_Geometry:
			return EMetalShaderStages::Geometry;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case SF_Mesh:
			return EMetalShaderStages::Mesh;
		case SF_Amplification:
			return EMetalShaderStages::Amplification;
#endif
		case SF_Compute:
			return EMetalShaderStages::Compute;
		default:
			return EMetalShaderStages::Num;
	}
}

FORCEINLINE FString NSStringToFString(NS::String* InputString)
{
    return FString((__bridge CFStringRef)InputString);
}

FORCEINLINE NS::String* FStringToNSString(const FString& InputString)
{
    return ((NS::String*)InputString.GetCFString())->autorelease();
}

// helper functions to reduce copy and pasted checks all around
bool ShouldUseMemoryless(ETextureCreateFlags Flags);
bool AllowMSAA();
