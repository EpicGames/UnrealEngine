// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHIRenderQuery.cpp: Metal RHI Render Query Implementation.
=============================================================================*/

#include "MetalRHIRenderQuery.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalDevice.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "MetalProfiler.h"
#include "MetalLLM.h"
#include "MetalCommandBuffer.h"
#include "MetalRHIContext.h"
#include "HAL/PThreadEvent.h"
#include "RenderCore.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Buffer Resource Class -

static int32 GMetalMaxQueryBufferSize = 1024 * 256;
static FAutoConsoleVariableRef CVarMetalMaxQueryBufferSize(
	TEXT("rhi.Metal.MaxQueryBufferSize"),
	GMetalMaxQueryBufferSize,
	TEXT("Maximum size of the the query buffer in a single context. Default = 512kb on Mac and 256k on iOS"),
	ECVF_ReadOnly);

static int32 GQueryUseControlPrecisionBit = 1;
FAutoConsoleVariableRef CVarQueryUseControlPrecisionBit(
	TEXT("rhi.Metal.QueryUseControlPrecisionBit"),
	GQueryUseControlPrecisionBit,
	TEXT("Whether to use VisibilityResultModeCounting (if supported) when doing a render query. (Default : 1 (on), 0 (off))\n"),
	ECVF_ReadOnly
);

FMetalQueryBuffer::FMetalQueryBuffer(FMetalQueryBufferPool* InPool, FMetalBufferPtr InBuffer)
	: Pool(InPool)
	, Buffer(InBuffer)
	, WriteOffset(0)
{
	// void
}

FMetalQueryBuffer::~FMetalQueryBuffer()
{
	if (GIsMetalInitialized && Buffer && Pool)
	{
		Pool->ReleaseQueryBuffer(Buffer);
	}
}

uint64 FMetalQueryBuffer::GetResult(uint32 Offset)
{
	uint64 Result = 0;
	
	check(Buffer.IsValid());
	if(Buffer.IsValid())
	{
		MTL_SCOPED_AUTORELEASE_POOL;
		{
			Result = *((uint64 const*)(((uint8*)Buffer->Contents()) + Offset));
		}
	}

	return Result;
}


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Buffer Pool Class -


FMetalQueryBufferPool::FMetalQueryBufferPool(FMetalDevice& InDevice)
	: CurrentBuffer{nullptr}
	, Buffers{}
	, Device{InDevice}
{
	BufferSize = GMetalMaxQueryBufferSize;
	if (!Device.GetDevice()->supportsFamily(MTL::GPUFamilyApple7))
	{
		// On A13 and below devices the offset in setVisibilityResultMode needs to be <= 65528
		BufferSize = FMath::Min(65528, GMetalMaxQueryBufferSize);
	}
}

FMetalQueryBufferPool::~FMetalQueryBufferPool()
{
	// void
}

void FMetalQueryBufferPool::Allocate(FMetalQueryResult& NewQuery)
{
	check(IsValidRef(CurrentBuffer));
	FMetalQueryBufferRef QB = CurrentBuffer;

	uint32 Offset = Align(QB->WriteOffset, EQueryBufferAlignment);
	uint32 End = Align(Offset + EQueryResultMaxSize, EQueryBufferAlignment);

	if (Align(QB->WriteOffset, EQueryBufferAlignment) + EQueryResultMaxSize <= BufferSize)
	{
		NewQuery.SourceBuffer = QB;
		NewQuery.Offset = Align(QB->WriteOffset, EQueryBufferAlignment);
		QB->WriteOffset = Align(QB->WriteOffset, EQueryBufferAlignment) + EQueryResultMaxSize;
	}
}

FMetalQueryBufferRef FMetalQueryBufferPool::AcquireQueryBuffer(uint32_t NumOcclusionQueries)
{
	uint32_t RequiredSize = NumOcclusionQueries * EQueryResultMaxSize;
	
	if(CurrentBuffer)
	{
		// If we currently have a buffer and the results fit, then use it
		if(Align(CurrentBuffer->WriteOffset, EQueryBufferAlignment) + RequiredSize <= BufferSize)
		{
			return CurrentBuffer;
		}
		else
		{
			ReleaseCurrentQueryBuffer();
		}
	}
	
	// Need to resize if queries don't fit in our current buffer size
	if(RequiredSize > BufferSize)
	{
		BufferSize = FMath::RoundUpToPowerOfTwo(RequiredSize);
	
		for(FMetalBufferPtr Buffer : Buffers)
		{
			FMetalDynamicRHI::Get().DeferredDelete(Buffer);
		}
		
		Buffers.Empty();
	}
	
	FMetalBufferPtr Buffer;
	
	if (Buffers.Num())
	{
		Buffer = Buffers.Pop();
	}
	else
	{		
		MTL::ResourceOptions HazardTrackingMode = MTL::ResourceHazardTrackingModeUntracked;
		static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
		if(bSupportsHeaps)
		{
			HazardTrackingMode = MTL::ResourceHazardTrackingModeTracked;
		}
		
		Buffer = Device.GetResourceHeap().CreateBuffer(BufferSize, 16, BUF_Dynamic, FMetalCommandQueue::GetCompatibleResourceOptions((MTL::ResourceOptions)(BUFFER_CACHE_MODE | HazardTrackingMode | MTL::ResourceStorageModeShared)), true);
		
		FMemory::Memzero((((uint8*)Buffer->Contents())), BufferSize);
	}
	
	check(Buffer);
	
	CurrentBuffer = new FMetalQueryBuffer(this, MoveTemp(Buffer));
	return CurrentBuffer;
}

FMetalQueryBufferRef FMetalQueryBufferPool::GetCurrentQueryBuffer()
{
	return CurrentBuffer;
}

void FMetalQueryBufferPool::ReleaseCurrentQueryBuffer()
{
	if (IsValidRef(CurrentBuffer) && (CurrentBuffer->WriteOffset > 0))
	{
		// Keep a reference of this until we know the GPU is finished.
		FMetalDynamicRHI::Get().DeferredDelete([InBuffer = MoveTemp(CurrentBuffer)]() mutable
		{
			InBuffer = nullptr;
		});
	}
}

void FMetalQueryBufferPool::ReleaseQueryBuffer(FMetalBufferPtr Buffer)
{
	if(Buffer->GetLength() >= BufferSize)
	{
		Buffers.Add(Buffer);
	}
	else
	{
		FMetalDynamicRHI::Get().DeferredDelete(Buffer);
	}
}


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Result Class -

void FMetalQueryResult::Reset()
{
	bCompleted = false;
}

uint64 FMetalQueryResult::GetResult()
{
	if (IsValidRef(SourceBuffer))
	{
		return SourceBuffer->GetResult(Offset);
	}
	return 0;
}

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Command Context Functions -

void FMetalDynamicRHI::RHIBeginRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	FMetalRHIRenderQuery* Query = ResourceCast(RenderQuery);
	Query->Begin_TopOfPipe();

	FDynamicRHI::RHIBeginRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FMetalDynamicRHI::RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	FMetalRHIRenderQuery* Query = ResourceCast(RenderQuery);
	auto& QueryBatchData = RHICmdList.GetQueryBatchData(Query->Type);

	if (QueryBatchData[0])
	{
		// This query belongs to a batch. Use the sync point we created earlier
		Query->SyncPoint = static_cast<FMetalSyncPoint*>(QueryBatchData[0]);
	}
	else
	{
		// Queries issued outside of a batch use one sync point per query.
		Query->SyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);

		RHICmdList.EnqueueLambda([SyncPoint = Query->SyncPoint](FRHICommandListBase& RHICmdList) mutable
		{
			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
			Context.BatchedSyncPoints.ToSignal.Emplace(MoveTemp(SyncPoint));
		});
	}
	
	FDynamicRHI::RHIEndRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FMetalDynamicRHI::RHIBeginRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType)
{
	auto& QueryBatchData = RHICmdList.GetQueryBatchData(QueryType);
	FMetalSyncPointRef QuerySyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);
	
	QueryBatchData[0] = QuerySyncPoint.GetReference();
	QuerySyncPoint->AddRef();
}

void FMetalDynamicRHI::RHIEndRenderQueryBatch_TopOfPipe(FRHICommandListBase& RHICmdList, ERenderQueryType QueryType)
{
	auto& QueryBatchData = RHICmdList.GetQueryBatchData(QueryType);
	checkf(QueryBatchData[0], TEXT("A query batch for this type is not open on this command list."));

	FMetalSyncPointRef SyncPoint = static_cast<FMetalSyncPoint*>(QueryBatchData[0]);

	// Clear the sync point reference on the RHI command list
	SyncPoint->Release();
	QueryBatchData[0] = nullptr;

	RHICmdList.EnqueueLambda([SyncPoint = MoveTemp(SyncPoint), QueryType](FRHICommandListBase& ExecutingCmdList)
	{
		FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(ExecutingCmdList);
		Context.BatchedSyncPoints.ToSignal.Add(SyncPoint);
	});
}

void FMetalRHICommandContext::RHIBeginRenderQuery(FRHIRenderQuery* QueryRHI)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	FMetalRHIRenderQuery* Query = ResourceCast(QueryRHI);
	Query->Begin(this);
}

void FMetalRHICommandContext::RHIEndRenderQuery(FRHIRenderQuery* QueryRHI)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	FMetalRHIRenderQuery* Query = ResourceCast(QueryRHI);
	Query->End(this);
}

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Render Query Class -


FMetalRHIRenderQuery::FMetalRHIRenderQuery(FMetalDevice& MetalDevice, ERenderQueryType InQueryType)
	: Device(MetalDevice)
	, Type{InQueryType}
	, Buffer{}
	, Result{0}
	, bAvailable{false}
{
	// void
}

FMetalRHIRenderQuery::~FMetalRHIRenderQuery()
{
	Buffer.Offset = 0;
}

void FMetalRHIRenderQuery::Begin_TopOfPipe()
{
	Buffer.Reset();
	bAvailable = false;
}

void FMetalRHIRenderQuery::End_TopOfPipe()
{
	if (Type == RQT_AbsoluteTime)
	{
		Buffer.Reset();
	}
	bAvailable = false;
}

void FMetalRHIRenderQuery::Begin(FMetalRHICommandContext* Context)
{
	Buffer.SourceBuffer = nullptr;
	Buffer.Offset = 0;

	Result = 0;
	bAvailable = false;

	switch (Type)
	{
		case RQT_Occlusion:
		{
			// allocate our space in the current buffer
			Context->GetQueryBufferPool()->Allocate(Buffer);

			// AddRef to ensure that the object isn't freed until after the results are sampled
			AddRef();

			Buffer.bCompleted = false;

			if (Device.SupportsFeature(EMetalFeaturesCountingQueries) && (GQueryUseControlPrecisionBit > 0))
			{
				Context->GetStateCache().SetVisibilityResultMode(MTL::VisibilityResultModeCounting, Buffer.Offset);
			}
			else
			{
				Context->GetStateCache().SetVisibilityResultMode(MTL::VisibilityResultModeBoolean, Buffer.Offset);
			}
			break;
		}
		case RQT_AbsoluteTime:
		{
			break;
		}
		default:
		{
			check(0);
			break;
		}
	}
}

void FMetalRHIRenderQuery::End(FMetalRHICommandContext* Context)
{
	switch (Type)
	{
		case RQT_Occlusion:
		{
			// switch back to non-occlusion rendering
			Context->GetStateCache().SetVisibilityResultMode(MTL::VisibilityResultModeDisabled, 0);
			Context->GetCurrentCommandBuffer()->OcclusionQueries.Add(this);
			break;
		}
		case RQT_AbsoluteTime:
		{
			AddRef();

			// Reset the result availability state
			Buffer.SourceBuffer = nullptr;
			Buffer.Offset = 0;
			Buffer.bCompleted = false;
			Result = 0;
			bAvailable = false;
			
			CommandBuffer = Context->GetCurrentCommandBuffer();
			CommandBuffer->TimestampQueries.Add(this);
            
			break;
		}
		default:
		{
			check(0);
			break;
		}
	}
}

void FMetalRHIRenderQuery::SampleOcclusionResult()
{
	checkSlow(Type == RQT_Occlusion);
	Result = Buffer.GetResult();

	// Now we have sampled the occlusion release the reference to this
	Release();
}

bool FMetalRHIRenderQuery::GetResult(uint64& OutNumPixels, bool bWait, uint32 GPUIndex)
{
	if (!bAvailable)
	{
		if (!SyncPoint->IsComplete())
		{
			if (bWait)
			{
				SyncPoint->Wait();
			}
			else
			{
				return false;
			}
		}
	}

	OutNumPixels = Result;
	return true;
}
