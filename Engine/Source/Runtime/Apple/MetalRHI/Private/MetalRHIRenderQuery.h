// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHIRenderQuery.h: Metal RHI Render Query Definitions.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "RHIResources.h"
#include "MetalSubmission.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Buffer Forward Declarations


class FMetalRHICommandContext;
class FMetalQueryBufferPool;
class FMetalQueryResult;
class FMetalDevice;
class FMetalBuffer;
typedef TSharedPtr<FMetalBuffer, ESPMode::ThreadSafe> FMetalBufferPtr;

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Buffer Resource Class -


class FMetalQueryBuffer : public FRefCountedObject
{
public:
	FMetalQueryBuffer(FMetalQueryBufferPool* Pool, FMetalBufferPtr InBuffer);
	virtual ~FMetalQueryBuffer();

	uint64 GetResult(uint32 Offset);

	FMetalQueryBufferPool* Pool = nullptr;
	FMetalBufferPtr Buffer;
	uint32 WriteOffset;
};

typedef TRefCountPtr<FMetalQueryBuffer> FMetalQueryBufferRef;

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Buffer Pool Class -


class FMetalQueryBufferPool
{
public:
	enum
	{
		EQueryBufferAlignment = 8,
		EQueryResultMaxSize   = 8
	};

	// Disallow a default constructor
	FMetalQueryBufferPool() = delete;

	FMetalQueryBufferPool(FMetalDevice& InDevice);
	~FMetalQueryBufferPool();

	void Allocate(FMetalQueryResult& NewQuery);
	
	FMetalQueryBufferRef AcquireQueryBuffer(uint32_t NumOcclusionQueries);
	FMetalQueryBufferRef GetCurrentQueryBuffer();
	void ReleaseCurrentQueryBuffer();
	void ReleaseQueryBuffer(FMetalBufferPtr Buffer);

	uint32_t BufferSize = 0;
	
	FMetalQueryBufferRef CurrentBuffer;
	TArray<FMetalBufferPtr> Buffers;
	FMetalDevice& Device;
};


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Private Query Result Class -


class FMetalQueryResult
{
public:
	FMetalQueryResult() = default;
	~FMetalQueryResult() = default;

	void Reset();
	uint64 GetResult();

	FMetalQueryBufferRef SourceBuffer = nullptr;
	uint32 Offset = 0;
	bool bCompleted = false;
	bool bBatchFence = false;
};


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Render Query Class -

class FMetalRHIRenderQuery : public FRHIRenderQuery
{
	friend class FMetalDynamicRHI;
public:
	FMetalRHIRenderQuery(FMetalDevice& MetalDevice, ERenderQueryType InQueryType);
	virtual ~FMetalRHIRenderQuery();

	void Begin_TopOfPipe();
	void End_TopOfPipe();
	
	/**
	 * Kick off an occlusion test
	 */
	void Begin(FMetalRHICommandContext* Context);

	/**
	 * Finish up an occlusion test
	 */
	void End(FMetalRHICommandContext* Context);

	/**
	 * Samples the occlusion result on the submission thread
	 */
	void SampleOcclusionResult();
	
	/**
	 * Get the query result
	 */
	bool GetResult(uint64& OutNumPixels, bool bWait, uint32 GPUIndex);

private:
	FMetalDevice& Device;
	
	// The type of query
	ERenderQueryType Type;

	// Query buffer allocation details as the buffer is already set on the command-encoder
	FMetalQueryResult Buffer;

	// Query result.
	volatile uint64 Result;

	// Result availability - if not set the first call to acquire it will read the buffer & cache
	std::atomic<bool> bAvailable;

	// Signaled when the result is available. Nullptr if the query has never been used.
	FMetalSyncPointRef SyncPoint;
	
	// Temporary
	FMetalCommandBuffer* CommandBuffer = nullptr;
};
