// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHIStagingBuffer.cpp: Metal RHI Staging Buffer Class.
=============================================================================*/


#include "MetalRHIStagingBuffer.h"
#include "MetalResources.h"
#include "MetalDevice.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Staging Buffer Class


FMetalRHIStagingBuffer::FMetalRHIStagingBuffer(FMetalDevice& InDevice)
	: FRHIStagingBuffer()
	, Device(InDevice)
{
	// void
}

FMetalRHIStagingBuffer::~FMetalRHIStagingBuffer()
{
	if (ShadowBuffer)
	{
		FMetalDynamicRHI::Get().DeferredDelete(ShadowBuffer);
		ShadowBuffer = nullptr;
	}
}

void *FMetalRHIStagingBuffer::Lock(uint32 Offset, uint32 NumBytes)
{
	check(ShadowBuffer);
	check(!bIsLocked);
	bIsLocked = true;
	uint8* BackingPtr = (uint8*)ShadowBuffer->Contents();
	return BackingPtr + Offset;
}

void FMetalRHIStagingBuffer::Unlock()
{
	// does nothing in metal.
	check(bIsLocked);
	bIsLocked = false;
}

uint64 FMetalRHIStagingBuffer::GetGPUSizeBytes() const
{
	return ShadowBuffer ? ShadowBuffer->GetLength() : 0;
}
