// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHIStagingBuffer.h: Metal RHI Staging Buffer Class.
=============================================================================*/

#pragma once

#include "RHIResources.h"

class FMetalDevice;
class FMetalBuffer;
typedef TSharedPtr<FMetalBuffer, ESPMode::ThreadSafe> FMetalBufferPtr;

class FMetalRHIStagingBuffer final : public FRHIStagingBuffer
{
	friend class FMetalRHICommandContext;

public:
	FMetalRHIStagingBuffer(FMetalDevice& Device);
	virtual ~FMetalRHIStagingBuffer() final override;

	/**
	 * Returns the pointer to read the buffer.
	 *
	 * There is no actual locking, the buffer is always shared.  If this is not
	 * fenced correctly, it will not have the expected data.
	 */
	virtual void* Lock(uint32 Offset, uint32 NumBytes) final override;

	virtual void Unlock() final override;
	
	uint64 GetGPUSizeBytes() const override;

private:
	FMetalDevice& Device;
	FMetalBufferPtr ShadowBuffer = nullptr;
};
