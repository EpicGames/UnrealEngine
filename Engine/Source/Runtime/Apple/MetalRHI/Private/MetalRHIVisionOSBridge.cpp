// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalRHIVisionOSBridge.h"

#if PLATFORM_VISIONOS

#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"

DEFINE_LOG_CATEGORY(LogMetalVisionOS);

void MetalRHIVisionOS::PresentImmersive(const MetalRHIVisionOS::PresentImmersiveParams& Params)
{
	FMetalRHICommandContext* RHICommandContext = static_cast<FMetalRHICommandContext*>(Params.RHICommandContext);
	check(RHICommandContext);
	check(RHICommandContext->CustomPresentViewport);

	FMetalViewport* Viewport = ResourceCast(RHICommandContext->CustomPresentViewport);
	check(Viewport);
	
	Viewport->PresentImmersive(Params);
}
#endif // PLATFORM_VISIONOS
