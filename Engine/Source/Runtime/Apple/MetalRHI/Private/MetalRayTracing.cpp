// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MetalRayTracing.cpp: MetalRT Implementation
==============================================================================*/

#include "MetalRayTracing.h"

#if METAL_RHI_RAYTRACING

#include "MetalDynamicRHI.h"
#include "MetalRHIContext.h"
#include "MetalShaderTypes.h"
#include "MetalBindlessDescriptors.h"
#include "MetalResourceCollection.h"
#include "MetalStaticSamplers.h"
#include "Shaders/Types/MetalRayShader.h"
#include "BuiltInRayTracingShaders.h"
#include "RayTracingValidationShaders.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Async/ParallelFor.h"
#include "Apple/ScopeAutoreleasePool.h"

static int32 GMetalRayTracingAllowCompaction = 1;
static FAutoConsoleVariableRef CVarMetalRayTracingAllowCompaction(
	TEXT("r.Metal.RayTracing.AllowCompaction"),
	GMetalRayTracingAllowCompaction,
	TEXT("Whether to automatically perform compaction for static acceleration structures to save GPU memory. (default = 1)\n"),
	ECVF_ReadOnly
);

static int32 GRayTracingDebugForceBuildMode = 0;
static FAutoConsoleVariableRef CVarMetalRayTracingDebugForceFastTrace(
	TEXT("r.Metal.RayTracing.DebugForceBuildMode"),
	GRayTracingDebugForceBuildMode,
	TEXT("Forces specific acceleration structure build mode (not runtime-tweakable).\n")
	TEXT("0: Use build mode requested by high-level code (Default)\n")
	TEXT("1: Force fast build mode\n")
	TEXT("2: Force fast trace mode\n"),
	ECVF_ReadOnly
);

static int32 GMetalRayTracingMaxBatchedCompaction = 64;
static FAutoConsoleVariableRef CVarMetalRayTracingMaxBatchedCompaction(
	TEXT("r.Metal.RayTracing.MaxBatchedCompaction"),
	GMetalRayTracingMaxBatchedCompaction,
	TEXT("Maximum of amount of compaction requests and rebuilds per frame. (default = 64)\n"),
	ECVF_ReadOnly
);

DECLARE_CYCLE_STAT(TEXT("RTPSO Create Pipeline"), STAT_RTPSO_CreatePipeline, STATGROUP_MetalRHI);
DECLARE_CYCLE_STAT(TEXT("Ray Dispatch Time"), STAT_MetalRayDispatchTime, STATGROUP_MetalRHI);
				   
struct FMetalShaderIdentifier
{
	union 
	{
		uint64 Data[4] = {0, 0, 0, ~0ull};
		IRShaderIdentifier ShaderIdentifier;
	};
	
	FMetalRayShader* Shader;

	FMetalShaderIdentifier()
		: FMetalShaderIdentifier(0, 0, 0)
	{}

	FMetalShaderIdentifier(uint64_t IntersectionShaderHandle, uint64_t ShaderHandle, uint64 StaticSamplerTableVA)
	: ShaderIdentifier()
	{
		ShaderIdentifier.intersectionShaderHandle = IntersectionShaderHandle;
		ShaderIdentifier.shaderHandle = ShaderHandle;
		ShaderIdentifier.localRootSignatureSamplersBuffer = StaticSamplerTableVA;
		ShaderIdentifier.pad0 = ~0ull;
	}

	// No shader is executed if a shader binding table record with null identifier is encountered.
	static const FMetalShaderIdentifier Null;

	bool operator == (const FMetalShaderIdentifier& Other) const
	{
		return Data[0] == Other.Data[0]
			&& Data[1] == Other.Data[1]
			&& Data[2] == Other.Data[2]
			&& Data[3] == Other.Data[3]
			&& Shader == Other.Shader;
	}

	bool operator != (const FMetalShaderIdentifier& Other) const
	{
		return !(*this == Other);
	}

	bool IsValid() const
	{
		return *this != FMetalShaderIdentifier();
	}

	void SetData(const void* InData)
	{
		FMemory::Memcpy(Data, InData, sizeof(Data));
	}
};

const FMetalShaderIdentifier FMetalShaderIdentifier::Null(0, 0, 0);

struct FMetalRayShaderLibrary
{
	void Reserve(uint32 NumShaders)
	{
		Shaders.Reserve(NumShaders);
		Identifiers.Reserve(NumShaders);
	}

	int32 Find(FShaderHash Hash) const
	{
		for (int32 Index = 0; Index < Shaders.Num(); ++Index)
		{
			if (Hash == Shaders[Index]->GetHash())
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}

	TArray<TRefCountPtr<FMetalRayShader>> Shaders;
	TArray<const FMetalShaderIdentifier*> Identifiers;
};

inline uint64 GetShaderHash64(FRHIRayTracingShader* ShaderRHI)
{
	return ShaderRHI->GetHash().Hash;
}

inline FString GenerateShaderName(const TCHAR* Prefix, uint64 Hash)
{
	return FString::Printf(TEXT("%s_%016llx"), Prefix, Hash);
}

inline FString GenerateShaderName(FRHIRayTracingShader* ShaderRHI)
{
	const FMetalRayShader* Shader = ResourceCast(ShaderRHI);
	uint64 ShaderHash = GetShaderHash64(ShaderRHI);
	return GenerateShaderName(*(Shader->EntryPoint), ShaderHash);
}


// Cache for ray tracing pipeline collection objects, containing single shaders that can be linked into full pipelines.
class FMetalRayTracingPipelineCache
{
public:

	UE_NONCOPYABLE(FMetalRayTracingPipelineCache)

	FMetalRayTracingPipelineCache()
	{
		// Default empty local root signature
		LLM_SCOPE_BYNAME(TEXT("FMetalRayTracing/PipelineCache"));

		IRVersionedRootSignatureDescriptor LocalRootSignatureDesc;
		LocalRootSignatureDesc.version = IRRootSignatureVersion_1_1;
		LocalRootSignatureDesc.desc_1_1.Flags = IRRootSignatureFlagNone;
		LocalRootSignatureDesc.desc_1_1.pStaticSamplers = nullptr;
		LocalRootSignatureDesc.desc_1_1.NumStaticSamplers = 0;
		LocalRootSignatureDesc.desc_1_1.pParameters = nullptr;
		LocalRootSignatureDesc.desc_1_1.NumParameters = 0;

		IRError *RootSignatureCreationError = nullptr;
		DefaultLocalRootSignature = IRRootSignatureCreateFromDescriptor(&LocalRootSignatureDesc, &RootSignatureCreationError);
		checkf(DefaultLocalRootSignature && !RootSignatureCreationError, TEXT("Error: Failed to create default local root signature descriptor, error code %u"), RootSignatureCreationError ? IRErrorGetCode(RootSignatureCreationError) : IRErrorCodeNoError);

		CurrentShaderIdentifier = 0ull;
	}

	~FMetalRayTracingPipelineCache()
	{
		Reset();
	}

	struct FKey
	{
		uint64 ShaderHash = 0;
		uint32 MaxAttributeSizeInBytes = 0;
		uint32 MaxPayloadSizeInBytes = 0;
		IRRootSignature* GlobalRootSignature = nullptr;
		IRRootSignature* LocalRootSignature = nullptr;

		bool operator == (const FKey& Other) const
		{
			return ShaderHash == Other.ShaderHash
				&& MaxAttributeSizeInBytes == Other.MaxAttributeSizeInBytes
				&& MaxPayloadSizeInBytes == Other.MaxPayloadSizeInBytes
				&& GlobalRootSignature == Other.GlobalRootSignature
				&& LocalRootSignature == Other.LocalRootSignature;
		}

		inline friend uint32 GetTypeHash(const FKey& Key)
		{
			return Key.ShaderHash;
		}
	};

	enum class ECollectionType
	{
		Unknown,
		RayGen,
		Miss,
		HitGroup,
		Callable,
	};

	struct FEntry
	{
		// Move-only type
		FEntry() = default;
		FEntry(FEntry&& Other) = default;

		FEntry(const FEntry&) = delete;
		FEntry& operator = (const FEntry&) = delete;
		FEntry& operator = (FEntry&& Other) = delete;

		const TCHAR* GetPrimaryExportNameChars()
		{
			checkf(ExportNames.Num()!=0, TEXT("This ray tracing shader collection does not export any symbols."));
			return *(ExportNames[0]);
		}

		ECollectionType CollectionType = ECollectionType::Unknown;

		TRefCountPtr<FMetalRayShader> Shader;

		bool bDeserialized = false;

		static constexpr uint32 MaxExports = 4;
		TArray<FString, TFixedAllocator<MaxExports>> ExportNames;

		float CompileTimeMS = 0.0f;
	};

	static const TCHAR* GetCollectionTypeName(ECollectionType Type)
	{
		switch (Type)
		{
			case ECollectionType::Unknown:
				return TEXT("Unknown");
			case ECollectionType::RayGen:
				return TEXT("RayGen");
			case ECollectionType::Miss:
				return TEXT("Miss");
			case ECollectionType::HitGroup:
				return TEXT("HitGroup");
			case ECollectionType::Callable:
				return TEXT("Callable");
			default:
				return TEXT("");
		}
	}

	FEntry* GetOrCompileShader(
		FMetalDevice& Device,
		FMetalRayShader* Shader,
		IRRootSignature* GlobalRootSignature,
		uint32 MaxAttributeSizeInBytes,
		uint32 MaxPayloadSizeInBytes,
		ECollectionType CollectionType,
		FGraphEventArray& CompletionList,
		bool* bOutCacheHit = nullptr)
	{
		FScopeLock Lock(&CriticalSection);

		const uint64 ShaderHash = GetShaderHash64(Shader);

		IRRootSignature* LocalRootSignature = nullptr;
		if (CollectionType == ECollectionType::RayGen)
		{
			// RayGen shaders use a default empty local root signature as all their resources bound via global RS.
			LocalRootSignature = DefaultLocalRootSignature;
		}
		else
		{
			// All other shaders (hit groups, miss, callable) use custom root signatures.
			LocalRootSignature = Shader->GetLocalRootSignature();
		}

		check(GlobalRootSignature || LocalRootSignature);

		FKey CacheKey;
		CacheKey.ShaderHash = ShaderHash;
		CacheKey.MaxAttributeSizeInBytes = MaxAttributeSizeInBytes;
		CacheKey.MaxPayloadSizeInBytes = MaxPayloadSizeInBytes;
		CacheKey.GlobalRootSignature = GlobalRootSignature;
		CacheKey.LocalRootSignature = LocalRootSignature;

		FEntry*& FindResult = Cache.FindOrAdd(CacheKey);

		if (FindResult)
		{
			if (bOutCacheHit) *bOutCacheHit = true;
		}
		else
		{
			if (bOutCacheHit) *bOutCacheHit = false;

			if (FindResult == nullptr)
			{
				FindResult = new FEntry;
			}

			FEntry& Entry = *FindResult;

			Entry.CollectionType = CollectionType;
			Entry.Shader = Shader;

			{
				// Generate primary export name, which is immediately required on the PSO creation thread.
				Entry.ExportNames.Add(GenerateShaderName(GetCollectionTypeName(CollectionType), ShaderHash));
				checkf(Entry.ExportNames.Num() == 1, TEXT("Primary export name must always be first."));
			}
		}

		return FindResult;
	}

	void Reset()
	{
		FScopeLock Lock(&CriticalSection);

		for (auto It : Cache)
		{
			delete It.Value;
		}

		Cache.Reset();

		IRRootSignatureDestroy(DefaultLocalRootSignature);
	}


	uint64_t GetCurrentShaderIdentifier()
	{
		return CurrentShaderIdentifier;
	}

	uint64_t NextShaderIdentifier()
	{
		return ++CurrentShaderIdentifier;
	}

private:
	std::atomic<uint64_t> CurrentShaderIdentifier;

	FCriticalSection CriticalSection;
	TMap<FKey, FEntry*> Cache;
	IRRootSignature *DefaultLocalRootSignature; // Default empty root signature used for default hit shaders.
};

#define METAL_RAYTRACING_SHADER_RECORD_BYTE_ALIGNMENT 32
#define METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT 64

template< typename t_A, typename t_B >
inline t_A RoundUpToNextMultiple(const t_A& a, const t_B& b)
{
	return ((a - 1) / b + 1) * b;
}

class FMetalRayTracingPipelineState;

static void ProcessUniformBuffer(FMetalBufferPtr BackingBuffer, const TArray<TRefCountPtr<FRHIResource>>& CapturedResourceTable, FMetalStateCache& StateCache)
{
	StateCache.CacheOrSkipResourceResidencyUpdate(BackingBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);

	for(const TRefCountPtr<FRHIResource>& RHIResourceRef : CapturedResourceTable)
	{
		FRHIResource* Resource = RHIResourceRef.GetReference();
	
		if(!Resource)
		{
			continue;
		}
		
		switch(Resource->GetType())
		{
			case RRT_Texture:
			{
				FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(static_cast<FRHITexture*>(Resource));
				StateCache.IRMakeTextureResident(EMetalShaderStages::Compute, Surface->Texture.get());
				break;
			}
			case RRT_TextureReference:
			{
				FRHITexture* TextureRHI = ((FRHITextureReference*)Resource)->GetReferencedTexture();
				FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);
				StateCache.IRMakeTextureResident(EMetalShaderStages::Compute, Surface->Texture.get());
				break;
			}
			case RRT_UnorderedAccessView:
			{
				if (FRHIUnorderedAccessView* UAV = static_cast<FRHIUnorderedAccessView*>(Resource))
				{
					StateCache.IRMakeUAVResident(EMetalShaderStages::Compute, static_cast<FMetalUnorderedAccessView*>(UAV));
				}
				break;
			}
			case RRT_ShaderResourceView:
			{
				if (FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Resource))
				{
					StateCache.IRMakeSRVResident(EMetalShaderStages::Compute, static_cast<FMetalShaderResourceView*>(SRV));
				}
				break;
			}
			default:
				break;
		}
	}
}

static void ProcessUniformBuffer(FRHIUniformBuffer* UniformBuffer, FMetalStateCache& StateCache)
{
	FMetalUniformBuffer* UB = ResourceCast(UniformBuffer);
	StateCache.CacheOrSkipResourceResidencyUpdate(UB->BackingBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);
	ProcessUniformBuffer(UB->BackingBuffer, UniformBuffer->GetResourceTable(), StateCache);
}

class FMetalRayTracingShaderBindingTable : public FRHIShaderBindingTable
{
private:
	void WriteData(uint32 WriteOffset, const void* InData, uint32 InDataSize)
	{
#if DO_CHECK && DO_GUARD_SLOW
		Data.RangeCheck(WriteOffset);
		Data.RangeCheck(WriteOffset + InDataSize - 1);
#endif // DO_CHECK && DO_GUARD_SLOW

		void* DataOutOffset = Data.GetData() + WriteOffset;
		if(FMemory::Memcmp(DataOutOffset, InData, InDataSize) != 0)
		{
			MarkBufferDirty();
			FMemory::Memcpy(DataOutOffset, InData, InDataSize);
		}
	}

	void CompareData(uint32 Offset, const void* InData, uint32 InDataSize)
	{
#if DO_CHECK
		const uint8* CurrentData = Data.GetData() + Offset;
		const uint8* InBytes = static_cast<const uint8*>(InData);

		if (FMemory::Memcmp(CurrentData, InBytes, InDataSize) != 0)
		{
			// Find first differing byte
			uint32 FirstDiff = 0;
			for (uint32 i = 0; i < InDataSize; ++i)
			{
				if (CurrentData[i] != InBytes[i])
				{
					FirstDiff = i;
					break;
				}
			}

			const int32 BytesToDump = FMath::Min<int32>(64, InDataSize - FirstDiff);

			UE_LOGF(LogMetal, Error, "CompareData mismatch at buffer offset %u + %u (absolute byte %u), comparing %u bytes. Dumping %d bytes from first diff:",
				Offset, FirstDiff, Offset + FirstDiff, InDataSize, BytesToDump);
			UE_LOGF(LogMetal, Error, "  Current:  %ls", *BytesToHex(CurrentData + FirstDiff, BytesToDump));
			UE_LOGF(LogMetal, Error, "  Expected: %ls", *BytesToHex(InBytes + FirstDiff, BytesToDump));

			ensureMsgf(false, TEXT("SBT CompareData mismatch detected. See above log messages for byte offset and hex dump details."));
		}
#endif // DO_CHECK
	}

	void WriteLocalShaderRecord(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 OffsetWithinRecord, const void* InData, uint32 InDataSize)
	{		checkfSlow(OffsetWithinRecord % 4 == 0, TEXT("SBT record parameters must be written on DWORD-aligned boundary"));
		checkfSlow(InDataSize % 4 == 0, TEXT("SBT record parameters must be DWORD-aligned"));
		checkfSlow(OffsetWithinRecord + InDataSize <= LocalRecordSizeUnaligned, TEXT("SBT record write request is out of bounds"));

		const uint32 WriteOffset = ShaderTableOffset + LocalRecordStride * RecordIndex + OffsetWithinRecord;

		WriteData(WriteOffset, InData, InDataSize);
	}

	void CompareLocalShaderRecord(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 OffsetWithinRecord, const void* InData, uint32 InDataSize)
	{
		const uint32 Offset = ShaderTableOffset + LocalRecordStride * RecordIndex + OffsetWithinRecord;
		CompareData(Offset, InData, InDataSize);
	}
	
public:
	FMetalRayTracingShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer, FMetalDevice& InDevice)
		: FRHIShaderBindingTable(Initializer),
		Device(InDevice)
	{
		checkf(Initializer.LocalBindingDataSize <= 4096, TEXT("The maximum size of a local root signature is 4KB.")); // as per section 4.22.1 of DXR spec v1.0
		check(Initializer.ShaderBindingMode != ERayTracingShaderBindingMode::Disabled);

		Lifetime = Initializer.Lifetime;
		HitGroupIndexingMode = Initializer.HitGroupIndexingMode;
		ShaderBindingMode = Initializer.ShaderBindingMode;
		NumShaderSlotsPerGeometrySegment = Initializer.NumShaderSlotsPerGeometrySegment;
		
		const uint32 NumHitGroupSlots = Initializer.HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow ?
										Initializer.NumGeometrySegments * Initializer.NumShaderSlotsPerGeometrySegment : 1;

		NumMissRecords = Initializer.NumMissShaderSlots;
		NumHitRecords = NumHitGroupSlots;
		NumCallableRecords = Initializer.NumCallableShaderSlots;
		
		LocalRecordSizeUnaligned = ShaderIdentifierSize + Initializer.LocalBindingDataSize;
		LocalRecordStride = RoundUpToNextMultiple(LocalRecordSizeUnaligned, METAL_RAYTRACING_SHADER_RECORD_BYTE_ALIGNMENT);

		uint32 TotalDataSize = 0;
		
		HitGroupShaderTableOffset = TotalDataSize;
		TotalDataSize += NumHitGroupSlots * LocalRecordStride;
		TotalDataSize = RoundUpToNextMultiple(TotalDataSize, METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

		CallableShaderTableOffset = TotalDataSize;
		TotalDataSize += Initializer.NumCallableShaderSlots * LocalRecordStride;
		TotalDataSize = RoundUpToNextMultiple(TotalDataSize, METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

		MissShaderTableOffset = TotalDataSize;
		TotalDataSize += Initializer.NumMissShaderSlots * LocalRecordStride;
		TotalDataSize = RoundUpToNextMultiple(TotalDataSize, METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT);

		Data.SetNumZeroed(TotalDataSize);
		
#if DO_CHECK
		bWasDefaultMissShaderSet = false;
#endif

		SetDefaultHitGroupIdentifier(FMetalShaderIdentifier::Null);
		SetDefaultMissShaderIdentifier(FMetalShaderIdentifier::Null);
		SetDefaultCallableShaderIdentifier(FMetalShaderIdentifier::Null);

		// Keep CPU-side data after upload
		Data.SetAllowCPUAccess(true);

		if (Initializer.NumGeometrySegments > 0)
		{
			const uint32 RecordBufferSize = FMath::Max(1u, Initializer.NumGeometrySegments) * sizeof(FMetalHitGroupSystemParameters);
			InlineGeometryParameterData.SetNumUninitialized(RecordBufferSize);
		}
		
		Lifetime = Initializer.Lifetime;
		HitGroupIndexingMode = Initializer.HitGroupIndexingMode;
		ShaderBindingMode = Initializer.ShaderBindingMode;
		NumShaderSlotsPerGeometrySegment = Initializer.NumShaderSlotsPerGeometrySegment;
	}
	
	~FMetalRayTracingShaderBindingTable()
	{
		if(Buffer)
		{
			FMetalDynamicRHI::Get().DeferredDelete(Buffer);
		}
		
		bWasDeleted = true;
	}
	
	template <typename T>
	void SetLocalShaderParameters(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	void SetLocalShaderParameters(uint32 ShaderTableOffset, uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const void* InData, uint32 InDataSize)
	{
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, InData, InDataSize);
	}

	template <typename T>
	void SetMissShaderParameters(uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		const uint32 ShaderTableOffset = MissShaderTableOffset;
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	template <typename T>
	void SetCallableShaderParameters(uint32 RecordIndex, uint32 InOffsetWithinRootSignature, const T& Parameters)
	{
		const uint32 ShaderTableOffset = CallableShaderTableOffset;
		WriteLocalShaderRecord(ShaderTableOffset, RecordIndex, ShaderIdentifierSize + InOffsetWithinRootSignature, &Parameters, sizeof(Parameters));
	}

	void CopyLocalShaderParameters(uint32 InShaderTableOffset, uint32 InDestRecordIndex, uint32 InSourceRecordIndex, uint32 InOffsetWithinRootSignature)
	{
		const uint32 BaseOffset = InShaderTableOffset + ShaderIdentifierSize + InOffsetWithinRootSignature;
		const uint32 DestOffset = BaseOffset + LocalRecordStride * InDestRecordIndex;
		const uint32 SourceOffset = BaseOffset + LocalRecordStride * InSourceRecordIndex;
		const uint32 CopySize = LocalRecordStride - ShaderIdentifierSize - InOffsetWithinRootSignature;
		checkSlow(CopySize <= LocalRecordStride);

		FMemory::Memcpy(
			Data.GetData() + DestOffset,
			Data.GetData() + SourceOffset,
			CopySize);
	}

	void CopyHitGroupParameters(uint32 InDestRecordIndex, uint32 InSourceRecordIndex, uint32 InOffsetWithinRootSignature)
	{
		const uint32 ShaderTableOffset = HitGroupShaderTableOffset;
		CopyLocalShaderParameters(ShaderTableOffset, InDestRecordIndex, InSourceRecordIndex, InOffsetWithinRootSignature);
	}	

	void SetMissIdentifier(uint32 RecordIndex, const FMetalShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = MissShaderTableOffset + RecordIndex * LocalRecordStride;
#if DO_CHECK
		if (RecordIndex == 0)
		{
			bWasDefaultMissShaderSet = true;
		}
#endif
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetCallableIdentifier(uint32 RecordIndex, const FMetalShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = CallableShaderTableOffset + RecordIndex * LocalRecordStride;
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetDefaultHitGroupIdentifier(const FMetalShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = HitGroupShaderTableOffset;
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}

	void SetHitGroupSystemParameters(uint32 RecordIndex, const FMetalHitGroupSystemParameters& SystemParameters)
	{
		const uint32 ShaderTableOffset = HitGroupShaderTableOffset + RecordIndex * LocalRecordStride;
		WriteData(ShaderTableOffset + sizeof(IRShaderIdentifier), &SystemParameters, sizeof(SystemParameters));
	}

	void SetHitGroupIdentifier(uint32 RecordIndex, const FMetalShaderIdentifier& ShaderIdentifier)
	{
		const uint32 WriteOffset = HitGroupShaderTableOffset + RecordIndex * LocalRecordStride;
		WriteData(WriteOffset, ShaderIdentifier.Data, ShaderIdentifierSize);
	}	

	void SetDefaultMissShaderIdentifier(const FMetalShaderIdentifier& ShaderIdentifier)
	{
		// Set all slots to the same default
		for (uint32 Index = 0; Index < NumMissRecords; ++Index)
		{
			SetMissIdentifier(Index, ShaderIdentifier);
		}
		
#if DO_CHECK
		bWasDefaultMissShaderSet = false;
#endif
	}

	void SetDefaultCallableShaderIdentifier(const FMetalShaderIdentifier& ShaderIdentifier)
	{
		for (uint32 Index = 0; Index < NumCallableRecords; ++Index)
		{
			SetCallableIdentifier(Index, ShaderIdentifier);
		}
	}
	
	void Commit(FMetalRHICommandContext* Context, FRHIBuffer* InlineBindingDataBuffer)
	{
		check(Context);
		
		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline))
		{			
			if (InlineBindingDataBuffer)
			{
				TRHICommandList_RecursiveHazardous<FMetalRHICommandContext> RHICmdList(Context);

				const uint32 ParameterBufferSize = InlineGeometryParameterData.Num();
				FMetalRHIBuffer* InlineBuffer = (FMetalRHIBuffer*)InlineBindingDataBuffer;
				
				FMetalBufferPtr TempBuffer = Device.GetResourceHeap().CreateBuffer(ParameterBufferSize, BufferBackedLinearTextureOffsetAlignment, BUF_Dynamic, MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared, true);

				void* Result = TempBuffer->Contents();				
				FMemory::Memcpy(Result, InlineGeometryParameterData.GetData(), ParameterBufferSize);
				
				Context->CopyFromBufferToBuffer(TempBuffer, 0, InlineBuffer->GetCurrentBuffer(), 0, ParameterBufferSize);
				
				FMetalDynamicRHI::Get().DeferredDelete(TempBuffer);
			}
		}
		
		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
		{
			CopyToGPU(*Context);
		}
	}
	
	void SetUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 Offset, uint32 RecordIndex, const uint32 NumUniformBuffers,
									FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize);

	void SetHitGroupUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 RecordIndex, const uint32 NumUniformBuffers,
									FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize);

	void SetMissUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 RecordIndex, const uint32 NumUniformBuffers,
									FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize);

	void SetCallableUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 RecordIndex, const uint32 NumUniformBuffers,
									FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize);

	void ResetResourceBindings()
	{
		for (uint32 i = 0; i < MaxBindingWorkers; ++i)
		{
			ResourceBindings[i].Reset();
		}
	}
	
	const IRShaderIdentifier &ShaderIdentifierAtOffset(uint32 Offset) const
	{
		return *reinterpret_cast<const IRShaderIdentifier *>(Data.GetData() + Offset);
	}
	
	void SetInlineGeometryParameters(uint32 SegmentIndex, const void* InData, uint32 InDataSize)
	{
		const uint32 WriteOffset = InDataSize * SegmentIndex;
		FMemory::Memcpy(&InlineGeometryParameterData[WriteOffset], InData, InDataSize);
	}
	
	virtual FRHISizeAndStride GetInlineBindingDataSizeAndStride() const override
	{
		return FRHISizeAndStride { (uint64)InlineGeometryParameterData.Num(), sizeof(FMetalHitGroupSystemParameters) };
	}
	
	uint32 GetInlineRecordIndex(uint32 RecordIndex) const
	{	
		// Only care about shader slot 0 for inline geometry parameters -> remap the record index
		return (RecordIndex % NumShaderSlotsPerGeometrySegment == 0) ? RecordIndex / NumShaderSlotsPerGeometrySegment : INDEX_NONE;
	}
	
	uint32 GetNumShaderSlotsPerGeometrySegment() { return NumShaderSlotsPerGeometrySegment; }
	ERayTracingShaderBindingTableLifetime GetLifetime() { return Lifetime; }
	
	void CopyToGPU(FMetalRHICommandContext& Context);
	
	void MakeResident(FMetalStateCache& StateCache)
	{
		for (uint32_t WorkerIdx = 0; WorkerIdx < MaxBindingWorkers; WorkerIdx++)
		{	
			for(MTL::Heap* Heap : BindingTableHeaps[WorkerIdx])
			{
				StateCache.CacheHeapResidency(Heap, EMetalShaderStages::Compute);
			}
		}
		StateCache.CacheOrSkipResourceResidencyUpdate(Buffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);
		
		for (const FBoundUniformBuffer& BoundUB : BoundUniformBuffers)
		{
			ProcessUniformBuffer(BoundUB.BackingBuffer, BoundUB.CapturedResourceTable, StateCache);
		}
	}
	
	uint64 GetShaderTableAddress() const
	{
		return Buffer->GetGPUAddress();
	}

	IRDispatchRaysDescriptor GetDispatchRaysDesc(const FMetalShaderIdentifier& RayGenShaderIdentifier) const
	{
		const uint64 ShaderTableAddress = GetShaderTableAddress();

		IRDispatchRaysDescriptor Desc = {};
		
		FMetalBufferPtr RayGenRecords = Device.GetUniformAllocator()->Allocate(ShaderIdentifierSize);
		FMemory::Memcpy(RayGenRecords->Contents(), &RayGenShaderIdentifier, ShaderIdentifierSize);

		Desc.RayGenerationShaderRecord.StartAddress = RayGenRecords->GetGPUAddress(); 
		Desc.RayGenerationShaderRecord.SizeInBytes = METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT;

		Desc.MissShaderTable.StartAddress = ShaderTableAddress + MissShaderTableOffset;
		Desc.MissShaderTable.StrideInBytes = LocalRecordStride;
		Desc.MissShaderTable.SizeInBytes = LocalRecordStride * NumMissRecords;

		if (NumCallableRecords)
		{
		   Desc.CallableShaderTable.StartAddress = ShaderTableAddress + CallableShaderTableOffset;
		   Desc.CallableShaderTable.StrideInBytes = LocalRecordStride;
		   Desc.CallableShaderTable.SizeInBytes = NumCallableRecords * LocalRecordStride;
		}

		if (HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow)
		{
		   Desc.HitGroupTable.StartAddress = ShaderTableAddress + HitGroupShaderTableOffset;
		   Desc.HitGroupTable.StrideInBytes = LocalRecordStride;
		   Desc.HitGroupTable.SizeInBytes = NumHitRecords * LocalRecordStride;
		}
		else
		{
		   Desc.HitGroupTable.StartAddress = ShaderTableAddress + HitGroupShaderTableOffset;
		   Desc.HitGroupTable.StrideInBytes = 0; // Zero stride effectively disables SBT indexing
		   Desc.HitGroupTable.SizeInBytes = METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT; // Minimal table with only one record
		}

		return Desc;
	}

	// Ray tracing shader bindings can be processed in parallel.
	// Each concurrent worker gets its own dedicated descriptor cache instance to avoid contention or locking.
	// Scaling beyond 5 total threads does not yield any speedup in practice.
	static constexpr uint32 MaxBindingWorkers = 5; // RHI thread + 4 parallel workers.
	
	ERayTracingShaderBindingMode GetShaderBindingMode() const { return ShaderBindingMode; }
	ERayTracingHitGroupIndexingMode GetHitGroupIndexingMode() const { return HitGroupIndexingMode; }
	
	void ResetBindingTableHeaps()
	{
		for (uint32 WorkerIdx = 0; WorkerIdx < MaxBindingWorkers; WorkerIdx++)
		{
			BindingTableHeaps[WorkerIdx].Reset();	
		}
	}
	
	void AddBindingTableHeaps(uint32 WorkerIndex, TSet<MTL::Heap*>& InHeaps)
	{
		BindingTableHeaps[WorkerIndex].Append(InHeaps);
	}
	
	void MarkBufferDirty() {bBufferDirty = true;}
	
public:
	uint32 NumHitRecords = 0;
	uint32 NumCallableRecords = 0;
	uint32 NumMissRecords = 0;
	
private:
	ERayTracingShaderBindingTableLifetime Lifetime = ERayTracingShaderBindingTableLifetime::Transient;
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Allow;
	ERayTracingShaderBindingMode ShaderBindingMode = ERayTracingShaderBindingMode::RTPSO;
	
	static constexpr uint32 ShaderIdentifierSize = sizeof(IRShaderIdentifier);

	uint32 MissShaderTableOffset = 0;
	uint32 HitGroupShaderTableOffset = 0;
	uint32 CallableShaderTableOffset = 0;
	
	TArray<uint8> InlineGeometryParameterData;
	uint32 NumShaderSlotsPerGeometrySegment = 0;
	FMetalDevice& Device;

	uint32 LocalRecordSizeUnaligned = 0; // size of the shader identifier + local root parameters, not aligned to SHADER_RECORD_BYTE_ALIGNMENT (used for out-of-bounds access checks)
	uint32 LocalRecordStride = 0; // size of shader identifier + local root parameters, aligned to SHADER_RECORD_BYTE_ALIGNMENT (same for hit groups and callable shaders)
	TResourceArray<uint8, METAL_RAYTRACING_SHADER_TABLE_BYTE_ALIGNMENT> Data;
	
	struct FBoundUniformBuffer
	{
		FRHIUniformBuffer* UniformBuffer;
		FMetalBufferPtr BackingBuffer;
		TArray<TRefCountPtr<FRHIResource>> CapturedResourceTable;
	};
	TArray<FBoundUniformBuffer> BoundUniformBuffers;

	TSet<MTL::Heap*> BindingTableHeaps[MaxBindingWorkers];
	
	FMetalBufferPtr Buffer;
	bool bBufferDirty = true;
	bool bWasDeleted = false;
	
#if DO_CHECK
	bool bWasDefaultMissShaderSet = false;
#endif
	
	struct FResourceBinder
	{
		static constexpr uint32 MaxUniformBuffers = 8;
		uint64 LooseParametersVA = 0;
		FRHIUniformBuffer* UniformBuffers[MaxUniformBuffers];
		FMetalBufferPtr BackingBuffers[MaxUniformBuffers];
		TArray<TRefCountPtr<FRHIResource>> CapturedResourceTables[MaxUniformBuffers];
		uint32 NumUniformBuffers = 0;
		uint32 BindingOffset;
	};

	FCriticalSection ResourceCS;
	TArray<FResourceBinder> ResourceBindings[MaxBindingWorkers];
};

class FMetalRayTracingPipelineState : public FRHIRayTracingPipelineState
{
public:
	UE_NONCOPYABLE(FMetalRayTracingPipelineState)

	FMetalRayTracingPipelineState(FMetalDevice& Device, const FRayTracingPipelineStateInitializer& Initializer) 
			: FRHIRayTracingPipelineState(Initializer)
	{
		SCOPE_CYCLE_COUNTER(STAT_RTPSO_CreatePipeline);
		
		checkf(Initializer.GetRayGenTable().Num() > 0 || Initializer.bPartial, TEXT("Ray tracing pipelines must have at leat one ray generation shader."));
		
		uint64 TotalCreationTime = 0;
		uint64 CompileTime = 0;
		uint64 LinkTime = 0;
		uint32 NumCacheHits = 0;
		
		TotalCreationTime -= FPlatformTime::Cycles64();

		TArrayView<FRHIRayTracingShader*> InitializerHitGroups = Initializer.GetHitGroupTable();
		TArrayView<FRHIRayTracingShader*> InitializerMissShaders = Initializer.GetMissTable();
		TArrayView<FRHIRayTracingShader*> InitializerRayGenShaders = Initializer.GetRayGenTable();
		TArrayView<FRHIRayTracingShader*> InitializerCallableShaders = Initializer.GetCallableTable();
		
		const uint32 MaxTotalShaders = InitializerRayGenShaders.Num() + InitializerMissShaders.Num() + InitializerHitGroups.Num() + InitializerCallableShaders.Num();
		checkf(MaxTotalShaders >= 1, TEXT("Ray tracing pipelines are expected to contain at least one shader"));
		
		FMetalRayTracingPipelineCache* PipelineCache = Device.GetRayTracingPipelineCache();

		check(!GRHISupportsRayTracingPSOAdditions);
		
		TArray<FMetalRayTracingPipelineCache::FEntry*> UniqueShaderCollections;
		UniqueShaderCollections.Reserve(MaxTotalShaders);
		
		FGraphEventArray CompileCompletionList;
		CompileCompletionList.Reserve(MaxTotalShaders);
		
		uint32 ShaderIdentifierIndex = 0;
		const uint64 StaticSamplersTableVA = Device.GetStaticSamplers()->GetGPUAddress();
		
		auto AddShaderCollection = [&Device, PipelineCache,
									 &UniqueShaderHashes = this->PipelineShaderHashes, &UniqueShaderCollections, &Initializer, &NumCacheHits, &CompileTime,
									 &CompileCompletionList, &ShaderIdentifierIndex, StaticSamplersTableVA]
		(FMetalRayShader* Shader, FMetalRayTracingPipelineCache::ECollectionType CollectionType) -> const FMetalShaderIdentifier*
		{
			const uint64 ShaderHash = GetShaderHash64(Shader);
			bool bCacheHit = false;
			
			CompileTime -= FPlatformTime::Cycles64();
			
			FMetalRayTracingPipelineCache::FEntry* ShaderCacheEntry = PipelineCache->GetOrCompileShader(Device, Shader, Shader->GetGlobalRootSignature(),
																										Initializer.MaxAttributeSizeInBytes,
																										Initializer.MaxPayloadSizeInBytes,
																										CollectionType, CompileCompletionList,
																										&bCacheHit);
			
			if (FMetalShaderIdentifier** Info = UniqueShaderHashes.Find(ShaderHash))
			{
				return *Info;
			}
			
			FMetalShaderIdentifier* Identifier = UniqueShaderHashes.Add(ShaderHash, new FMetalShaderIdentifier);
			
			if (Shader->HasVisibleFunctionEntryPoint())
			{
				Identifier->ShaderIdentifier.shaderHandle = ++ShaderIdentifierIndex;
			}
			if (Shader->HasIntersectionFunctionEntryPoint())
			{
				Identifier->ShaderIdentifier.intersectionShaderHandle = ++ShaderIdentifierIndex; 
			}
			
			Identifier->ShaderIdentifier.localRootSignatureSamplersBuffer = StaticSamplersTableVA;
			
			Identifier->Shader = Shader;
			
			CompileTime += FPlatformTime::Cycles64();
			
			UniqueShaderCollections.Add(ShaderCacheEntry);	
			if (bCacheHit)
			{
				NumCacheHits++;
			}
			
			return Identifier;
		};

		MaxLocalRootSignatureSize = 0;
		
		auto CollectShaders = [&](TArrayView<FRHIRayTracingShader*>& InitializerShaders, TArray<const FMetalShaderIdentifier*>& Entries, FMetalRayShaderLibrary& Library,
								  TMap<FShaderHash, int32>* ShaderIndexByHash, EShaderFrequency Frequency, FMetalRayTracingPipelineCache::ECollectionType Type)
		{
			Entries.Reserve(InitializerShaders.Num());
			Library.Reserve(InitializerShaders.Num());
			
			for (FRHIRayTracingShader* ShaderRHI : InitializerShaders)
			{
				FMetalRayShader* Shader = ResourceCast(ShaderRHI);
				
				checkf(Shader, TEXT("A valid ray tracing shader must be provided for all elements in the FRayTracingPipelineStateInitializer shader table."));
				check(Shader->GetShaderFrequency() == Frequency);

				if(Frequency != SF_RayGen)
				{
					MaxLocalRootSignatureSize = FMath::Max(MaxLocalRootSignatureSize, Shader->GetLocalRootSignatureSize());
				}
				else
				{
					if(ShaderIndexByHash)
					{
						ShaderIndexByHash->Add(Shader->GetHash(), Library.Shaders.Num());
					}
				}
				
				const FMetalShaderIdentifier* ShaderCacheEntry = AddShaderCollection(Shader, Type);
				
				Entries.Add(ShaderCacheEntry);
				Library.Shaders.Add(Shader);
			}
		};
		
		TArray<const FMetalShaderIdentifier*> RayGenShaderEntries;
		TMap<FShaderHash, int32> RayGenShaderIndexByHash;
		CollectShaders(InitializerRayGenShaders, RayGenShaderEntries, RayGenShaders, &RayGenShaderIndexByHash, SF_RayGen, FMetalRayTracingPipelineCache::ECollectionType::RayGen);
		
		TArray<const FMetalShaderIdentifier*> MissShaderEntries;
		CollectShaders(InitializerMissShaders, MissShaderEntries, MissShaders, nullptr, SF_RayMiss, FMetalRayTracingPipelineCache::ECollectionType::Miss);
		
		TArray<const FMetalShaderIdentifier*> HitGroupEntries;
		CollectShaders(InitializerHitGroups, HitGroupEntries, HitGroupShaders, nullptr, SF_RayHitGroup, FMetalRayTracingPipelineCache::ECollectionType::HitGroup);
		
		TArray<const FMetalShaderIdentifier*> CallableShaderEntries;
		CollectShaders(InitializerCallableShaders, CallableShaderEntries, CallableShaders, nullptr, SF_RayCallable, FMetalRayTracingPipelineCache::ECollectionType::Callable);

		// Wait for all compilation tasks to be complete and then gather the compiled collection descriptors
		CompileTime -= FPlatformTime::Cycles64();
		
		FTaskGraphInterface::Get().WaitUntilTasksComplete(CompileCompletionList);
		
		CompileTime += FPlatformTime::Cycles64();
		
		if (Initializer.bPartial)
		{
			// Partial pipelines don't have a linking phase, so exit immediately after compilation tasks are complete.
			return;
		}

		MTL::Device* MTLDevice = Device.GetDevice();
		
		MTL::ComputePipelineDescriptor* ComputePSODescriptor = MTL::ComputePipelineDescriptor::alloc()->init();
		
		TArray<MTL::Function*> NonBinaryFunctions;
		TArray<MTL::Function*> BinaryFunctions;
		
		auto AddFunction = [&NonBinaryFunctions, &BinaryFunctions](MTLFunctionPtr Function)
		{
			if (Function->options() & MTLFunctionOptionCompileToBinary)
			{
				if(!BinaryFunctions.Contains(Function.get()))
				{
					BinaryFunctions.Add(Function.get());
				}
			}
			else
			{
				if(!NonBinaryFunctions.Contains(Function.get()))
				{
					NonBinaryFunctions.Add(Function.get());
				}
			}
		};

		auto UpdateRayTracingLibrary = [&AddFunction, PipelineCache](FMetalRayShaderLibrary &Library, TArray<const FMetalShaderIdentifier*> &Entries, EShaderFrequency ExpectedFrequency)
		{
			Library.Identifiers.SetNumUninitialized(Entries.Num());
			for (int32 Index = 0; Index < Entries.Num(); ++Index)
			{
				FMetalRayShader &Shader = *Entries[Index]->Shader;
				check(Shader.GetShaderFrequency() == ExpectedFrequency);
				
				const FMetalShaderIdentifier *ShaderIdentifier = Entries[Index];
				
				if (Shader.HasVisibleFunctionEntryPoint())
				{
					// RayGen, ClosestHit, Miss, Callable
					AddFunction(Shader.GetEntryPointFunction());
				}
				if (Shader.HasIntersectionFunctionEntryPoint())
				{
					// AnyHit/Intersection
					AddFunction(Shader.GetAnyHitAndIntersectionFunction());
				}
				
				Library.Identifiers[Index] = ShaderIdentifier;
			}
		};

		// Retrieve intersection function wrappers and raygen
		{
			IRCompiler* CompilerInstance = IRCompilerCreate();
			
			IRRayTracingPipelineConfiguration* PipelineConfig = IRRayTracingPipelineConfigurationCreate();
			IRRayTracingPipelineConfigurationSetPipelineFlags(PipelineConfig, IRRaytracingPipelineFlagNone);
			IRRayTracingPipelineConfigurationSetIntrinsicMasks(PipelineConfig, IRIntrinsicMaskClosestHitAll, IRIntrinsicMaskMissShaderAll, IRIntrinsicMaskAnyHitShaderAll, IRIntrinsicMaskCallableShaderAll);
			IRRayTracingPipelineConfigurationSetMaxRecursiveDepth(PipelineConfig, RAY_TRACING_MAX_ALLOWED_RECURSION_DEPTH);
			IRRayTracingPipelineConfigurationSetRayGenerationCompilationMode(PipelineConfig, IRRayGenerationCompilationVisibleFunction);
			IRRayTracingPipelineConfigurationSetIntersectionFunctionCompilationMode(PipelineConfig, IRIntersectionFunctionCompilationVisibleFunction);
			IRRayTracingPipelineConfigurationSetMaxAttributeSizeInBytes(PipelineConfig, RAY_TRACING_MAX_ALLOWED_ATTRIBUTE_SIZE);
			
			IRCompilerSetRayTracingPipelineConfiguration(CompilerInstance, PipelineConfig);
			
			auto CreateIndirectIntersectionFunction = [&](IRHitGroupType HitGroupType) -> MTLFunctionPtr
			{
				IRCompilerSetHitgroupType(CompilerInstance, HitGroupType);
				IRMetalLibBinary *Binary = IRMetalLibBinaryCreate();
				const bool bIntersectionWrapperCreated = IRMetalLibSynthesizeIndirectIntersectionFunction(CompilerInstance, Binary);
				check(bIntersectionWrapperCreated);

				NS::Error* Error = nullptr;
				MTLLibraryPtr Library = NS::TransferPtr(MTLDevice->newLibrary(IRMetalLibGetBytecodeData(Binary), &Error));
				
				NS::Array* FunctionNames = Library->functionNames();
				check(FunctionNames->count() == 1);
				
				MTL::FunctionDescriptor* Descriptor = MTL::FunctionDescriptor::alloc()->init();
				Descriptor->setName((NS::String*)FunctionNames->object(0));
				
				MTLFunctionPtr IntersectionWrapperFunction = NS::TransferPtr(Library->newFunction(Descriptor, &Error));
				Descriptor->release();
				
				checkf(IntersectionWrapperFunction && !Error, TEXT("%s"), *NSStringToFString(Error->description()));
				AddFunction(IntersectionWrapperFunction);

				IRMetalLibBinaryDestroy(Binary);
				
				return IntersectionWrapperFunction;
			};
			
			// Generate intersection functions for Triangles (0), and Procedural (1) which is used when generating instance descriptors
			TriangleIntersectionFunction = CreateIndirectIntersectionFunction(IRHitGroupTypeTriangles);
			ProceduralIntersectionFunction = CreateIndirectIntersectionFunction(IRHitGroupTypeProceduralPrimitive);
			
			// Retrieve raygen indirection function
			{
				IRMetalLibBinary* pMetalLibBin = IRMetalLibBinaryCreate();
				bool Result = IRMetalLibSynthesizeIndirectRayDispatchFunction(CompilerInstance, pMetalLibBin);
				check(Result);
				
				NS::Error* Error = nullptr;
				
				MTLLibraryPtr Library = NS::TransferPtr(MTLDevice->newLibrary(IRMetalLibGetBytecodeData(pMetalLibBin), &Error));
				check(Library);
				
				MTLFunctionPtr RaygenIndirection = NS::TransferPtr(Library->newFunction(NS::String::string(kIRRayDispatchIndirectionKernelName, NS::UTF8StringEncoding)));
				check(RaygenIndirection);
				
				ComputePSODescriptor->setComputeFunction(RaygenIndirection.get());
				
				IRMetalLibBinaryDestroy(pMetalLibBin);
			}
			
			IRRayTracingPipelineConfigurationDestroy(PipelineConfig);
			IRCompilerDestroy(CompilerInstance);
		}
		
		UpdateRayTracingLibrary(RayGenShaders, RayGenShaderEntries, SF_RayGen);
		UpdateRayTracingLibrary(HitGroupShaders, HitGroupEntries, SF_RayHitGroup);
		UpdateRayTracingLibrary(MissShaders, MissShaderEntries, SF_RayMiss);
		UpdateRayTracingLibrary(CallableShaders, CallableShaderEntries, SF_RayCallable);

		AllShaders.Reset();
		AllShaders.SetNumUninitialized(ShaderIdentifierIndex + 1);
		
		AllShaderFunctions.Reset();
		AllShaderFunctions.SetNumZeroed(ShaderIdentifierIndex + 1);

		auto MapShaderIdentifiersToShaders = [this](TArray<const FMetalShaderIdentifier*> &Entries)
		{
			for (int32 Index = 0; Index < Entries.Num(); ++Index)
			{
				FMetalRayShader* Shader = Entries[Index]->Shader;
				const IRShaderIdentifier ShaderIdentifier = Entries[Index]->ShaderIdentifier;
				if (ShaderIdentifier.shaderHandle)
				{
					AllShaders[ShaderIdentifier.shaderHandle] = Shader;
					AllShaderFunctions[ShaderIdentifier.shaderHandle] = Shader->GetEntryPointFunction();
				}
				if (ShaderIdentifier.intersectionShaderHandle)
				{
					AllShaders[ShaderIdentifier.intersectionShaderHandle] = Shader;
					AllShaderFunctions[ShaderIdentifier.intersectionShaderHandle] = Shader->GetAnyHitAndIntersectionFunction();
				}
			}
		};

		MapShaderIdentifiersToShaders(RayGenShaderEntries);
		MapShaderIdentifiersToShaders(HitGroupEntries);
		MapShaderIdentifiersToShaders(MissShaderEntries);
		MapShaderIdentifiersToShaders(CallableShaderEntries);

		MTL::LinkedFunctions *LinkedFunctions = MTL::LinkedFunctions::alloc()->init();
		auto SortFunctionsByName = [](TArray<MTL::Function*>& InArray) -> NS::Array*
		{
			InArray.Sort([](const MTL::Function& A, const MTL::Function& B)
			{
				return NSStringToFString(A.name()) < NSStringToFString(B.name()); 
			});
			
			NS::Array* OutArray = NS::Array::alloc()->init((const NS::Object* const*)InArray.GetData(), InArray.Num())->autorelease();
			return OutArray;
		};
		
		LinkedFunctions->setFunctions(SortFunctionsByName(NonBinaryFunctions));
		LinkedFunctions->setBinaryFunctions(SortFunctionsByName(BinaryFunctions));
		
		ComputePSODescriptor->setSupportAddingBinaryFunctions(true);

		ComputePSODescriptor->setLinkedFunctions(LinkedFunctions);
		ComputePSODescriptor->setMaxCallStackDepth(RAY_TRACING_MAX_ALLOWED_RECURSION_DEPTH + 1);

		NS::Error *Error = nullptr;
		
		PipelineStateObject = FMetalShaderPipelinePtr(new FMetalShaderPipeline(Device));
		PipelineStateObject->ComputePipelineState = NS::TransferPtr(MTLDevice->newComputePipelineState(ComputePSODescriptor, 0, nullptr, &Error));
		
		UE_CLOGF((PipelineStateObject->ComputePipelineState.get() == nullptr), LogMetal, Error, "Invalid Compute Pipeline State, Descriptor: %ls", *NSStringToFString(ComputePSODescriptor->description()));
		
		LinkedFunctions->release();

		if (!PipelineStateObject || Error)
		{
			UE_LOGF(LogMetal, Error, "Cannot create pipeline state with error: %ls", *NSStringToFString(Error->description()));
		}
		else
		{
			UE_LOGF(LogMetal, Log, "Created a PSO with %llu non-binary and %llu binary functions", (uint64)NonBinaryFunctions.Num(), (uint64)BinaryFunctions.Num());
		}
		check(PipelineStateObject);

		ComputePSODescriptor->release();
		BinaryFunctions.Empty();
		NonBinaryFunctions.Empty();

		{
			MTL::VisibleFunctionTableDescriptor* Desc = MTL::VisibleFunctionTableDescriptor::alloc()->init();
			Desc->setFunctionCount(AllShaders.Num()+1);
			VisibleFunctionTable = PipelineStateObject->ComputePipelineState->newVisibleFunctionTable(Desc);
			
			check(AllShaders.Num());
			check(VisibleFunctionTable);
			
			// Entry 0 is deliberately empty
			for (uint64 Index = 1; Index < AllShaderFunctions.Num(); ++Index)
			{
				MTLFunctionPtr Function = AllShaderFunctions[Index];
				if (Function)
				{
					MTL::FunctionHandle* Handle = PipelineStateObject->ComputePipelineState->functionHandle(Function.get());
					checkf(Handle, TEXT("Failed to find function '%s' in the PSO"), ANSI_TO_TCHAR(Function->name()->utf8String()));
					VisibleFunctionTable->setFunction(Handle, Index);
				}
			}
			Desc->release();
		}

		{
			MTL::IntersectionFunctionTableDescriptor* Desc = MTL::IntersectionFunctionTableDescriptor::alloc()->init();
			Desc->setFunctionCount(2);
			IntersectionFunctionTable = PipelineStateObject->ComputePipelineState->newIntersectionFunctionTable(Desc);
			
			// Triangle Intersection
			MTL::FunctionHandle* TriangleIntersectionHandle = PipelineStateObject->ComputePipelineState->functionHandle(TriangleIntersectionFunction.get());
			check(TriangleIntersectionHandle);

			IntersectionFunctionTable->setFunction(TriangleIntersectionHandle, 0);

			// Procedural Intersection
			MTL::FunctionHandle* ProceduralIntersectionHandle = PipelineStateObject->ComputePipelineState->functionHandle(ProceduralIntersectionFunction.get());
			check(ProceduralIntersectionHandle);

			IntersectionFunctionTable->setFunction(ProceduralIntersectionHandle, 1);
			
			Desc->release();
		}

		check(PipelineStateObject);

		LinkTime += FPlatformTime::Cycles64();

		TotalCreationTime += FPlatformTime::Cycles64();

		// Report stats for pipelines that take a long time to create
#if !NO_LOGGING
		const double TotalCreationTimeMS = 1000.0 * FPlatformTime::ToSeconds64(TotalCreationTime);
		const float CreationTimeWarningThresholdMS = 10.0f;
		const bool bAllowLogSlowCreation = !Initializer.bBackgroundCompilation; // Only report creation stalls on the critical path
		if (bAllowLogSlowCreation && TotalCreationTimeMS > CreationTimeWarningThresholdMS)
		{
			const double CompileTimeMS = 1000.0 * FPlatformTime::ToSeconds64(CompileTime);
			const double LinkTimeMS = 1000.0 * FPlatformTime::ToSeconds64(LinkTime);
			const uint32 NumUniqueShaders = UniqueShaderCollections.Num();
			UE_LOGF(LogMetal, Log,
				"Creating RTPSO with %d shaders (%d cached, %d new) took %.2f ms. Compile time %.2f ms, link time %.2f ms.",
				PipelineShaderHashes.Num(), NumCacheHits, NumUniqueShaders - NumCacheHits, (float)TotalCreationTimeMS, (float)CompileTimeMS, (float)LinkTimeMS);
		}
#endif //!NO_LOGGING
	}

	~FMetalRayTracingPipelineState()
	{
		for (auto & Pair : PipelineShaderHashes)
		{
			delete Pair.Value;
		}
		
		VisibleFunctionTable->release();
		IntersectionFunctionTable->release();
	}

	FMetalRayShaderLibrary RayGenShaders;
	FMetalRayShaderLibrary MissShaders;
	FMetalRayShaderLibrary HitGroupShaders;
	FMetalRayShaderLibrary CallableShaders;

	FMetalShaderPipelinePtr PipelineStateObject;
	MTLFunctionPtr TriangleIntersectionFunction;
	MTLFunctionPtr ProceduralIntersectionFunction;
	MTL::VisibleFunctionTable* VisibleFunctionTable = nullptr;
	MTL::IntersectionFunctionTable* IntersectionFunctionTable = nullptr;

	TArray<FMetalRayShader*> AllShaders;
	TArray<MTLFunctionPtr> AllShaderFunctions; // Used to generate VisibleFunctionTable

	static constexpr uint32 ShaderIdentifierSize = sizeof(IRShaderIdentifier);

	uint32 MaxLocalRootSignatureSize = 0;

	TMap<uint64, FMetalShaderIdentifier*> PipelineShaderHashes;

	uint32 PipelineStackSize = 0;

#if !NO_LOGGING
	struct FShaderStats
	{
		const TCHAR* Name = nullptr;
		float CompileTimeMS = 0;
		uint32 StackSize = 0;
		uint32 ShaderSize = 0;
	};
	TArray<FShaderStats> ShaderStats;
#endif // !NO_LOGGING
};

void FMetalRayTracingShaderBindingTable::CopyToGPU(FMetalRHICommandContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ShaderTableCopyToGPU);

#if DO_CHECK
	checkf(bWasDefaultMissShaderSet, TEXT("At least the first miss shader must have been set before copying to GPU."));
#endif
	
	FMetalStateCache& StateCache = Context.GetStateCache();
	
	checkf(Data.Num(), TEXT("Shader table is expected to be initialized before copying to GPU."));
	
	{
		FScopeLock Lock(&ResourceCS);

		BoundUniformBuffers.Empty();

		for (uint32 WorkerIdx = 0; WorkerIdx < MaxBindingWorkers; ++WorkerIdx)
		{
			for (FResourceBinder& Binder : ResourceBindings[WorkerIdx])
			{
				const bool bHasLooseParameters = (Binder.LooseParametersVA != 0);

				// +1 to accommodate loose parameters entry in slot 0 when present
				uint64 Entries[FResourceBinder::MaxUniformBuffers + 1];
				uint32 EntryCount = Binder.NumUniformBuffers;
				uint32 CurEntryIdx = 0;
				if (bHasLooseParameters)
				{
					EntryCount++;
					CurEntryIdx++;
				}

				FMemory::Memset(&Entries[0], 0, sizeof(Entries[0]) * EntryCount);
				if (bHasLooseParameters)
				{
					Entries[0] = Binder.LooseParametersVA;
				}

				for (uint32 i = 0; i < Binder.NumUniformBuffers; ++i)
				{
					FMetalBufferPtr BackingBuffer = Binder.BackingBuffers[i];
					if (BackingBuffer)
					{
						StateCache.CacheOrSkipResourceResidencyUpdate(BackingBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);

						Entries[CurEntryIdx] = BackingBuffer->GetGPUAddress();
						BoundUniformBuffers.Add({Binder.UniformBuffers[i], BackingBuffer, MoveTemp(Binder.CapturedResourceTables[i])});

						CurEntryIdx++;
					}
				}
				WriteData(Binder.BindingOffset, &Entries[0], sizeof(Entries[0]) * EntryCount);
			}
		}

		// Clear per-worker arrays after processing (ready for next frame)
		ResetResourceBindings();
	}

	if (bBufferDirty)
	{
		if(Buffer)
		{
			FMetalDynamicRHI::Get().DeferredDelete(Buffer);
		}
		
		Buffer = Device.GetResourceHeap().CreateBuffer(Data.GetResourceDataSize(),
																			 16,
																			 BUF_Static,
																			 MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared);
		
		uint32* DataBuffer = (uint32*)Data.GetResourceData();
		FMemory::Memcpy(Buffer->Contents(), (void*)DataBuffer, Data.GetResourceDataSize());
		
		bBufferDirty = false;
	}
	
	StateCache.CacheOrSkipResourceResidencyUpdate(Buffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);
}

void FMetalRayTracingShaderBindingTable::SetUniformBindings(FMetalRayTracingPipelineState* Pipeline,
															uint32 WorkerIndex,
															uint32 TableOffset,
															uint32 RecordIndex,
															const uint32 NumUniformBuffers,
															FRHIUniformBuffer* const* UniformBuffers,
															const void* LooseParameterData,
															uint32 LooseParameterDataSize)
{
	const uint32 ShaderTableOffset = TableOffset + RecordIndex * LocalRecordStride;

	const IRShaderIdentifier &ShaderIdentifier = ShaderIdentifierAtOffset(ShaderTableOffset);
	check(ShaderIdentifier.shaderHandle || ShaderIdentifier.intersectionShaderHandle);

	const FMetalRayShader *Shader = Pipeline->AllShaders[ShaderIdentifier.shaderHandle ?: ShaderIdentifier.intersectionShaderHandle];
	const TArray<IRRootParameter1> &LocalRootParams = Shader->GetLocalRootParams();

	const uint32 NumCBs = (LocalRootParams.Num() >= 1) ? (LocalRootParams.Num() - 1) : LocalRootParams.Num();
	
	// Early exit if shader has no bindings.
	if (NumCBs == 0)
	{
		return;
	}

	FResourceBinder ResourceBinder;
	check(sizeof(IRShaderIdentifier) + sizeof(FMetalHitGroupSystemParameters) + sizeof(uint64) * NumUniformBuffers <= LocalRecordStride);

	checkf(NumUniformBuffers >= (NumCBs), TEXT("Shader expects %d constant buffers, but provided only %d"), NumCBs, NumUniformBuffers);

	if (LooseParameterData != nullptr)
	{
		checkf(UniformBuffers[0] == nullptr, TEXT("Uniform buffer in slot 0 is reserved for loose parameters (global constant buffer) and must not be set by user code."))
		
		FMetalBufferPtr LooseParams = Device.GetUniformAllocator()->Allocate(LooseParameterDataSize);
		ResourceBinder.LooseParametersVA = LooseParams->GetGPUAddress();
		FMemory::Memcpy(LooseParams->Contents(), LooseParameterData, LooseParameterDataSize);
	}

	for (uint32 i = 0; i < NumUniformBuffers; ++i)
	{
		ResourceBinder.UniformBuffers[i] = UniformBuffers[i];
		checkf(UniformBuffers[i] == nullptr || (uint64_t)UniformBuffers[i] >= 1000, TEXT("UniformBuffer %u of %u is %p"), i, NumUniformBuffers, UniformBuffers[i]);

		if (UniformBuffers[i])
		{
			FMetalUniformBuffer* UB = ResourceCast(UniformBuffers[i]);

			ResourceBinder.BackingBuffers[i] = UB->BackingBuffer;
			ResourceBinder.CapturedResourceTables[i] = UniformBuffers[i]->GetResourceTable();
		}
	}
	ResourceBinder.NumUniformBuffers = NumUniformBuffers;
	
	ResourceBinder.BindingOffset = ShaderTableOffset + sizeof(IRShaderIdentifier) + sizeof(FMetalHitGroupSystemParameters);

	checkf(WorkerIndex < MaxBindingWorkers, TEXT("WorkerIndex %u exceeds MaxBindingWorkers %u"), WorkerIndex, MaxBindingWorkers);
	ResourceBindings[WorkerIndex].Add(MoveTemp(ResourceBinder));

	check(ShaderTableOffset + sizeof(IRShaderIdentifier) + sizeof(FMetalHitGroupSystemParameters) + NumUniformBuffers * sizeof(uint64 /*IRDescriptorTableEntry*/) <= ShaderTableOffset + LocalRecordStride);
}

void FMetalRayTracingShaderBindingTable::SetHitGroupUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 RecordIndex, const uint32 NumUniformBuffers,
																	FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize)
{
	SetUniformBindings(Pipeline, WorkerIndex, HitGroupShaderTableOffset, RecordIndex, NumUniformBuffers, UniformBuffers, LooseParameterData, LooseParameterDataSize);
}

void FMetalRayTracingShaderBindingTable::SetMissUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 RecordIndex, const uint32 NumUniformBuffers,
																FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize)
{
	SetUniformBindings(Pipeline, WorkerIndex, MissShaderTableOffset, RecordIndex, NumUniformBuffers, UniformBuffers, LooseParameterData, LooseParameterDataSize);
}

void FMetalRayTracingShaderBindingTable::SetCallableUniformBindings(FMetalRayTracingPipelineState* Pipeline, uint32 WorkerIndex, uint32 RecordIndex, const uint32 NumUniformBuffers,
																FRHIUniformBuffer* const* UniformBuffers, const void* LooseParameterData, uint32 LooseParameterDataSize)
{
	SetUniformBindings(Pipeline, WorkerIndex, CallableShaderTableOffset, RecordIndex, NumUniformBuffers, UniformBuffers, LooseParameterData, LooseParameterDataSize);
}

FMetalAccelerationStructure::~FMetalAccelerationStructure()
{
	if(IndirectArgumentBuffer)
	{
		FMetalDynamicRHI::Get().DeferredDelete(IndirectArgumentBuffer);
	}
	AccelerationStructure.reset();
}

void FMetalAccelerationStructure::SetIndirectArgumentBuffer(FMetalBufferPtr IndirectArgs)
{
	if(IndirectArgumentBuffer)
	{
		FMetalDynamicRHI::Get().DeferredDelete(IndirectArgumentBuffer);
	}
	IndirectArgumentBuffer = IndirectArgs;
}

static ERayTracingAccelerationStructureFlags GetRayTracingAccelerationStructureBuildFlags(const FRayTracingGeometryInitializer& Initializer)
{
	ERayTracingAccelerationStructureFlags BuildFlags = ERayTracingAccelerationStructureFlags::None;

	if (Initializer.bFastBuild)
	{
		BuildFlags = ERayTracingAccelerationStructureFlags::FastBuild;
	}
	else
	{
		BuildFlags = ERayTracingAccelerationStructureFlags::FastTrace;
	}

	if (Initializer.bAllowUpdate)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate);
	}

	if (!Initializer.bFastBuild && !Initializer.bAllowUpdate && Initializer.bAllowCompaction && GMetalRayTracingAllowCompaction)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction);
	}

	if (GRayTracingDebugForceBuildMode == 1)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastBuild);
		EnumRemoveFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastTrace);
	}
	else if (GRayTracingDebugForceBuildMode == 2)
	{
		EnumAddFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastTrace);
		EnumRemoveFlags(BuildFlags, ERayTracingAccelerationStructureFlags::FastBuild);
	}

	return BuildFlags;
}

static bool ShouldCompactAfterBuild(ERayTracingAccelerationStructureFlags BuildFlags)
{
	return EnumHasAllFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction | ERayTracingAccelerationStructureFlags::FastTrace)
		&& !EnumHasAnyFlags(BuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate);
}

// Manages all the pending BLAS compaction requests
class FMetalRayTracingCompactionRequestHandler
{
public:
	UE_NONCOPYABLE(FMetalRayTracingCompactionRequestHandler)

	FMetalRayTracingCompactionRequestHandler(FMetalDevice& DeviceContext);
	~FMetalRayTracingCompactionRequestHandler();

	void RequestCompact(FMetalRayTracingGeometry* InRTGeometry);
	bool ReleaseRequest(FMetalRayTracingGeometry* InRTGeometry);

	void Update(FMetalRHICommandContext& Context);

private:
	/** Enqueued requests (waiting on size request submit). */
	TArray<FMetalRayTracingGeometry*> PendingRequests;

	/** Enqueued compaction requests (submitted compaction size requests; waiting on readback and actual compaction). */
	TArray<FMetalRayTracingGeometry*> ActiveRequests;

	/** Buffer used for compacted size readback. */
	FMetalBufferPtr CompactedStructureSizeBuffer;

	/** Size entry allocated in the CompactedStructureSizeBuffer (in element count). */
	uint32 SizeBufferMaxCapacity;
	
	FCriticalSection CS;
	FMetalSyncPoint* CompactionQuerySyncPoint = nullptr;
};

FMetalRayTracingCompactionRequestHandler::FMetalRayTracingCompactionRequestHandler(FMetalDevice& Device)
	: SizeBufferMaxCapacity(GMetalRayTracingMaxBatchedCompaction)
{
	MTL::Buffer* BufferPtr = Device.GetDevice()->newBuffer(GMetalRayTracingMaxBatchedCompaction * sizeof(uint32), MTL::ResourceStorageModeShared);
	CompactedStructureSizeBuffer = FMetalBufferPtr(new FMetalBuffer(BufferPtr, FMetalBuffer::FreePolicy::Owner));

	check(CompactedStructureSizeBuffer);
}

FMetalRayTracingCompactionRequestHandler::~FMetalRayTracingCompactionRequestHandler()
{
	check(PendingRequests.IsEmpty());
	
	FMetalDynamicRHI::Get().DeferredDelete(CompactedStructureSizeBuffer);
	
	if(CompactionQuerySyncPoint)
	{
		CompactionQuerySyncPoint->Release();
		CompactionQuerySyncPoint = nullptr;
	}
}

void FMetalRayTracingCompactionRequestHandler::RequestCompact(FMetalRayTracingGeometry* InRTGeometry)
{
	FScopeLock Lock(&CS);
	
	check(InRTGeometry->GetAccelerationStructure());
	ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(InRTGeometry->Initializer);
	check(EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowCompaction) &&
		EnumHasAllFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::FastTrace) &&
		!EnumHasAnyFlags(GeometryBuildFlags, ERayTracingAccelerationStructureFlags::AllowUpdate));

	PendingRequests.Add(InRTGeometry);
}

bool FMetalRayTracingCompactionRequestHandler::ReleaseRequest(FMetalRayTracingGeometry* InRTGeometry)
{
	FScopeLock Lock(&CS);

	// Remove from pending list, not found then try active requests
	if (PendingRequests.Remove(InRTGeometry) <= 0)
	{
		// If currently enqueued, then clear pointer to not handle the compaction request anymore			
		for (int32 BLASIndex = 0; BLASIndex < ActiveRequests.Num(); ++BLASIndex)
		{
			if (ActiveRequests[BLASIndex] == InRTGeometry)
			{
				ActiveRequests[BLASIndex] = nullptr;
				return true;
			}
		}

		return false;
	}
	
	return true;
}

void FMetalRHICommandContext::WriteCompactedAccelerationStructureSize(MTLAccelerationStructurePtr AccelerationStructure, FMetalBufferPtr CompactedStructureSizeBuffer, uint32 Offset)
{
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	if(!CurrentEncoder.IsAccelerationStructureCommandEncoderActive())
	{
		StateCache.ClearPreviousComputeState();
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
		}
		
		// TODO: Carl - Sample Counters
		CurrentEncoder.BeginAccelerationStructureCommandEncoding();
	}

	MTL::AccelerationStructureCommandEncoder* CommandEncoder = CurrentEncoder.GetAccelerationStructureCommandEncoder();
	CommandEncoder->writeCompactedAccelerationStructureSize(AccelerationStructure.get(), CompactedStructureSizeBuffer->GetMTLBuffer(), Offset);
}

void FMetalRHICommandContext::CopyAndCompactAccelerationStructure(MTLAccelerationStructurePtr AccelerationStructureSrc, MTLAccelerationStructurePtr AccelerationStructureDest)
{
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	if(!CurrentEncoder.IsAccelerationStructureCommandEncoderActive())
	{
		StateCache.ClearPreviousComputeState();
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
		}
		
		// TODO: Carl - Sample Counters
		CurrentEncoder.BeginAccelerationStructureCommandEncoding();
	}

	MTL::AccelerationStructureCommandEncoder* CommandEncoder = CurrentEncoder.GetAccelerationStructureCommandEncoder();
	CommandEncoder->copyAndCompactAccelerationStructure(AccelerationStructureSrc.get(), AccelerationStructureDest.get());
}

void FMetalRayTracingCompactionRequestHandler::Update(FMetalRHICommandContext& Context)
{
	FScopeLock Lock(&CS);
	
	// Early exit to avoid unecessary encoding breaks.
	if (PendingRequests.IsEmpty() && ActiveRequests.IsEmpty())
	{
		return;
	}
	
	// Process active requests.
	if(CompactionQuerySyncPoint && CompactionQuerySyncPoint->IsComplete())
	{
		// Try to readback active requests.
		uint32* CompactedSizes = (uint32*)CompactedStructureSizeBuffer->Contents();
		
		for (FMetalRayTracingGeometry* ActiveRequestsTail : ActiveRequests)
		{	
			if(!ActiveRequestsTail)
			{
				continue;
			}
			
			uint32 CompactedSize = CompactedSizes[ActiveRequestsTail->CompactionSizeIndex];
			check(CompactedSize != 0);
			
			FMetalAccelerationStructure* SrcBuffer = ActiveRequestsTail->GetAccelerationStructure();
			MTLAccelerationStructurePtr SrcBLAS = SrcBuffer->GetAccelerationStructure();
			
			// Allocate new acceleration structure with compacted size
			FMetalAccelerationStructure* DestAccelerationStructure = Context.GetDevice().GetResourceHeap().CreateAccelerationStructure(CompactedSize);
			
			if(GetEmitDrawEvents())
			{
				FString DebugNameString = ActiveRequestsTail->Initializer.DebugName.ToString();
				DestAccelerationStructure->GetAccelerationStructure()->setLabel(FStringToNSString(DebugNameString));
			}
			check(DestAccelerationStructure);
			
			MTLAccelerationStructurePtr DestBLAS = DestAccelerationStructure->GetAccelerationStructure();
			Context.CopyAndCompactAccelerationStructure(SrcBLAS, DestBLAS);
			
			// Swap acceleration structure buffers 
			ActiveRequestsTail->SetAccelerationStructure(DestAccelerationStructure);	
		}
		
		ActiveRequests.Empty(ActiveRequests.Num());
		
		CompactionQuerySyncPoint->Release();
		CompactionQuerySyncPoint = nullptr;
	}
	
	// Process pending requests.
	if(ActiveRequests.IsEmpty())
	{
		uint32* CompactedSizes = (uint32*)CompactedStructureSizeBuffer->Contents();
		uint32 CompactionIndex = 0;
		
		while (!PendingRequests.IsEmpty())
		{
			FMetalRayTracingGeometry* Geometry = PendingRequests[0];
			
			Geometry->CompactionSizeIndex = CompactionIndex++;
			CompactedSizes[Geometry->CompactionSizeIndex] = 0;
			
			FMetalAccelerationStructure* AccelerationStructure = Geometry->GetAccelerationStructure();
			Context.WriteCompactedAccelerationStructureSize(AccelerationStructure->GetAccelerationStructure(), CompactedStructureSizeBuffer, Geometry->CompactionSizeIndex * sizeof(uint32));
			
			ActiveRequests.Add(Geometry);			
			PendingRequests.Remove(Geometry);
			
			// enqueued enough requests for this update round
			if (ActiveRequests.Num() >= GMetalRayTracingMaxBatchedCompaction)
			{
				break;
			}
		}
		
		CompactionQuerySyncPoint = Context.GetContextSyncPoint();
		CompactionQuerySyncPoint->AddRef();
	}
}

/** Fills a MTLPrimitiveAccelerationStructureDescriptor with infos provided by the UE5 geometry descriptor.
 * This function assumes that GeometryDescriptors has already been allocated, and that you are responsible of its lifetime.
 */
static void FillPrimitiveAccelerationStructureDesc(FMetalDevice& Device, MTL::PrimitiveAccelerationStructureDescriptor* AccelerationStructureDescriptor, const FRayTracingGeometryInitializer& Initializer)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	TArray<MTL::AccelerationStructureGeometryDescriptor*> GeometryDescriptors;
	
	// Populate Segments Descriptors.
	FMetalRHIBuffer* IndexBuffer = ResourceCast(Initializer.IndexBuffer.GetReference());

	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		// Vertex Buffer Infos
		FMetalRHIBuffer* VertexBuffer = ResourceCast(Segment.VertexBuffer.GetReference());
		check(VertexBuffer);
		
		const FMetalBufferPtr VertexBufferRes = VertexBuffer->GetCurrentBufferOrNull();
		
		if (Initializer.GeometryType == RTGT_Triangles)
		{
			MTL::AccelerationStructureTriangleGeometryDescriptor* GeometryDescriptor = MTL::AccelerationStructureTriangleGeometryDescriptor::alloc()->init();
			
			switch (Segment.VertexBufferElementType)
			{
				case VET_Float4:
					GeometryDescriptor->setVertexFormat(MTL::AttributeFormatFloat4);
					break;
				case VET_Float3:
					GeometryDescriptor->setVertexFormat(MTL::AttributeFormatFloat3);
					break;
				case VET_Float2:
					GeometryDescriptor->setVertexFormat(MTL::AttributeFormatFloat2);
					break;
				case VET_Half2:
					GeometryDescriptor->setVertexFormat(MTL::AttributeFormatHalf2);
					break;
				default:
					checkNoEntry();
					break;
			}
			
			GeometryDescriptor->setOpaque(Segment.bForceOpaque);
			GeometryDescriptor->setTriangleCount((Segment.bEnabled) ? Segment.NumPrimitives : 0);
			GeometryDescriptor->setAllowDuplicateIntersectionFunctionInvocation(Segment.bAllowDuplicateAnyHitShaderInvocation);
			
			// Index Buffer Infos
			if (IndexBuffer != nullptr)
			{
				FMetalBufferPtr IndexBufferRes = IndexBuffer->GetCurrentBufferOrNull();
				
				// Metal does not provide the same size for an AS without an index buffer, so provide a dummy, size doesn't matter
				if(!IndexBufferRes)
				{
					IndexBufferRes = Device.DummyIndexBuffer;
				}
				
				GeometryDescriptor->setIndexType(IndexBuffer->GetIndexType());
				GeometryDescriptor->setIndexBuffer(IndexBufferRes ? IndexBufferRes->GetMTLBuffer() : nullptr);
				
				const uint32 IndexStride = IndexBuffer ? IndexBuffer->GetStride() : 0;
				uint32 IndexBufferOffset = Initializer.IndexBufferOffset + IndexStride * Segment.FirstPrimitive * FMetalRayTracingGeometry::IndicesPerPrimitive;
				GeometryDescriptor->setIndexBufferOffset(IndexBufferRes ? IndexBufferRes->GetOffset() + IndexBufferOffset : 0);
			}
			
			GeometryDescriptor->setVertexBuffer(VertexBufferRes ? VertexBufferRes->GetMTLBuffer() : nullptr);
			GeometryDescriptor->setVertexBufferOffset(VertexBufferRes ? VertexBufferRes->GetOffset() + Segment.VertexBufferOffset : 0);
			GeometryDescriptor->setVertexStride(Segment.VertexBufferStride);
			
			GeometryDescriptor->setIntersectionFunctionTableOffset(0);
			
			GeometryDescriptors.Add(GeometryDescriptor);
		}
		else if (Initializer.GeometryType == RTGT_Procedural)
		{
			MTL::AccelerationStructureBoundingBoxGeometryDescriptor* GeometryDescriptor = MTL::AccelerationStructureBoundingBoxGeometryDescriptor::alloc()->init();
			
			GeometryDescriptor->setOpaque(Segment.bForceOpaque);
			GeometryDescriptor->setAllowDuplicateIntersectionFunctionInvocation(Segment.bAllowDuplicateAnyHitShaderInvocation);
			
			GeometryDescriptor->setBoundingBoxStride(Segment.VertexBufferStride);
			GeometryDescriptor->setBoundingBoxBuffer(VertexBufferRes ? VertexBufferRes->GetMTLBuffer() : nullptr);
			GeometryDescriptor->setBoundingBoxBufferOffset(VertexBufferRes ? VertexBufferRes->GetOffset() + Segment.VertexBufferOffset : 0);
			GeometryDescriptor->setBoundingBoxCount(Segment.NumPrimitives);
			
			GeometryDescriptor->setIntersectionFunctionTableOffset(1);
			
			GeometryDescriptors.Add(GeometryDescriptor);
		}
		else
		{
			check(0);
		}
	}

	// Populate Acceleration Structure Descriptor.
	uint32 Usage = MTLAccelerationStructureUsageNone;

	if (Initializer.bAllowUpdate)
	{
		Usage = MTL::AccelerationStructureUsageRefit;
	}
	else if (Initializer.bFastBuild)
	{
		Usage = MTL::AccelerationStructureUsagePreferFastBuild;
	}

	AccelerationStructureDescriptor->setUsage(Usage);
	AccelerationStructureDescriptor->setGeometryDescriptors(NS::Array::alloc()->init((const NS::Object* const*)GeometryDescriptors.GetData(), GeometryDescriptors.Num()));
}

void ReleasePrimitiveAccelerationStructureGeometryDescriptors(MTL::PrimitiveAccelerationStructureDescriptor* Desc)
{
	// Because of lack of NS::MutableArray in Metal-cpp we need to manually free descriptors for all acceleration structures
	// instead of creating them on the stack when calculating size 
	NS::Array* GeometryDescriptors = Desc->geometryDescriptors();
	
	for(uint32 Idx = 0; Idx < GeometryDescriptors->count(); Idx++)
	{
		GeometryDescriptors->object(Idx)->release();	
	}
	
	GeometryDescriptors->release();	
}

static FRayTracingAccelerationStructureSize CalcRayTracingGeometrySize(FMetalDevice& Device, MTL::AccelerationStructureDescriptor* AccelerationStructureDescriptor)
{
	MTL::AccelerationStructureSizes DescriptorSize = Device.GetDevice()->accelerationStructureSizes(AccelerationStructureDescriptor);

	FRayTracingAccelerationStructureSize SizeInfo = {};
	SizeInfo.ResultSize = Align(DescriptorSize.accelerationStructureSize, GRHIRayTracingAccelerationStructureAlignment);
	SizeInfo.BuildScratchSize = Align(DescriptorSize.buildScratchBufferSize, GRHIRayTracingScratchBufferAlignment);
	SizeInfo.UpdateScratchSize = Align(FMath::Max(1u, (uint32)DescriptorSize.refitScratchBufferSize), GRHIRayTracingScratchBufferAlignment);
	
	return SizeInfo;
}

FRayTracingAccelerationStructureSize FMetalDynamicRHI::RHICalcRayTracingGeometrySize(const FRayTracingGeometryInitializer& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	
	MTL::PrimitiveAccelerationStructureDescriptor* AccelerationStructureDescriptor = MTL::PrimitiveAccelerationStructureDescriptor::descriptor();
	FillPrimitiveAccelerationStructureDesc(*Device, AccelerationStructureDescriptor, Initializer);

	FRayTracingAccelerationStructureSize Ret = CalcRayTracingGeometrySize(*Device, AccelerationStructureDescriptor);
	
	ReleasePrimitiveAccelerationStructureGeometryDescriptors(AccelerationStructureDescriptor);
	return Ret;
}

FRayTracingAccelerationStructureSize FMetalDynamicRHI::RHICalcRayTracingSceneSize(const FRayTracingSceneInitializer& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    MTL::InstanceAccelerationStructureDescriptor* InstanceDescriptor = MTL::InstanceAccelerationStructureDescriptor::descriptor();
    InstanceDescriptor->setInstanceCount(Initializer.MaxNumInstances);

    return CalcRayTracingGeometrySize(*Device, InstanceDescriptor);
}

FMetalRayTracingGeometry::FMetalRayTracingGeometry(FMetalDevice& InDevice, FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& InInitializer)
	: FRHIRayTracingGeometry(InInitializer),
	bHasPendingCompactionRequests(false),
	Device(InDevice)
{
	uint32 IndexBufferStride = 0;

	if (Initializer.IndexBuffer)
	{
		// In case index buffer in initializer is not yet in valid state during streaming we assume the geometry is using UINT32 format.
		IndexBufferStride = Initializer.IndexBuffer->GetSize() > 0
			? Initializer.IndexBuffer->GetStride()
			: 4;
	}
	checkf(!Initializer.IndexBuffer || (IndexBufferStride == 2 || IndexBufferStride == 4), TEXT("Index buffer must be 16 or 32 bit if in use."));

	RebuildDescriptors();

	// NOTE: We do not use the RHI API in order to avoid re-filling another descriptor.
	SizeInfo = CalcRayTracingGeometrySize(Device, AccelerationStructureDescriptor);
	
	AccelerationStructureIndex = 0;

	// If this RayTracingGeometry going to be used as streaming destination
	// we don't want to allocate its memory as it will be replaced later by streamed version
	// but we still need correct SizeInfo as it is used to estimate its memory requirements outside of RHI.
	if (Initializer.Type == ERayTracingGeometryInitializerType::StreamingDestination)
	{
		return;
	}

	FString DebugNameString = Initializer.DebugName.ToString();
	
	AccelerationStructure = Device.GetResourceHeap().CreateAccelerationStructure(SizeInfo.ResultSize);
	if(GetEmitDrawEvents())
	{
		AccelerationStructure->GetAccelerationStructure()->setLabel(FStringToNSString(DebugNameString));
	}
	check(AccelerationStructure);
}

FMetalRayTracingGeometry::~FMetalRayTracingGeometry()
{
	ReleaseUnderlyingResource();
	ReleaseBindlessHandles();
}

void FMetalRayTracingGeometry::ReleaseUnderlyingResource()
{
	RemoveCompactionRequest();
	
	if(AccelerationStructure)
	{
		FMetalDynamicRHI::Get().DeferredDelete(AccelerationStructure);
	}
	AccelerationStructure = nullptr;
	
	ReleaseDescriptors();
}

void FMetalRayTracingGeometry::ReleaseDescriptors()
{
	if(AccelerationStructureDescriptor)
	{
		ReleasePrimitiveAccelerationStructureGeometryDescriptors(AccelerationStructureDescriptor);
		AccelerationStructureDescriptor->release();
		AccelerationStructureDescriptor = nullptr;
	}
}

void FMetalRayTracingGeometry::Swap(FMetalRayTracingGeometry& Other)
{
	::Swap(AccelerationStructureDescriptor, Other.AccelerationStructureDescriptor);
	
	::Swap(AccelerationStructure, Other.AccelerationStructure);
	::Swap(AccelerationStructureIndex, Other.AccelerationStructureIndex);

	Initializer = Other.Initializer;

	SetupHitGroupSystemParameters();
	RebuildDescriptors();
	
	SizeInfo = CalcRayTracingGeometrySize(Device, AccelerationStructureDescriptor);
}

void FMetalRayTracingGeometry::RemoveCompactionRequest()
{
	if (bHasPendingCompactionRequests)
	{
		check(GetAccelerationStructure());
		bool bRequestFound = Device.GetRayTracingCompactionRequestHandler()->ReleaseRequest(this);
		bHasPendingCompactionRequests = false;
	}
}

void FMetalRayTracingGeometry::RebuildDescriptors()
{
	if(AccelerationStructureDescriptor)
	{
		ReleaseDescriptors();
	}
	
	AccelerationStructureDescriptor = MTL::PrimitiveAccelerationStructureDescriptor::alloc()->init();
	FillPrimitiveAccelerationStructureDesc(Device, AccelerationStructureDescriptor, Initializer);
}

static void SetRayTracingHitGroup(
	FMetalRayTracingShaderBindingTable* ShaderTable,
	uint32 WorkerIndex,
	uint32 RecordIndex,
	FMetalRayTracingPipelineState* Pipeline,
	const FMetalRayTracingGeometry* Geometry,
	uint32 GeometrySegmentIndex,
	uint32 HitGroupIndex,
	uint32 NumUniformBuffers,
	FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize,
	const void* LooseParameterData,
	uint32 UserData)
{
	ERayTracingShaderBindingMode ShaderBindingMode = ShaderTable->GetShaderBindingMode();
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode = ShaderTable->GetHitGroupIndexingMode();

	// If Shader table doesn't support hit group indexing then only set the hit group identifier and it should be first record index
	if (HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Disallow &&
		EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
	{
		check(RecordIndex == 0 && Pipeline);
		if (Pipeline)
		{
			ShaderTable->SetHitGroupIdentifier(RecordIndex, *Pipeline->HitGroupShaders.Identifiers[HitGroupIndex]);
		}
		return;
	}
	
	checkf(RecordIndex < ShaderTable->NumHitRecords, TEXT("Hit group record index is invalid. Make sure that NumGeometrySegments and NumShaderSlotsPerGeometrySegment is correct in FRayTracingShaderBindingTableInitializer."));
	
	if (HitGroupIndexingMode == ERayTracingHitGroupIndexingMode::Allow && Geometry)
	{		
		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO))
		{
			ShaderTable->SetHitGroupIdentifier(RecordIndex, *Pipeline->HitGroupShaders.Identifiers[HitGroupIndex]);
			
			FMetalHitGroupSystemParameters GeometryParameters = Geometry->HitGroupSystemParameters[GeometrySegmentIndex];
			
			GeometryParameters.RootConstants.UserData = UserData;
			ShaderTable->SetHitGroupSystemParameters(RecordIndex, GeometryParameters);
			ShaderTable->SetHitGroupUniformBindings(Pipeline, WorkerIndex, RecordIndex, NumUniformBuffers, UniformBuffers, LooseParameterData, LooseParameterDataSize);
		}

		if (EnumHasAnyFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline))
		{			
			// Only care about shader slot 0 for inline geometry parameters
			uint32 InlineRecordIndex = ShaderTable->GetInlineRecordIndex(RecordIndex);
			
			if(InlineRecordIndex != INDEX_NONE)
			{
				ShaderTable->SetInlineGeometryParameters(InlineRecordIndex, &Geometry->HitGroupSystemParameters[GeometrySegmentIndex], sizeof(FMetalHitGroupSystemParameters));
			}
		}
	}
}

static void SetRayTracingCallableShader(
	FMetalRayTracingShaderBindingTable* ShaderTable,
	uint32 WorkerIndex,
	FMetalRayTracingPipelineState* Pipeline,
	uint32 RecordIndex,
	uint32 ShaderIndexInPipeline,
	uint32 NumUniformBuffers,
	FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize,
	const void* LooseParameterData,
	uint32 UserData)
{
	check(LooseParameterDataSize == 0 && LooseParameterData == nullptr);

	const uint32 UserDataOffset = offsetof(FMetalHitGroupSystemParameters, RootConstants) + offsetof(FHitGroupSystemRootConstants, UserData);
	ShaderTable->SetCallableShaderParameters(RecordIndex, UserDataOffset, UserData);

	check(ShaderIndexInPipeline < Pipeline->CallableShaders.Shaders.Num());
	check(ShaderIndexInPipeline < Pipeline->CallableShaders.Identifiers.Num());

	ShaderTable->SetCallableIdentifier(RecordIndex, *Pipeline->CallableShaders.Identifiers[ShaderIndexInPipeline]);
	ShaderTable->SetCallableUniformBindings(Pipeline, WorkerIndex, RecordIndex, NumUniformBuffers, UniformBuffers, LooseParameterData, LooseParameterDataSize);

	UE_LOGF(LogMetal, Log, "Binding callable shader: RecordIndex=%u, ShaderIndex=%u, Handle=%llu",
		  RecordIndex, ShaderIndexInPipeline,
		  Pipeline->CallableShaders.Identifiers[ShaderIndexInPipeline]->ShaderIdentifier.shaderHandle);
}

static void SetRayTracingMissShader(
	FMetalRayTracingShaderBindingTable* ShaderTable,
	uint32 WorkerIndex,
	FMetalRayTracingPipelineState* Pipeline,
	uint32 RecordIndex,
	uint32 ShaderIndexInPipeline,
	uint32 NumUniformBuffers,
	FRHIUniformBuffer* const* UniformBuffers,
	uint32 LooseParameterDataSize,
	const void* LooseParameterData,
	uint32 UserData)
{
	check(LooseParameterDataSize == 0 && LooseParameterData == nullptr);

	const uint32 UserDataOffset = offsetof(FMetalHitGroupSystemParameters, RootConstants) + offsetof(FHitGroupSystemRootConstants, UserData);
	ShaderTable->SetMissShaderParameters(RecordIndex, UserDataOffset, UserData);

	check(ShaderIndexInPipeline < Pipeline->MissShaders.Shaders.Num());
	check(ShaderIndexInPipeline < Pipeline->MissShaders.Identifiers.Num());

	ShaderTable->SetMissIdentifier(RecordIndex, *Pipeline->MissShaders.Identifiers[ShaderIndexInPipeline]);
	ShaderTable->SetMissUniformBindings(Pipeline, WorkerIndex, RecordIndex, NumUniformBuffers, UniformBuffers, LooseParameterData, LooseParameterDataSize);
}

void FMetalRHICommandContext::RHICommitShaderBindingTable(FRHIShaderBindingTable* InSBT, FRHIBuffer* InlineBindingDataBuffer)
{
	FMetalRayTracingShaderBindingTable* SBT = ResourceCast(InSBT);
	
	SBT->Commit(this, InlineBindingDataBuffer);
}

void FMetalRHICommandContext::RHISetBindingsOnShaderBindingTable(
	FRHIShaderBindingTable* InSBT,
	FRHIRayTracingPipelineState* InPipeline,
	uint32 NumBindings, 
	const FRayTracingLocalShaderBindings* Bindings,
	ERayTracingBindingType BindingType)
{
	FMetalRayTracingShaderBindingTable* ShaderTable = ResourceCast(InSBT);
	FMetalRayTracingPipelineState* Pipeline = (FMetalRayTracingPipelineState*)InPipeline;
	
	FGraphEventArray TaskList;

	const uint32 NumWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
	const uint32 MaxTasks = FApp::ShouldUseThreadingForPerformance() ? FMath::Min<uint32>(NumWorkerThreads, FMetalRayTracingShaderBindingTable::MaxBindingWorkers) : 1;

	struct FTaskContext
	{
		uint32 WorkerIndex = 0;
	};

	TArray<FTaskContext, TInlineAllocator<FMetalRayTracingShaderBindingTable::MaxBindingWorkers>> TaskContexts;
	for (uint32 WorkerIndex = 0; WorkerIndex < MaxTasks; ++WorkerIndex)
	{
		TaskContexts.Add(FTaskContext{ WorkerIndex });
	}

	FCriticalSection CacheLock;
	
	TLockFreePointerListUnordered<MTL::Resource, PLATFORM_CACHE_LINE_SIZE> UsedResources;
	
	if (BindingType == ERayTracingBindingType::HitGroup)
	{
		ShaderTable->ResetBindingTableHeaps();
	}
	
	auto BindingTask = [this, Pipeline, Bindings, ShaderTable, BindingType, &CacheLock, &UsedResources](const FTaskContext& Context, int32 CurrentIndex)
	{
		TSet<MTL::Heap*> Heaps;
		const FRayTracingLocalShaderBindings& Binding = Bindings[CurrentIndex];

		if (BindingType == ERayTracingBindingType::HitGroup)
		{
			const FMetalRayTracingGeometry* Geometry = static_cast<const FMetalRayTracingGeometry*>(Binding.Geometry);

			if (Binding.BindingType != ERayTracingLocalShaderBindingType::Clear)
			{
				SetRayTracingHitGroup(ShaderTable,
									  Context.WorkerIndex,
									  Binding.RecordIndex,
									  Pipeline,
									  Geometry,
									  Binding.SegmentIndex,
									  Binding.ShaderIndexInPipeline,
									  Binding.NumUniformBuffers,
									  Binding.UniformBuffers,
									  Binding.LooseParameterDataSize,
									  Binding.LooseParameterData,
									  Binding.UserData);
				
				if(Geometry)
				{
					auto AddHeapFromResource = [&Heaps](MTL::Resource* Resource)
					{
						if(Resource->heap())
						{
							Heaps.Add(Resource->heap());
						}
					};
					
					AddHeapFromResource(Geometry->GetAccelerationStructure()->GetAccelerationStructure().get());
					
					if(Geometry->GetAccelerationStructure()->GetHitContributionsBuffer())
					{
						AddHeapFromResource(Geometry->GetAccelerationStructure()->GetHitContributionsBuffer()->GetMTLBuffer());
					}
					
					FMetalRHIBuffer* IndexBuffer = ResourceCast(Geometry->Initializer.IndexBuffer.GetReference());
					if(IndexBuffer)
					{
						AddHeapFromResource(IndexBuffer->GetCurrentBuffer()->GetMTLBuffer());
					}
					
					for (const FRayTracingGeometrySegment& Segment : Geometry->Initializer.Segments)
					{
						FMetalRHIBuffer* VertexBuffer = ResourceCast(Segment.VertexBuffer.GetReference());
						AddHeapFromResource(VertexBuffer->GetCurrentBuffer()->GetMTLBuffer());	
					}
				}
				
				ShaderTable->AddBindingTableHeaps(Context.WorkerIndex, Heaps);
			}
			else
			{
				check(ShaderTable->GetLifetime() == ERayTracingShaderBindingTableLifetime::Transient);
			}
		}
		else if (BindingType == ERayTracingBindingType::CallableShader)
		{
			SetRayTracingCallableShader(
							ShaderTable,
							Context.WorkerIndex,
							Pipeline,
							Binding.RecordIndex,
							Binding.ShaderIndexInPipeline,
							Binding.NumUniformBuffers,
							Binding.UniformBuffers,
							Binding.LooseParameterDataSize,
							Binding.LooseParameterData,
							Binding.UserData);
		}
		else if (BindingType == ERayTracingBindingType::MissShader)
		{
			SetRayTracingMissShader(
							ShaderTable,
							Context.WorkerIndex,
							Pipeline,
							Binding.RecordIndex,
							Binding.ShaderIndexInPipeline,
							Binding.NumUniformBuffers,
							Binding.UniformBuffers,
							Binding.LooseParameterDataSize,
							Binding.LooseParameterData,
							Binding.UserData);
		}
		else
		{
			checkNoEntry();
		}
	};

	// One helper worker task will be created at most per this many work items, plus one worker for current thread (unless running on a task thread),
	// up to a hard maximum of FMetalRayTracingShaderTable::MaxBindingWorkers.
	// Internally, parallel for tasks still subdivide the work into smaller chunks and perform fine-grained load-balancing.
	const int32 ItemsPerTask = 1024;

	ParallelForWithExistingTaskContext(TEXT("SetRayTracingBindings"), MakeArrayView(TaskContexts), NumBindings, ItemsPerTask, BindingTask);
	
	ShaderTable->MarkBufferDirty();
}

void FMetalRayTracingGeometry::SetAccelerationStructure(FMetalAccelerationStructure* InBuffer)
{
	if(AccelerationStructure)
	{
		FMetalDynamicRHI::Get().DeferredDelete(AccelerationStructure);
	}
	AccelerationStructure = InBuffer;
}

void FMetalRayTracingGeometry::SetupHitGroupSystemParameters()
{
	const bool bIsTriangles = (Initializer.GeometryType == RTGT_Triangles);

	FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
	auto GetBindlessHandle = [BindlessDescriptorManager](FMetalRHIBuffer* Buffer, uint32 ExtraOffset)
	{
		if (Buffer)
		{
			FRHIDescriptorHandle BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::BufferSRV, Buffer->GetCurrentBuffer().Get(), ExtraOffset);	
			return BindlessHandle;
		}
		return FRHIDescriptorHandle();
	};

	ReleaseBindlessHandles();

	HitGroupSystemParameters.Reset(Initializer.Segments.Num());

	FMetalRHIBuffer* IndexBuffer = ResourceCast(Initializer.IndexBuffer.GetReference());
	const uint32 IndexStride = IndexBuffer ? IndexBuffer->GetStride() : 0;
	
	if(IndexBuffer)
	{
		HitGroupSystemIndexView = GetBindlessHandle(IndexBuffer, 0);
		check(HitGroupSystemIndexView.GetIndex());
	}
	
	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		FMetalRHIBuffer* VertexBuffer = ResourceCast(Segment.VertexBuffer.GetReference());
		const FRHIDescriptorHandle VBHandle = GetBindlessHandle(VertexBuffer, Segment.VertexBufferOffset);
		HitGroupSystemVertexViews.Add(VBHandle);

		FMetalHitGroupSystemParameters& SystemParameters = HitGroupSystemParameters.AddZeroed_GetRef();
		SystemParameters.RootConstants.SetVertexAndIndexStride(Segment.VertexBufferStride, IndexStride);
		
		SystemParameters.BindlessHitGroupSystemVertexBuffer = VBHandle.GetIndex();
		check(SystemParameters.BindlessHitGroupSystemVertexBuffer);
		
		if (bIsTriangles && (IndexBuffer != nullptr))
		{
			SystemParameters.BindlessHitGroupSystemIndexBuffer = HitGroupSystemIndexView.GetIndex();
			SystemParameters.RootConstants.IndexBufferOffsetInBytes = Initializer.IndexBufferOffset + IndexStride * Segment.FirstPrimitive * FMetalRayTracingGeometry::IndicesPerPrimitive;
			SystemParameters.RootConstants.FirstPrimitive = Segment.FirstPrimitive;
		}
	}
}

void FMetalRayTracingGeometry::ReleaseBindlessHandles()
{
	FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();

	for (FRHIDescriptorHandle BindlesHandle : HitGroupSystemVertexViews)
	{
		FMetalDynamicRHI::Get().DeferredDelete(BindlesHandle);
	}
	HitGroupSystemVertexViews.Reset(Initializer.Segments.Num());

	if (HitGroupSystemIndexView.IsValid())
	{
		FMetalDynamicRHI::Get().DeferredDelete(HitGroupSystemIndexView);
		HitGroupSystemIndexView = {};
	}
}

FMetalRayTracingScene::FMetalRayTracingScene(FMetalDevice& InDevice, FRayTracingSceneInitializer InInitializer)
	: Device(InDevice),
	Initializer(MoveTemp(InInitializer))
{
	MTL::InstanceAccelerationStructureDescriptor* InstanceDescriptor = MTL::InstanceAccelerationStructureDescriptor::alloc()->init();
	InstanceDescriptor->setInstanceCount(Initializer.MaxNumInstances);

	SizeInfo = CalcRayTracingGeometrySize(Device, InstanceDescriptor);
	
	InstanceDescriptor->release();
}

FMetalRayTracingScene::~FMetalRayTracingScene()
{
}

void FMetalRayTracingScene::BindBuffer(FRHIBuffer* InBuffer, uint32 InBufferOffset)
{
	check(IsInRHIThread() || !IsRunningRHIInSeparateThread());
	check(SizeInfo.ResultSize + InBufferOffset <= InBuffer->GetSize());

	AccelerationStructureBuffer = ResourceCast(InBuffer);
	
	check(AccelerationStructureBuffer->IsAccelerationStructure());
	
	check(InBufferOffset % GRHIRayTracingAccelerationStructureAlignment == 0);
	
	if(GetEmitDrawEvents())
	{
		GetAccelerationStructure()->GetAccelerationStructure()->setLabel(FStringToNSString(Initializer.DebugName.ToString()));
	}
}

uint64 FMetalRayTracingGeometry::GetGPUAddress() const
{
	return AccelerationStructure->GetAccelerationStructure()->gpuResourceID()._impl;
}

void FMetalRHICommandContext::BuildAccelerationStructure(FMetalBufferPtr CurInstanceBuffer, uint32 InstanceBufferOffset, 
														 FMetalBufferPtr ScratchBuffer, uint32 ScratchBufferOffset,
														 FMetalBufferPtr HitGroupContributionsBuffer, uint32 HitGroupContributionsBufferOffset, 
														 uint32 MaxNumInstances, FMetalAccelerationStructure* AS)
{
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	if(!CurrentEncoder.IsAccelerationStructureCommandEncoderActive())
	{
		StateCache.ClearPreviousComputeState();
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
		}
		
		// TODO: Carl - Sample Counters
		CurrentEncoder.BeginAccelerationStructureCommandEncoding();
	}

	MTL::AccelerationStructureCommandEncoder* CommandEncoder = CurrentEncoder.GetAccelerationStructureCommandEncoder();
	
	MTL::InstanceAccelerationStructureDescriptor* InstanceDescriptor = MTL::InstanceAccelerationStructureDescriptor::descriptor();
	InstanceDescriptor->setInstanceCount(MaxNumInstances);
	InstanceDescriptor->setInstanceDescriptorBuffer(CurInstanceBuffer->GetMTLBuffer());
	InstanceDescriptor->setInstanceDescriptorBufferOffset(InstanceBufferOffset);
	InstanceDescriptor->setInstanceDescriptorStride(GRHIRayTracingInstanceDescriptorSize);
	InstanceDescriptor->setInstanceDescriptorType(MTL::AccelerationStructureInstanceDescriptorTypeIndirect);

	AS->SetHitContributionsBuffer(HitGroupContributionsBuffer);
	
	FMetalBufferPtr IndirectArgs = AS->GetIndirectArgumentBuffer();
	MTLAccelerationStructurePtr NativeAS = AS->GetAccelerationStructure();

	IRRaytracingAccelerationStructureGPUHeader* Header = (IRRaytracingAccelerationStructureGPUHeader*)IndirectArgs->Contents();
	memset(Header, 0, sizeof(IRRaytracingAccelerationStructureGPUHeader));
	
	Header->accelerationStructureID = NativeAS->gpuResourceID()._impl;
	Header->addressOfInstanceContributions = (uint64_t)HitGroupContributionsBuffer->GetGPUAddress() + HitGroupContributionsBufferOffset;

	check(Header->addressOfInstanceContributions);
	CommandEncoder->buildAccelerationStructure(NativeAS.get(), InstanceDescriptor, ScratchBuffer->GetMTLBuffer(), ScratchBufferOffset);
}

void FMetalRHICommandContext::BuildAccelerationStructure(MTLAccelerationStructurePtr AS, MTL::AccelerationStructureDescriptor* Descriptor,
														 FMetalBufferPtr ScratchBuffer, uint32 ScratchBufferOffset)
{
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	if(!CurrentEncoder.IsAccelerationStructureCommandEncoderActive())
	{
		StateCache.ClearPreviousComputeState();
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
		}
		
		// TODO: Carl - Sample Counters
		CurrentEncoder.BeginAccelerationStructureCommandEncoding();
	}

	MTL::AccelerationStructureCommandEncoder* CommandEncoder = CurrentEncoder.GetAccelerationStructureCommandEncoder();

	CommandEncoder->buildAccelerationStructure(AS.get(), Descriptor, ScratchBuffer->GetMTLBuffer(), ScratchBufferOffset);
}

void FMetalRHICommandContext::RefitAccelerationStructure(MTLAccelerationStructurePtr SrcBLAS, MTLAccelerationStructurePtr DestBLAS, MTL::PrimitiveAccelerationStructureDescriptor* Descriptor, MTL::Buffer* ScratchBuffer, uint32 ScratchOffset)
{
	if (!CurrentEncoder.GetCommandBuffer())
	{
		StartCommandBuffer();
		check(CurrentEncoder.GetCommandBuffer());
	}
	
	if(!CurrentEncoder.IsAccelerationStructureCommandEncoderActive())
	{
		StateCache.ClearPreviousComputeState();
		if(CurrentEncoder.IsAnyCommandEncoderActive())
		{
			CurrentEncoderFence = CurrentEncoder.EndEncoding();
		}
		
		// TODO: Carl - Sample Counters
		CurrentEncoder.BeginAccelerationStructureCommandEncoding();
	}

	MTL::AccelerationStructureCommandEncoder* CommandEncoder = CurrentEncoder.GetAccelerationStructureCommandEncoder();
	CommandEncoder->refitAccelerationStructure(SrcBLAS.get(), Descriptor, DestBLAS.get(), ScratchBuffer, ScratchOffset);
}

void FMetalRayTracingScene::BuildAccelerationStructure(FMetalRHICommandContext& CommandContext,
														FMetalRHIBuffer* InScratchBuffer, uint32 ScratchOffset,
														FMetalRHIBuffer* InstanceBuffer, uint32 InstanceOffset,
													    FMetalRHIBuffer* HitGroupContributionsBuffer, uint32 HitGroupContributionOffset, 
													    uint32 NumInstances)
{
	check(InstanceBuffer != nullptr);
	check(HitGroupContributionsBuffer != nullptr);

	FMetalBufferPtr CurInstanceBuffer = InstanceBuffer->GetCurrentBuffer();
	FMetalBufferPtr CurHitGroupContributionsBuffer = HitGroupContributionsBuffer->GetCurrentBuffer();
	check(CurInstanceBuffer);

	uint32 InstanceBufferOffset = InstanceOffset + static_cast<uint32>(CurInstanceBuffer->GetOffset());

	TRefCountPtr<FMetalRHIBuffer> ScratchBuffer;
	if (InScratchBuffer == nullptr)
	{
		{
			TRHICommandList_RecursiveHazardous<FMetalRHICommandContext> RHICmdList(&CommandContext);
			
			const FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateStructured(TEXT("BuildScratchTLAS"), SizeInfo.BuildScratchSize, 0)
													.AddUsage(EBufferUsageFlags::UnorderedAccess)
													.SetInitialState(ERHIAccess::UAVCompute);
			
			ScratchBuffer = ResourceCast(RHICmdList.CreateBuffer(CreateDesc).GetReference());
			InScratchBuffer = ScratchBuffer.GetReference();
			ScratchOffset = 0;
		}
	}
	
	// Create new Indirect Args buffer
	FMetalBufferPtr IndirectArgumentBuffer = Device.GetResourceHeap().CreateBuffer(sizeof(IRRaytracingAccelerationStructureGPUHeader),
																   sizeof(IRRaytracingAccelerationStructureGPUHeader),
																   BUF_Dynamic,
																   MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared,
																   true);
	
	
	GetAccelerationStructure()->SetIndirectArgumentBuffer(IndirectArgumentBuffer);
	AccelerationStructureBuffer->UpdateLinkedViews(&CommandContext);
	
	FMetalBufferPtr CurScratchBuffer = InScratchBuffer->GetCurrentBuffer();
	check(CurScratchBuffer);

	CommandContext.BuildAccelerationStructure(CurInstanceBuffer, InstanceBufferOffset, CurScratchBuffer, CurScratchBuffer->GetOffset() + ScratchOffset,
											  CurHitGroupContributionsBuffer, HitGroupContributionOffset, NumInstances, GetAccelerationStructure());
}

void FMetalRHICommandContext::RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params)
{
	for (const FRayTracingSceneBuildParams& SceneBuildParams : Params)
	{
		FMetalRayTracingScene* const Scene = ResourceCast(SceneBuildParams.Scene);
		FMetalRHIBuffer* const ScratchBuffer = ResourceCast(SceneBuildParams.ScratchBuffer);
		FMetalRHIBuffer* const InstanceBuffer = ResourceCast(SceneBuildParams.InstanceBuffer);
		FMetalRHIBuffer* const HitGroupContributionsBuffer = ResourceCast(SceneBuildParams.HitGroupContributionsBuffer); 
		
		Scene->SetReferencedResources(SceneBuildParams.ReferencedGeometries, SceneBuildParams.ReferencedBuffers);

		Scene->BuildAccelerationStructure(
			*this,
			ScratchBuffer, SceneBuildParams.ScratchBufferOffset,
			InstanceBuffer, SceneBuildParams.InstanceBufferOffset,
			HitGroupContributionsBuffer, SceneBuildParams.HitGroupContributionsBufferOffset,
			SceneBuildParams.NumInstances);
	}
}

void FMetalRHICommandContext::RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange)
{
	checkf(ScratchBufferRange.Buffer != nullptr, TEXT("BuildAccelerationStructures requires valid scratch buffer"));

	// Update geometry vertex buffers
	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FMetalRayTracingGeometry* const Geometry = ResourceCast(P.Geometry.GetReference());

		if (P.Segments.Num())
		{
			checkf(P.Segments.Num() == Geometry->Initializer.Segments.Num(),
				TEXT("If updated segments are provided, they must exactly match existing geometry segments. Only vertex buffer bindings may change."));

			for (int32 i = 0; i < P.Segments.Num(); ++i)
			{
				checkf(P.Segments[i].VertexBuffer != nullptr, TEXT("Segments used to build/update ray tracing geometry must have a valid VertexBuffer."));

				checkf(P.Segments[i].MaxVertices <= Geometry->Initializer.Segments[i].MaxVertices,
					TEXT("Maximum number of vertices in a segment (%u) must not be smaller than what was declared during FRHIRayTracingGeometry creation (%u), as this controls BLAS memory allocation."),
					P.Segments[i].MaxVertices, Geometry->Initializer.Segments[i].MaxVertices
				);

				Geometry->Initializer.Segments[i].VertexBuffer = P.Segments[i].VertexBuffer;
				Geometry->Initializer.Segments[i].VertexBufferElementType = P.Segments[i].VertexBufferElementType;
				Geometry->Initializer.Segments[i].VertexBufferStride = P.Segments[i].VertexBufferStride;
				Geometry->Initializer.Segments[i].VertexBufferOffset = P.Segments[i].VertexBufferOffset;
			}
		}
	}

	uint32 ScratchBufferSize = ScratchBufferRange.Size ? ScratchBufferRange.Size : ScratchBufferRange.Buffer->GetSize();

	checkf(ScratchBufferSize + ScratchBufferRange.Offset <= ScratchBufferRange.Buffer->GetSize(),
		TEXT("BLAS scratch buffer range size is %lld bytes with offset %lld, but the buffer only has %d bytes. "),
		ScratchBufferRange.Size, ScratchBufferRange.Offset, ScratchBufferRange.Buffer->GetSize());

	const uint64 ScratchAlignment = GRHIRayTracingScratchBufferAlignment;
	FMetalRHIBuffer* ScratchBuffer = ResourceCast(ScratchBufferRange.Buffer);
	uint32 ScratchBufferOffset = static_cast<uint32>(ScratchBufferRange.Offset);

	struct FGeometryBuildData
	{
		FMetalRayTracingGeometry* Geometry;
		uint32 Offset;
	};
	
	TArray<FGeometryBuildData, TInlineAllocator<32>> GeometryToBuild;
	TArray<FGeometryBuildData, TInlineAllocator<32>> GeometryToRefit;
	GeometryToBuild.Reserve(Params.Num());
	GeometryToRefit.Reserve(Params.Num());
	
	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FMetalRayTracingGeometry* const Geometry = ResourceCast(P.Geometry.GetReference());
		const bool bIsUpdate = P.BuildMode == EAccelerationStructureBuildMode::Update;

		Geometry->RebuildDescriptors();
		Geometry->SizeInfo = CalcRayTracingGeometrySize(Device, Geometry->AccelerationStructureDescriptor);
		
		uint64 ScratchBufferRequiredSize = bIsUpdate ? Geometry->SizeInfo.UpdateScratchSize : Geometry->SizeInfo.BuildScratchSize;
		
		checkf(ScratchBufferRequiredSize + ScratchBufferOffset <= ScratchBufferSize,
			TEXT("BLAS scratch buffer size is %ld bytes with offset %ld (%ld bytes available), but the build requires %lld bytes. "),
			ScratchBufferSize, ScratchBufferOffset, ScratchBufferSize - ScratchBufferOffset, ScratchBufferRequiredSize);

		if (!bIsUpdate)
		{
			GeometryToBuild.Add({Geometry, ScratchBufferOffset});
		}
		else
		{
			GeometryToRefit.Add({Geometry, ScratchBufferOffset});
		}

		ScratchBufferOffset = Align(ScratchBufferOffset + ScratchBufferRequiredSize, ScratchAlignment);
	}

	FMetalBufferPtr ScratchBufferRes = ScratchBuffer->GetCurrentBuffer();
	check(ScratchBufferRes);

	for (FGeometryBuildData& BuildRequest : GeometryToBuild)
	{
		FMetalRayTracingGeometry* Geometry = BuildRequest.Geometry;
		
		BuildAccelerationStructure(Geometry->GetAccelerationStructure()->GetAccelerationStructure(), Geometry->AccelerationStructureDescriptor,
										   ScratchBufferRes, BuildRequest.Offset);
	}

	for (FGeometryBuildData& RefitRequest : GeometryToRefit)
	{
		FMetalRayTracingGeometry* Geometry = RefitRequest.Geometry;
		uint32 ScratchOffset = RefitRequest.Offset;
 
		FMetalAccelerationStructure* ReadStruct = Geometry->GetAccelerationStructure();
		
		MTLAccelerationStructurePtr SrcBLAS = ReadStruct->GetAccelerationStructure();
		MTLAccelerationStructurePtr DstBLAS = {};

		RefitAccelerationStructure(
			SrcBLAS,
			DstBLAS,
			Geometry->AccelerationStructureDescriptor,
			ScratchBufferRes->GetMTLBuffer(),
			ScratchOffset
		);
	}

	for (const FRayTracingGeometryBuildParams& P : Params)
	{
		FMetalRayTracingGeometry* const Geometry = ResourceCast(P.Geometry.GetReference());
		const bool bIsUpdate = P.BuildMode == EAccelerationStructureBuildMode::Update;

		if (!bIsUpdate)
		{
			ERayTracingAccelerationStructureFlags GeometryBuildFlags = GetRayTracingAccelerationStructureBuildFlags(Geometry->Initializer);
			if (ShouldCompactAfterBuild(GeometryBuildFlags))
			{
				Device.GetRayTracingCompactionRequestHandler()->RequestCompact(Geometry);
				Geometry->bHasPendingCompactionRequests = true;
			}
		}
		
		Geometry->SetupHitGroupSystemParameters();
	}
}

void FMetalRHICommandContext::RHIBindAccelerationStructureMemory(FRHIRayTracingScene* InScene, FRHIBuffer* InBuffer, uint32 InBufferOffset)
{
	FMetalRayTracingScene* MetalScene = ResourceCast(InScene);
	MetalScene->BindBuffer(InBuffer, InBufferOffset);
}

static void DispatchRays(FMetalRHICommandContext& Context,
						FMetalCommandEncoder& Encoder,
						FMetalDevice& Device,
						FMetalStateCache& StateCache,
						const FRayTracingShaderBindings& GlobalResourceBindings,
						FMetalRayShader* RayGenShader,
						const FMetalRayTracingPipelineState* Pipeline,
						const IRDispatchRaysDescriptor& RaysDescriptor,
						FMetalBufferPtr ArgBuffer = FMetalBufferPtr(),
						uint32 ArgumentOffset = 0)
{
	FMetalBufferPtr DispatchRayArgs = Device.GetUniformAllocator()->Allocate(sizeof(IRDispatchRaysArgument));
	check(DispatchRayArgs);
	
	IRDispatchRaysArgument* DispatchRaysArgument = (IRDispatchRaysArgument*)DispatchRayArgs->Contents();
	check(DispatchRaysArgument);
	memset(DispatchRaysArgument, 0, sizeof(IRDispatchRaysArgument));
	
	DispatchRaysArgument->DispatchRaysDesc = RaysDescriptor;
	
	uint32 IndirectDimensionSize = 0;
	uint32 IndirectArgsOffset = 0;
	
	// Upload Argument buffer if applicable
	if (ArgBuffer)
	{
		Context.BeginBlitEncoder();
		
		Encoder.PushDebugGroup(FStringToNSString(FString(TEXT("DispatchRays: Upload ArgBuffer"))));
		 
		uint32 DispatchRayDescSize = sizeof(IRDispatchRaysDescriptor);
		IndirectArgsOffset = offsetof(IRDispatchRaysDescriptor, Width);
		
		// Copy GPU computed indirect args to resource
		Context.CopyFromBufferToBuffer(ArgBuffer, ArgumentOffset, DispatchRayArgs, IndirectArgsOffset, 12);
		
		Encoder.PopDebugGroup();
		
		Context.EndBlitEncoder();
	}
	else
	{
		check(RaysDescriptor.Width != 0 && RaysDescriptor.Height != 0);
	}
	
	Context.PushDescriptorUpdates();

	Context.BeginComputeEncoder();
	
	Encoder.PushDebugGroup(FStringToNSString(FString(TEXT("DispatchRays: Compute"))));
	
	Encoder.SetComputePipelineState(Pipeline->PipelineStateObject);
	
	const uint64 GRSSize = RayGenShader->GetGlobalRootSignatureSize();
	check(GRSSize % sizeof(uint64) == 0);
	const uint64 GRSEntryCount = GRSSize / sizeof(uint64);
	
	FMetalBufferPtr GRSBuffer = Device.GetUniformAllocator()->Allocate(GRSSize);
	uint64* GRSData = (uint64*)GRSBuffer->Contents();

	// Bind GlobalResources to the Root UB.
	for (uint64 Index = 0; Index < 16; ++Index)
	{
		if (GlobalResourceBindings.UniformBuffers[Index])
		{
			FMetalUniformBuffer* UB = ResourceCast(GlobalResourceBindings.UniformBuffers[Index]);
			FMetalBufferPtr Buffer = UB->BackingBuffer;
			
			GRSData[Index+RayGenShader->GetGlobalRootSignatureStartIdx()] = Buffer->GetGPUAddress();	
			ProcessUniformBuffer(GlobalResourceBindings.UniformBuffers[Index], StateCache);
		}
	}

	// Update resource residency
	for(int i = 0; i < UE_ARRAY_COUNT(GlobalResourceBindings.Textures); i++)
	{
		if(GlobalResourceBindings.Textures[i])
		{
			FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(GlobalResourceBindings.Textures[i]);
			StateCache.IRMakeTextureResident(EMetalShaderStages::Compute, Surface->Texture.get());
		}
	}
	
	for(int i = 0; i < UE_ARRAY_COUNT(GlobalResourceBindings.SRVs); i++)
	{
		if(GlobalResourceBindings.SRVs[i])
		{
			StateCache.IRMakeSRVResident(EMetalShaderStages::Compute, static_cast<FMetalShaderResourceView*>(GlobalResourceBindings.SRVs[i]));
		}
	}
	
	for(int i = 0; i < UE_ARRAY_COUNT(GlobalResourceBindings.UAVs); i++)
	{
		if(GlobalResourceBindings.UAVs[i])
		{
			StateCache.IRMakeUAVResident(EMetalShaderStages::Compute, static_cast<FMetalUnorderedAccessView*>(GlobalResourceBindings.UAVs[i]));
		}
	}
	
	for(const FRHIShaderParameterResource& Parameter : GlobalResourceBindings.BindlessParameters) 
	{
		switch(Parameter.Type)
		{
			case FRHIShaderParameterResource::EType::ResourceView:
			{
				if (FRHIShaderResourceView* SRV = static_cast<FRHIShaderResourceView*>(Parameter.Resource))
				{
					StateCache.IRMakeSRVResident(EMetalShaderStages::Compute, static_cast<FMetalShaderResourceView*>(SRV));		
				}
				break;
			}
			case FRHIShaderParameterResource::EType::UnorderedAccessView:
			{
				if (FRHIUnorderedAccessView* UAV = static_cast<FRHIUnorderedAccessView*>(Parameter.Resource))
				{
					StateCache.IRMakeUAVResident(EMetalShaderStages::Compute, static_cast<FMetalUnorderedAccessView*>(UAV));
				}
				break;
			}
			case FRHIShaderParameterResource::EType::Sampler:
			{
				break;
			}
			case FRHIShaderParameterResource::EType::UniformBuffer:
			{
				FRHIUniformBuffer* RHIUniformBuffer = static_cast<FRHIUniformBuffer*>(Parameter.Resource);
				ProcessUniformBuffer(RHIUniformBuffer, StateCache);
				
				break;
			}
			case FRHIShaderParameterResource::EType::Texture:
			{
				if (FRHITexture* Texture = static_cast<FRHITexture*>(Parameter.Resource))
				{
					FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(Texture);
					StateCache.IRMakeTextureResident(EMetalShaderStages::Compute, Surface->Texture.get());
				}
				break;
			}
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			case FRHIShaderParameterResource::EType::ResourceCollection:
			{
				if (FMetalResourceCollection* MetalResourceCollection = ResourceCast(static_cast<FRHIResourceCollection*>(Parameter.Resource)))
				{
					if (FRHIShaderResourceView* SRV = MetalResourceCollection->GetShaderResourceView())
					{
						StateCache.IRMakeSRVResident(EMetalShaderStages::Compute, static_cast<FMetalShaderResourceView*>(SRV));
					}
				}
				break;
			}
#endif
			default:
			{
				break;
			}
		}
	}
	
	const FRHIShaderBindingLayout& ShaderBindingLayout = Context.GetShaderBindingLayout();
	const TArray<FRHIUniformBuffer*>& StaticUniformBuffers = Context.GetStaticUniformBuffers();
	
	for (uint32 Index = 0; Index < ShaderBindingLayout.GetNumUniformBufferEntries(); ++Index)
	{
		const FRHIUniformBufferShaderBindingLayout& LayoutEntry = ShaderBindingLayout.GetUniformBufferEntry(Index);
		const uint32 RootParameterSlotIndex = LayoutEntry.CBVResourceIndex;

		FRHIUniformBuffer* UniformBuffer = StaticUniformBuffers[Index];
		checkf(UniformBuffer, TEXT("Static uniform buffer at index %d is referenced in the shader binding layout but not provided in the last RHISetStaticUniformBuffers() command"), Index);

		FMetalUniformBuffer *UB = ResourceCast(UniformBuffer);
		FMetalBufferPtr BackingBuffer = UB->BackingBuffer;
		
		StateCache.CacheOrSkipResourceResidencyUpdate(BackingBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);
		ProcessUniformBuffer(UniformBuffer, StateCache);
		
		check(GRSEntryCount >= RootParameterSlotIndex + 1);
		GRSData[RootParameterSlotIndex] = BackingBuffer->GetGPUAddress();
	}

	// Static samplers always bound to the last slot
	GRSData[GRSEntryCount - 1] = Device.GetStaticSamplers()->GetGPUAddress();

	FMetalBindlessDescriptorManager* DescriptorManager = Device.GetBindlessDescriptorManager();
	
	DispatchRaysArgument->GRS = GRSBuffer->GetGPUAddress();
	
	FMetalBufferPtr StandardResourceHeap = DescriptorManager->GetStandardResourceHeap()->GetCurrentBuffer();
	FMetalBufferPtr SamplerResourceHeap = DescriptorManager->GetSamplerResourceHeap()->GetCurrentBuffer();
	
	DispatchRaysArgument->ResDescHeap = StandardResourceHeap->GetGPUAddress();
	DispatchRaysArgument->SmpDescHeap = SamplerResourceHeap->GetGPUAddress();
	DispatchRaysArgument->VisibleFunctionTable = Pipeline->VisibleFunctionTable->gpuResourceID();
	DispatchRaysArgument->IntersectionFunctionTable = Pipeline->IntersectionFunctionTable->gpuResourceID();

	TArray<MTL::Resource*> UseResources;
	UseResources.Reserve(2);
	UseResources.Add(Pipeline->VisibleFunctionTable);
	UseResources.Add(Pipeline->IntersectionFunctionTable);
	
	Encoder.UseResources(UseResources, MTL::ResourceUsageRead);
	
	StateCache.CacheOrSkipResourceResidencyUpdate(StandardResourceHeap->GetMTLBuffer(), EMetalShaderStages::Compute, true);
	StateCache.CacheOrSkipResourceResidencyUpdate(SamplerResourceHeap->GetMTLBuffer(), EMetalShaderStages::Compute, true);
	StateCache.CacheOrSkipResourceResidencyUpdate(DispatchRayArgs->GetMTLBuffer(), EMetalShaderStages::Compute, true);
	StateCache.CacheOrSkipResourceResidencyUpdate(GRSBuffer->GetMTLBuffer(), EMetalShaderStages::Compute, true);

	MTL::ComputeCommandEncoder* ComputeCommandEncoder = Encoder.GetComputeCommandEncoder();
	
	ComputeCommandEncoder->setBuffer(DispatchRayArgs->GetMTLBuffer(), DispatchRayArgs->GetOffset(), kIRRayDispatchArgumentsBindPoint);

	StateCache.FlushUsedResources<EMetalShaderStages::Compute, MTL::FunctionTypeKernel>(&Encoder);
	
	if (ArgBuffer != nullptr)
	{
		ComputeCommandEncoder->dispatchThreadgroups(ArgBuffer->GetMTLBuffer(), 
													ArgBuffer->GetOffset() + ArgumentOffset, MTL::Size::Make(32, 1, 1));
	}
	else
	{
		const uint32 Width = RaysDescriptor.Width;
		const uint32 Height = RaysDescriptor.Height;
		
		const MTL::Size ThreadgroupSize = Height > 1 ? MTL::Size::Make(8, 4, 1) : MTL::Size::Make(32, 1, 1);
		ComputeCommandEncoder->dispatchThreads(MTL::Size::Make(Width, Height, 1), ThreadgroupSize);
	}
	
	StateCache.Reset();

	Encoder.PopDebugGroup();
	Context.EndComputeEncoder();
}

void FMetalRHICommandContext::RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* InRayGenShader,
												  FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
												  uint32 Width, uint32 Height)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalRayDispatchTime);
	const FMetalRayTracingPipelineState* Pipeline = ResourceCast(RayTracingPipelineState);
	FMetalRayShader* RayGenShader = ResourceCast(InRayGenShader);
	
	check(Pipeline);
	check(RayGenShader);

	const int32 RayGenShaderIndex = Pipeline->RayGenShaders.Find(RayGenShader->GetHash());
	checkf(RayGenShaderIndex != INDEX_NONE,
		TEXT("RayGen shader '%s' is not present in the given ray tracing pipeline. ")
		TEXT("All RayGen shaders must be declared when creating RTPSO."),
			*(RayGenShader->EntryPoint));
	
	FMetalRayTracingShaderBindingTable *SBT = ResourceCast(InSBT);
	check(SBT);
	
	const FMetalShaderIdentifier& RayGenShaderIdentifier = *Pipeline->RayGenShaders.Identifiers[RayGenShaderIndex];
	IRDispatchRaysDescriptor RaysDescriptor = SBT->GetDispatchRaysDesc(RayGenShaderIdentifier);
	RaysDescriptor.Width = Width;
	RaysDescriptor.Height = Height;
	RaysDescriptor.Depth = 1;
	
	SBT->MakeResident(StateCache);
	
	DispatchRays(*this, CurrentEncoder, Device, StateCache, GlobalResourceBindings, RayGenShader, Pipeline, RaysDescriptor);
}

void FMetalRHICommandContext::RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* InRayTracingPipelineState, FRHIRayTracingShader* InRayGenShader,
														   FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
														   FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalRayDispatchTime);
	const FMetalRayTracingPipelineState* Pipeline = ResourceCast(InRayTracingPipelineState);
	FMetalRayShader* RayGenShader = ResourceCast(InRayGenShader);
	
	FMetalRHIBuffer* MetalArgumentBuffer = ResourceCast(ArgumentBuffer);
	FMetalBufferPtr ArgBuffer = MetalArgumentBuffer->GetCurrentBufferOrNull();
	
	check(Pipeline);
	check(RayGenShader);

	check(ArgBuffer);
	
	const int32 RayGenShaderIndex = Pipeline->RayGenShaders.Find(RayGenShader->GetHash());
	checkf(RayGenShaderIndex != INDEX_NONE,
		TEXT("RayGen shader '%s' is not present in the given ray tracing pipeline. ")
		TEXT("All RayGen shaders must be declared when creating RTPSO."),
			*(RayGenShader->EntryPoint));
	
	FMetalRayTracingShaderBindingTable *SBT = ResourceCast(InSBT);
	check(SBT);
	
	const FMetalShaderIdentifier& RayGenShaderIdentifier = *Pipeline->RayGenShaders.Identifiers[RayGenShaderIndex];
	IRDispatchRaysDescriptor RaysDescriptor = SBT->GetDispatchRaysDesc(RayGenShaderIdentifier);
	
	SBT->MakeResident(StateCache);
	
	DispatchRays(*this, CurrentEncoder, Device, StateCache, GlobalResourceBindings, RayGenShader, Pipeline, RaysDescriptor, ArgBuffer, ArgumentOffset);
}

FRayTracingSceneRHIRef FMetalDynamicRHI::RHICreateRayTracingScene(FRayTracingSceneInitializer Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    return new FMetalRayTracingScene(*Device, MoveTemp(Initializer));
}

FRayTracingGeometryRHIRef FMetalDynamicRHI::RHICreateRayTracingGeometry(FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    return new FMetalRayTracingGeometry(*Device, RHICmdList, Initializer);
}

FRayTracingPipelineStateRHIRef FMetalDynamicRHI::RHICreateRayTracingPipelineState(const FRayTracingPipelineStateInitializer& Initializer)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	return new FMetalRayTracingPipelineState(*Device, Initializer);
}

FShaderBindingTableRHIRef FMetalDynamicRHI::RHICreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FRayTracingShaderBindingTableInitializer& Initializer)
{
	return new FMetalRayTracingShaderBindingTable(RHICmdList, Initializer, *Device);
}

void FMetalDevice::InitializeRayTracing()
{
	// Explicitly request a pointer to the DeviceContext since the CompactionHandler
	// is initialized before the global getter is setup.
	RayTracingCompactionRequestHandler = new FMetalRayTracingCompactionRequestHandler(*this);
	DummyIndexBuffer = GetResourceHeap().CreateBuffer(64, 16, BUF_Static, MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared, true);
	
	RayTracingPipelineCache = new FMetalRayTracingPipelineCache();
}

void FMetalDevice::UpdateRayTracing(FMetalRHICommandContext& Context)
{
	RayTracingCompactionRequestHandler->Update(Context);
}

void FMetalDevice::CleanUpRayTracing()
{
	delete RayTracingCompactionRequestHandler;
	delete RayTracingPipelineCache;
}
#endif // METAL_RHI_RAYTRACING
