// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MetalRayTracing.h: MetalRT Implementation
==============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "MetalResources.h"
#include "RayTracingBuiltInResources.h"

#if METAL_RHI_RAYTRACING

struct FMetalHitGroupSystemParameters
{	
	FHitGroupSystemRootConstants RootConstants;
	
	uint32 BindlessHitGroupSystemIndexBuffer;
	uint32 BindlessHitGroupSystemVertexBuffer;
};

class FMetalRayTracingGeometry : public FRHIRayTracingGeometry
{
public:
	FMetalRayTracingGeometry(FMetalDevice& InDevice, FRHICommandListBase& RHICmdList, const FRayTracingGeometryInitializer& InInitializer);
	~FMetalRayTracingGeometry();

	void ReleaseUnderlyingResource();
	void ReleaseDescriptors();

	uint64 GetGPUAddress() const;
	
	/** FRHIRayTracingGeometry Interface */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FRayTracingAccelerationStructureAddress GetAccelerationStructureAddress(uint64 GPUIndex) const final override { return GetGPUAddress(); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	/** FRHIRayTracingGeometry Interface */

	void Swap(FMetalRayTracingGeometry& Other);
	void RebuildDescriptors();

	void RemoveCompactionRequest();

	using FRHIRayTracingGeometry::Initializer;
	using FRHIRayTracingGeometry::SizeInfo;

	MTL::PrimitiveAccelerationStructureDescriptor* AccelerationStructureDescriptor = nullptr;

	bool bHasPendingCompactionRequests;
	uint32 CompactionSizeIndex;

	static constexpr uint32 IndicesPerPrimitive = 3; // Triangle geometry only

	inline FMetalAccelerationStructure* GetAccelerationStructure() const
	{
		return AccelerationStructure;
	}
	
	void SetAccelerationStructure(FMetalAccelerationStructure* InBuffer);
	void SetupHitGroupSystemParameters();
	void ReleaseBindlessHandles();
	
	TArray<FMetalHitGroupSystemParameters> HitGroupSystemParameters;
	
private:
	FMetalDevice& Device;
	
	TArray<MTL::AccelerationStructureTriangleGeometryDescriptor*> GeomArray;

	uint32 AccelerationStructureIndex;
	FMetalAccelerationStructure* AccelerationStructure = nullptr;
	
	TArray<FRHIDescriptorHandle> HitGroupSystemVertexViews;
	FRHIDescriptorHandle HitGroupSystemIndexView;
	
	NS::Array* GeometryDescriptors =  nullptr;
};

class FMetalRayTracingScene : public FRHIRayTracingScene
{
public:
	FMetalRayTracingScene(FMetalDevice& InDevice, FRayTracingSceneInitializer InInitializer);
	virtual ~FMetalRayTracingScene();

	void BindBuffer(FRHIBuffer* InBuffer, uint32 InBufferOffset);
	void BuildAccelerationStructure(
		FMetalRHICommandContext& CommandContext,
		FMetalRHIBuffer* ScratchBuffer, uint32 ScratchOffset,
		FMetalRHIBuffer* InstanceBuffer, uint32 InstanceOffset,
		FMetalRHIBuffer* HitGroupContributionsBuffer, uint32 HitGroupContributionsBufferOffset, 
		uint32 NumInstances);

	inline const FRayTracingSceneInitializer& GetInitializer() const override final { return Initializer; }

	using FRHIRayTracingAccelerationStructure::SizeInfo;
	using FRHIRayTracingScene::ReferencedGeometries;
	using FRHIRayTracingScene::ReferencedBuffers;
	using FRHIRayTracingScene::SetReferencedResources;

	FMetalAccelerationStructure* GetAccelerationStructure()
	{
		check(AccelerationStructureBuffer && AccelerationStructureBuffer->AccelerationStructure);
		return AccelerationStructureBuffer->AccelerationStructure;
	}
	
private:
	friend class FMetalRHICommandContext;

private:
	FMetalDevice& Device;
	
	/** The initializer provided to build the scene. Contains all the free standing stuff that used to be owned by the RT implementation. */
	const FRayTracingSceneInitializer Initializer;

	/** Acceleration Structure for the whole scene. */
	FMetalRHIBuffer* AccelerationStructureBuffer; 
};
#endif // METAL_RHI_RAYTRACING
