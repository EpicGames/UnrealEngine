// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRenderTarget.cpp: Metal render target implementation.
=============================================================================*/

#include "MetalRHIPrivate.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "ScreenRendering.h"
#include "MetalCommandBuffer.h"
#include "MetalCommandQueue.h"
#include "MetalDynamicRHI.h"
#include "MetalProfiler.h"
#include "ResolveShader.h"
#include "PipelineStateCache.h"
#include "Math/PackedVector.h"
#include "RHISurfaceDataConversion.h"

static FResolveRect GetDefaultRect(const FResolveRect& Rect, uint32 DefaultWidth, uint32 DefaultHeight)
{
	if (Rect.X1 >= 0 && Rect.X2 >= 0 && Rect.Y1 >= 0 && Rect.Y2 >= 0)
	{
		return Rect;
	}
	else
	{
		return FResolveRect(0, 0, DefaultWidth, DefaultHeight);
	}
}

int32 GMetalUseTexGetBytes = 1;
static FAutoConsoleVariableRef CVarMetalUseTexGetBytes(
								TEXT("rhi.Metal.UseTexGetBytes"),
								GMetalUseTexGetBytes,
								TEXT("If true prefer using -[MTLTexture getBytes:...] to retreive texture data, creating a temporary shared/managed texture to copy from private texture storage when required, rather than using a temporary MTLBuffer. This works around data alignment bugs on some GPU vendor's drivers and may be more appropriate on iOS. (Default: True)"),
								ECVF_RenderThreadSafe
								);

/** Helper for accessing R10G10B10A2 colors. */
struct FMetalR10G10B10A2
{
	uint32 R : 10;
	uint32 G : 10;
	uint32 B : 10;
	uint32 A : 2;
};

/** Helper for accessing R16G16 colors. */
struct FMetalRG16
{
	uint16 R;
	uint16 G;
};

/** Helper for accessing R16G16B16A16 colors. */
struct FMetalRGBA16
{
	uint16 R;
	uint16 G;
	uint16 B;
	uint16 A;
};

template <typename...> constexpr std::false_type AlwaysFalse;

template<typename ColorType>
void ReadSurfaceDataInternal(FMetalDevice* Device, FRHITexture* TextureRHI, FIntRect Rect, TArray<ColorType>& OutData, FReadSurfaceDataFlags InFlags)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	// allocate output space
	const uint32 SizeX = Rect.Width();
	const uint32 SizeY = Rect.Height();
	OutData.SetNumUninitialized(SizeX * SizeY);

	if (!ensure(TextureRHI))
	{
		FMemory::Memzero(OutData.GetData(), sizeof(FColor) * OutData.Num());
		return;
	}

	FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);

	ColorType* OutDataPtr = OutData.GetData();
	MTL::Region Region(Rect.Min.X, Rect.Min.Y, SizeX, SizeY);
	
	MTLTexturePtr Texture = Surface->Texture;
	if(!Texture && EnumHasAnyFlags(Surface->GetDesc().Flags, TexCreate_Presentable))
	{
		Texture = Surface->GetCurrentTexture();
	}
	
	if(!Texture)
	{
		UE_LOGF(LogRHI, Error, "Trying to read from an uninitialised texture.");
		return;
	}

	if (GMetalUseTexGetBytes && Surface->GetDesc().Format != PF_DepthStencil && Surface->GetDesc().Format != PF_ShadowDepth)
	{
		SCOPE_CYCLE_COUNTER(STAT_MetalTexturePageOffTime);
		
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		MTLTexturePtr TempTexture;
		
		RHICmdList.EnqueueLambda([Device, &Texture, &TempTexture, SizeX, SizeY, &Region, Surface, InFlags](FRHICommandListImmediate& RHICmdList)
		{
			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
			
			if (Texture->storageMode() == MTL::StorageModePrivate)
			{
#if PLATFORM_MAC
				MTL::StorageMode StorageMode = MTL::StorageModeManaged;
#else
#if WITH_IOS_SIMULATOR
				MTL::StorageMode StorageMode = MTL::StorageModePrivate;
#else
				MTL::StorageMode StorageMode = MTL::StorageModeShared;
#endif
#endif
				MTL::PixelFormat MetalFormat = (MTL::PixelFormat)GPixelFormats[Surface->GetDesc().Format].PlatformFormat;
				MTL::TextureDescriptor* Desc = MTL::TextureDescriptor::alloc()->init();
				check(Desc);
				
				Desc->setTextureType(Texture->textureType());
				Desc->setPixelFormat(Texture->pixelFormat());
				Desc->setWidth(SizeX);
				Desc->setHeight(SizeY);
				Desc->setDepth(1);
				Desc->setMipmapLevelCount(1); // Only consider a single subresource and not the whole texture (like in the other RHIs)
				Desc->setSampleCount(Texture->sampleCount());
				Desc->setArrayLength(Texture->arrayLength());
				
				MTL::ResourceOptions GeneralResourceOption = (MTL::ResourceOptions)FMetalCommandQueue::GetCompatibleResourceOptions(MTL::ResourceOptions(((NS::UInteger)Texture->cpuCacheMode() << MTL::ResourceCpuCacheModeShift) | ((NS::UInteger)StorageMode << MTL::ResourceStorageModeShift) | MTL::ResourceHazardTrackingModeUntracked));
				Desc->setResourceOptions(GeneralResourceOption);
				
				Desc->setCpuCacheMode(Texture->cpuCacheMode());
				Desc->setStorageMode(StorageMode);
				Desc->setUsage(Texture->usage());
				
				TempTexture = NS::TransferPtr(Device->GetDevice()->newTexture(Desc));
				Desc->release();
				
				Context.CopyFromTextureToTexture(Texture.get(), 0, InFlags.GetMip(), MTL::Origin(Region.origin), MTL::Size(Region.size), TempTexture.get(), 0, 0, MTL::Origin(0, 0, 0));
				
				Texture = TempTexture;
				Region = MTL::Region(0, 0, SizeX, SizeY);
			}
#if PLATFORM_MAC
			if(Texture->storageMode() == MTL::StorageModeManaged)
			{
				// Synchronise the texture with the CPU
				Context.SynchronizeTexture(Texture.get(), 0, InFlags.GetMip());
			}
#endif
		});

		//kick the current command buffer.
		RHICmdList.SubmitAndBlockUntilGPUIdle();
		
		const uint32 Stride = GPixelFormats[Surface->GetDesc().Format].BlockBytes * SizeX;
		const uint32 BytesPerImage = Stride * SizeY;

		TArray<uint8> Data;
		Data.AddUninitialized(BytesPerImage);
		
		Texture->getBytes(Data.GetData(), Stride, BytesPerImage, Region, 0, 0);
		
		if constexpr (std::is_same_v<ColorType, FColor>)
		{
			ConvertRAWSurfaceDataToFColor(TextureRHI->GetFormat(), SizeX, SizeY, (uint8*)Data.GetData(), Stride, OutDataPtr, InFlags);
		}
		else if constexpr (std::is_same_v<ColorType, FLinearColor>)
		{
			ConvertRAWSurfaceDataToFLinearColor(TextureRHI->GetFormat(), SizeX, SizeY, (uint8*)Data.GetData(), Stride, OutDataPtr, InFlags);
		}
		else
		{
			// Color type not recognized.
			static_assert(AlwaysFalse<ColorType>);
		}

		if (TempTexture)
		{
			FMetalDynamicRHI::Get().DeferredDelete(TempTexture);
		}
	}
	else
	{
		uint32 BytesPerPixel = (Surface->GetDesc().Format != PF_DepthStencil || !InFlags.GetOutputStencil()) ? GPixelFormats[Surface->GetDesc().Format].BlockBytes : 1;
		const uint32 Stride = BytesPerPixel * SizeX;
		const uint32 Alignment = PLATFORM_MAC ? 1u : 64u; // Mac permits natural row alignment (tightly-packed) but iOS does not.
		const uint32 AlignedStride = ((Stride - 1) & ~(Alignment - 1)) + Alignment;
		const uint32 BytesPerImage = AlignedStride * SizeY;
		
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		
		FMetalBufferPtr Buffer = Device->CreatePooledBuffer(FMetalPooledBufferArgs(Device, BytesPerImage, BUF_Dynamic, MTL::StorageModeShared));
		
		RHICmdList.EnqueueLambda([Device, &Buffer, &Texture, &Region, Surface, AlignedStride, BytesPerImage, InFlags](FRHICommandListImmediate& RHICmdList)
		{
			// Synchronise the texture with the CPU
			SCOPE_CYCLE_COUNTER(STAT_MetalTexturePageOffTime);
			
			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
			
			if (Surface->GetDesc().Format != PF_DepthStencil)
			{
				Context.CopyFromTextureToBuffer(Texture.get(), 0, InFlags.GetMip(), Region.origin, Region.size, Buffer, 0, AlignedStride, BytesPerImage, MTL::BlitOptionNone);
			}
			else
			{
				if (!InFlags.GetOutputStencil())
				{
					Context.CopyFromTextureToBuffer(Texture.get(), 0, InFlags.GetMip(), Region.origin, Region.size, Buffer, 0, AlignedStride, BytesPerImage, MTL::BlitOptionDepthFromDepthStencil);
				}
				else
				{
					Context.CopyFromTextureToBuffer(Texture.get(), 0, InFlags.GetMip(), Region.origin, Region.size, Buffer, 0, AlignedStride, BytesPerImage, MTL::BlitOptionStencilFromDepthStencil);
				}
			}
		});
		
		//kick the current command buffer.
		RHICmdList.SubmitAndBlockUntilGPUIdle();
		
		if constexpr (std::is_same_v<ColorType, FColor>)
		{
			ConvertRAWSurfaceDataToFColor(TextureRHI->GetFormat(), SizeX, SizeY, (uint8*)Buffer->Contents(), AlignedStride, OutDataPtr, InFlags);
		}
		else if constexpr (std::is_same_v<ColorType, FLinearColor>)
		{
			ConvertRAWSurfaceDataToFLinearColor(TextureRHI->GetFormat(), SizeX, SizeY, (uint8*)Buffer->Contents(), AlignedStride, OutDataPtr, InFlags);
		}
		else
		{
			// Color type not recognized.
			static_assert(AlwaysFalse<ColorType>);
		}
		
		FMetalDynamicRHI::Get().DeferredDelete(Buffer);
	}
}

void FMetalDynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect InRect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	ReadSurfaceDataInternal(Device, TextureRHI, InRect, OutData, InFlags);
}

void FMetalDynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect Rect, TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	ReadSurfaceDataInternal(Device, TextureRHI, Rect, OutData, InFlags);
}

#if PLATFORM_IOS
bool FMetalDynamicRHI::RHIReadViewportDataDirect(uint32& OutWidth, uint32& OutHeight, TArray<FColor>& OutData)
{
	check(IsInGameThread());

	FMetalViewport* Viewport = nullptr;
	{
		FScopeLock Lock(&FMetalViewport::ViewportsMutex);
		for (FMetalViewport* ViewportPtr : FMetalViewport::Viewports)
		{
			if (ViewportPtr)
			{
				Viewport = ViewportPtr;
				break;
			}
		}
	}

	if (!Viewport)
	{
		// No viewport available to capture from.
		return false;
	}

	FMetalSurface* BackBuffer = Viewport->GetBackBuffer(EMetalViewportAccessGame, {});
	if (!BackBuffer)
	{
		// Could not retrieve the back buffer to capture from.
		return false;
	}

	// allocate output space
	OutWidth = BackBuffer->GetDesc().Extent.X;
	OutHeight = BackBuffer->GetDesc().Extent.Y;
	OutData.SetNumUninitialized(OutWidth * OutHeight);

	MTLTexturePtr Texture = BackBuffer->Texture;
    if(!Texture && EnumHasAnyFlags(BackBuffer->GetDesc().Flags, TexCreate_Presentable))
    {
        Texture = BackBuffer->GetCurrentTexture();
    }
	
    if(!Texture)
    {
		OutData.Empty();

		UE_LOGF(LogRHI, Error, "Trying to read from an uninitialised texture.");
        return false;
    }

	const uint32 Stride = GPixelFormats[BackBuffer->GetDesc().Format].BlockBytes * OutWidth;
	const uint32 BytesPerImage = Stride * OutHeight;

	MTL::Region Region(0, 0, OutWidth, OutHeight);
	Texture->getBytes(OutData.GetData(), Stride, BytesPerImage, Region, 0, 0);

	return true;
}
#endif // PLATFORM_IOS

void FMetalDynamicRHI::RHIMapStagingSurface(FRHITexture* TextureRHI, FRHIGPUFence* FenceRHI, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.SubmitAndBlockUntilGPUIdle();
	
	if (FenceRHI && !FenceRHI->Poll())
	{
		ResourceCast(FenceRHI)->Wait(RHICmdList, FRHIGPUMask::FromIndex(GPUIndex));
	}
	
	FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);

	FRHILockTextureResult Result = Surface->Lock(FRHILockTextureArgs::Lock2D(Surface, 0, RLM_ReadOnly, false), false);

	OutWidth = Surface->GetSizeX();
	OutHeight = Surface->GetSizeY();
	OutData = Result.Data;
}

void FMetalDynamicRHI::RHIUnmapStagingSurface(FRHITexture* TextureRHI, uint32 GPUIndex)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);
    Surface->Unlock(FRHILockTextureArgs::Lock2D(Surface, 0, RLM_ReadOnly, false));
}

void FMetalDynamicRHI::RHIReadSurfaceFloatData(FRHITexture* TextureRHI, FIntRect Rect, TArray<FFloat16Color>& OutData, ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);
	
    MTLTexturePtr Texture = Surface->Texture;
    if(!Texture && EnumHasAnyFlags(Surface->GetDesc().Flags, TexCreate_Presentable))
    {
		Texture = Surface->GetCurrentTexture();
    }
    if(!Texture)
    {
        UE_LOGF(LogRHI, Error, "Trying to read from an uninitialised texture.");
        return;
    }
    
	// verify the input image format (but don't crash)
	if (Surface->GetDesc().Format != PF_FloatRGBA)
	{
		UE_LOGF(LogRHI, Log, "Trying to read non-FloatRGBA surface.");
	}

	if (TextureRHI->GetTextureCube())
	{
		// adjust index to account for cubemaps as texture arrays
		ArrayIndex *= CubeFace_MAX;
		ArrayIndex += GetMetalCubeFace(CubeFace);
	}
	
	// allocate output space
	const uint32 SizeX = Rect.Width();
	const uint32 SizeY = Rect.Height();
	OutData.SetNumUninitialized(SizeX * SizeY);
	
	MTL::Region Region = MTL::Region(Rect.Min.X, Rect.Min.Y, SizeX, SizeY);
	
	// function wants details about the destination, not the source
	const uint32 Stride = GPixelFormats[Surface->GetDesc().Format].BlockBytes * SizeX;
	const uint32 Alignment = PLATFORM_MAC ? 1u : 64u; // Mac permits natural row alignment (tightly-packed) but iOS does not.
	const uint32 AlignedStride = ((Stride - 1) & ~(Alignment - 1)) + Alignment;
	const uint32 BytesPerImage = AlignedStride  * SizeY;
	int32 FloatBGRADataSize = BytesPerImage;
	FMetalBufferPtr Buffer = Device->CreatePooledBuffer(FMetalPooledBufferArgs(Device, FloatBGRADataSize, BUF_Dynamic, MTL::StorageModeShared));
	{
		// Synchronise the texture with the CPU
		SCOPE_CYCLE_COUNTER(STAT_MetalTexturePageOffTime);
		
		{
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

			// Enqueue an RHI thread command to fully flush the GPU and write back caches
			RHICmdList.EnqueueLambda([&Texture, ArrayIndex, MipIndex, Region, &Buffer, AlignedStride, BytesPerImage](FRHICommandListImmediate& RHICmdList)
			{
				FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
				
				Context.CopyFromTextureToBuffer(Texture.get(), ArrayIndex, MipIndex, Region.origin, Region.size, Buffer, 0, AlignedStride, BytesPerImage, MTL::BlitOptionNone);
			});

			RHICmdList.SubmitAndBlockUntilGPUIdle();
		}
	}
	
	uint8* DataPtr = (uint8*)Buffer->Contents();
	FFloat16Color* OutDataPtr = OutData.GetData();
	if (Alignment > 1u)
	{
		for (uint32 Row = 0; Row < SizeY; Row++)
		{
			FFloat16Color* FloatBGRAData = (FFloat16Color*)DataPtr;
			FMemory::Memcpy(OutDataPtr, FloatBGRAData, Stride);
			DataPtr += AlignedStride;
			OutDataPtr += SizeX;
		}
	}
	else
	{
		FFloat16Color* FloatBGRAData = (FFloat16Color*)DataPtr;
		FMemory::Memcpy(OutDataPtr, FloatBGRAData, FloatBGRADataSize);
	}
	
	FMetalDynamicRHI::Get().DeferredDelete(Buffer);
}

void FMetalDynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* TextureRHI,FIntRect InRect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData)
{
    MTL_SCOPED_AUTORELEASE_POOL;

	FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);
	
	MTL::Texture* Texture = Surface->Texture.get();
	if(!Texture)
	{
		UE_LOGF(LogRHI, Error, "Trying to read from an uninitialised texture.");
		return;
	}
	
	// verify the input image format (but don't crash)
	if (Surface->GetDesc().Format != PF_FloatRGBA)
	{
		UE_LOGF(LogRHI, Log, "Trying to read non-FloatRGBA surface.");
	}
	
	// allocate output space
	const uint32 SizeX = InRect.Width();
	const uint32 SizeY = InRect.Height();
	const uint32 SizeZ = ZMinMax.Y - ZMinMax.X;
	OutData.SetNumUninitialized(SizeX * SizeY * SizeZ);
	
	MTL::Region Region = MTL::Region(InRect.Min.X, InRect.Min.Y, ZMinMax.X, SizeX, SizeY, SizeZ);
	
	// function wants details about the destination, not the source
	const uint32 Stride = GPixelFormats[Surface->GetDesc().Format].BlockBytes * SizeX;
	const uint32 Alignment = PLATFORM_MAC ? 1u : 64u; // Mac permits natural row alignment (tightly-packed) but iOS does not.
	const uint32 AlignedStride = ((Stride - 1) & ~(Alignment - 1)) + Alignment;
	const uint32 BytesPerImage = AlignedStride  * SizeY;
	int32 FloatBGRADataSize = BytesPerImage * SizeZ;
	FMetalBufferPtr Buffer = Device->CreatePooledBuffer(FMetalPooledBufferArgs(Device, FloatBGRADataSize, BUF_Dynamic, MTL::StorageModeShared));
	
	{
		// Synchronise the texture with the CPU
		SCOPE_CYCLE_COUNTER(STAT_MetalTexturePageOffTime);
		
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		
		RHICmdList.EnqueueLambda([this, &Buffer, &Texture, &Region, Surface, AlignedStride, BytesPerImage](FRHICommandListImmediate& RHICmdList)
		{
			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
			Context.CopyFromTextureToBuffer(Texture, 0, 0, Region.origin, Region.size, Buffer, 0, AlignedStride, BytesPerImage, MTL::BlitOptionNone);
		});
		
		//kick the current command buffer.
		RHICmdList.SubmitAndBlockUntilGPUIdle();
	}
	
	uint8* DataPtr = (uint8*)Buffer->Contents();
	FFloat16Color* OutDataPtr = OutData.GetData();
	if (Alignment > 1u)
	{
		for (uint32 Image = 0; Image < SizeZ; Image++)
		{
			for (uint32 Row = 0; Row < SizeY; Row++)
			{
				FFloat16Color* FloatBGRAData = (FFloat16Color*)DataPtr;
				FMemory::Memcpy(OutDataPtr, FloatBGRAData, Stride);
				DataPtr += AlignedStride;
				OutDataPtr += SizeX;
			}
		}
	}
	else
	{
		FFloat16Color* FloatBGRAData = (FFloat16Color*)DataPtr;
		FMemory::Memcpy(OutDataPtr, FloatBGRAData, FloatBGRADataSize);
	}
	
	FMetalDynamicRHI::Get().DeferredDelete(Buffer);
}
