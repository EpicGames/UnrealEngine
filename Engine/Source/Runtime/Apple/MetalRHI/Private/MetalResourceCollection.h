// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICoreResourceCollection.h"
#include "MetalRHIPrivate.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

using FMetalResourceCollection = UE::RHICore::FGenericResourceCollection;

template<>
struct TMetalResourceTraits<FRHIResourceCollection>
{
	using TConcreteType = FMetalResourceCollection;
};

#endif
