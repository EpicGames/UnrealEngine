// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderTypes.h: Metal RHI Shader Types.
=============================================================================*/

#pragma once

#include "Shaders/Types/Templates/MetalBaseShader.h"
#include "Shaders/Types/MetalVertexShader.h"
#include "Shaders/Types/MetalGeometryShader.h"
#include "Shaders/Types/MetalPixelShader.h"
#include "Shaders/Types/MetalComputeShader.h"
#include "Shaders/Types/MetalMeshShader.h"
#include "Shaders/Types/MetalAmplificationShader.h"
#include "Shaders/Types/MetalRayShader.h"
