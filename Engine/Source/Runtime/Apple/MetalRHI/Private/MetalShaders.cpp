// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaders.cpp: Metal shader RHI implementation.
=============================================================================*/

#include "MetalRHIPrivate.h"

#include "Shaders/Debugging/MetalShaderDebugCache.h"
#include "Shaders/MetalCompiledShaderKey.h"
#include "Shaders/MetalCompiledShaderCache.h"
#include "Shaders/MetalShaderLibrary.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "MetalShaderResources.h"
#include "MetalResources.h"
#include "MetalProfiler.h"
#include "MetalCommandBuffer.h"
#include "Serialization/MemoryReader.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/Compression.h"
#include "Misc/MessageDialog.h"

#define SHADERCOMPILERCOMMON_API
#	include "Developer/ShaderCompilerCommon/Public/ShaderCompilerCommon.h"
#undef SHADERCOMPILERCOMMON_API

MTL::LanguageVersion ValidateVersion(uint32 Version)
{
    MTL::LanguageVersion Result = MTL::LanguageVersion2_4;
#if PLATFORM_MAC
    Result = MTL::LanguageVersion2_4;
    switch(Version)
    {
        case 8:
            Result = MTL::LanguageVersion3_0;
            break;
        case 7:
            Result = MTL::LanguageVersion2_4;
            break;
		 case 6:
            Result = MTL::LanguageVersion2_3;
            break;
		case 5:
			// Fall through
        case 0:
            Version = 7;
            Result = MTL::LanguageVersion2_2; // minimum version as of UE5.1
            break;
        default:
            //EMacMetalShaderStandard::MacMetalSLStandard_Minimum is currently 2.2
            UE_LOGF(LogTemp, Warning, "The Metal version currently set is not supported anymore. Set it in the Project Settings. Defaulting to the minimum version.");
            Version = 5;
            Result = MTL::LanguageVersion2_2;
            break;
    }
#else
    Result = MTL::LanguageVersion2_4;
    switch(Version)
    {
        case 7:
            Result = MTL::LanguageVersion2_4;
            break;
        case 8:
            Result = MTL::LanguageVersion3_0;
            break;
        case 9:
            Result = MTL::LanguageVersion3_1;
            break;
        case 0:
            Version = 8;
            Result = MTL::LanguageVersion2_4; // minimum version as of UE5.3
            break;
        default:
            //EIOSMetalShaderStandard::IOSMetalSLStandard_Minimum is currently 2.4
            UE_LOGF(LogTemp, Warning, "The Metal version currently set is not supported anymore. Set it in the Project Settings. Defaulting to the minimum version.");
            Version = 7;
            Result = MTL::LanguageVersion2_4;
            break;
    }
#endif
	return Result;
}
