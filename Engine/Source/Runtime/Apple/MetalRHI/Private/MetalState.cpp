// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalState.cpp: Metal state implementation.
=============================================================================*/

#include "MetalState.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalDynamicRHI.h"
#include "MetalRHIPrivate.h"
#include "MetalProfiler.h"
#include "RHIUtilities.h"

#include "MetalBindlessDescriptors.h"

static uint32 GetMetalMaxAnisotropy(ESamplerFilter Filter, uint32 MaxAniso)
{
	switch (Filter)
	{
		case SF_AnisotropicPoint:
		case SF_AnisotropicLinear:	return ComputeAnisotropyRT(MaxAniso);
		default:					return 1;
	}
}

static MTL::SamplerMinMagFilter TranslateZFilterMode(ESamplerFilter Filter)
{	switch (Filter)
	{
		case SF_Point:				return MTL::SamplerMinMagFilterNearest;
		case SF_AnisotropicPoint:	return MTL::SamplerMinMagFilterNearest;
		case SF_AnisotropicLinear:	return MTL::SamplerMinMagFilterLinear;
		default:					return MTL::SamplerMinMagFilterLinear;
	}
}

static MTL::SamplerAddressMode TranslateWrapMode(ESamplerAddressMode AddressMode)
{
	switch (AddressMode)
	{
		case AM_Clamp:	return MTL::SamplerAddressModeClampToEdge;
		case AM_Mirror: return MTL::SamplerAddressModeMirrorRepeat;
#if PLATFORM_MAC
		case AM_Border: return MTL::SamplerAddressModeClampToBorderColor;
#else
		case AM_Border: return MTL::SamplerAddressModeClampToEdge;
#endif
		default:		return MTL::SamplerAddressModeRepeat;
	}
}

static MTL::CompareFunction TranslateCompareFunction(ECompareFunction CompareFunction)
{
	switch(CompareFunction)
	{
		case CF_Less:			return MTL::CompareFunctionLess;
		case CF_LessEqual:		return MTL::CompareFunctionLessEqual;
		case CF_Greater:		return MTL::CompareFunctionGreater;
		case CF_GreaterEqual:	return MTL::CompareFunctionGreaterEqual;
		case CF_Equal:			return MTL::CompareFunctionEqual;
		case CF_NotEqual:		return MTL::CompareFunctionNotEqual;
		case CF_Never:			return MTL::CompareFunctionNever;
		default:				return MTL::CompareFunctionAlways;
	};
}

static MTL::CompareFunction TranslateSamplerCompareFunction(ESamplerCompareFunction SamplerComparisonFunction)
{
	switch(SamplerComparisonFunction)
	{
		case SCF_Less:		return MTL::CompareFunctionLess;
		case SCF_Never:		return MTL::CompareFunctionNever;
		default:			return MTL::CompareFunctionNever;
	};
}

static MTL::StencilOperation TranslateStencilOp(EStencilOp StencilOp)
{
	switch(StencilOp)
	{
		case SO_Zero:				return MTL::StencilOperationZero;
		case SO_Replace:			return MTL::StencilOperationReplace;
		case SO_SaturatedIncrement:	return MTL::StencilOperationIncrementClamp;
		case SO_SaturatedDecrement:	return MTL::StencilOperationDecrementClamp;
		case SO_Invert:				return MTL::StencilOperationInvert;
		case SO_Increment:			return MTL::StencilOperationIncrementWrap;
		case SO_Decrement:			return MTL::StencilOperationDecrementWrap;
		default:					return MTL::StencilOperationKeep;
	};
}

static MTL::BlendOperation TranslateBlendOp(EBlendOperation BlendOp)
{
	switch(BlendOp)
	{
		case BO_Subtract:			return MTL::BlendOperationSubtract;
		case BO_Min:				return MTL::BlendOperationMin;
		case BO_Max:				return MTL::BlendOperationMax;
		case BO_ReverseSubtract:	return MTL::BlendOperationReverseSubtract;
		default:					return MTL::BlendOperationAdd;
	};
}


static MTL::BlendFactor TranslateBlendFactor(EBlendFactor BlendFactor)
{
	switch(BlendFactor)
	{
		case BF_One:					return MTL::BlendFactorOne;
		case BF_SourceColor:			return MTL::BlendFactorSourceColor;
		case BF_InverseSourceColor:		return MTL::BlendFactorOneMinusSourceColor;
		case BF_SourceAlpha:			return MTL::BlendFactorSourceAlpha;
		case BF_InverseSourceAlpha:		return MTL::BlendFactorOneMinusSourceAlpha;
		case BF_DestAlpha:				return MTL::BlendFactorDestinationAlpha;
		case BF_InverseDestAlpha:		return MTL::BlendFactorOneMinusDestinationAlpha;
		case BF_DestColor:				return MTL::BlendFactorDestinationColor;
		case BF_InverseDestColor:		return MTL::BlendFactorOneMinusDestinationColor;
		case BF_Source1Color:			return MTL::BlendFactorSource1Color;
		case BF_InverseSource1Color:	return MTL::BlendFactorOneMinusSource1Color;
		case BF_Source1Alpha:			return MTL::BlendFactorSource1Alpha;
		case BF_InverseSource1Alpha:	return MTL::BlendFactorOneMinusSource1Alpha;
		default:						return MTL::BlendFactorZero;
	};
}

static MTL::ColorWriteMask TranslateWriteMask(EColorWriteMask WriteMask)
{
	uint32 Result = 0;
	Result |= (WriteMask & CW_RED) ? (MTL::ColorWriteMaskRed) : 0;
	Result |= (WriteMask & CW_GREEN) ? (MTL::ColorWriteMaskGreen) : 0;
	Result |= (WriteMask & CW_BLUE) ? (MTL::ColorWriteMaskBlue) : 0;
	Result |= (WriteMask & CW_ALPHA) ? (MTL::ColorWriteMaskAlpha) : 0;
	
	return (MTL::ColorWriteMask)Result;
}

static EBlendOperation TranslateBlendOp(MTL::BlendOperation BlendOp)
{
	switch(BlendOp)
	{
        case MTL::BlendOperationSubtract:			return BO_Subtract;
        case MTL::BlendOperationMin:				return BO_Min;
        case MTL::BlendOperationMax:				return BO_Max;
		case MTL::BlendOperationReverseSubtract: 	return BO_ReverseSubtract;
        case MTL::BlendOperationAdd: default:		return BO_Add;
	};
}


static EBlendFactor TranslateBlendFactor(MTL::BlendFactor BlendFactor)
{
	switch(BlendFactor)
	{
        case MTL::BlendFactorOne:						return BF_One;
        case MTL::BlendFactorSourceColor:				return BF_SourceColor;
        case MTL::BlendFactorOneMinusSourceColor:		return BF_InverseSourceColor;
        case MTL::BlendFactorSourceAlpha:				return BF_SourceAlpha;
        case MTL::BlendFactorOneMinusSourceAlpha:		return BF_InverseSourceAlpha;
        case MTL::BlendFactorDestinationAlpha:			return BF_DestAlpha;
        case MTL::BlendFactorOneMinusDestinationAlpha:	return BF_InverseDestAlpha;
        case MTL::BlendFactorDestinationColor:			return BF_DestColor;
        case MTL::BlendFactorOneMinusDestinationColor:	return BF_InverseDestColor;
        case MTL::BlendFactorSource1Color:				return BF_Source1Color;
        case MTL::BlendFactorOneMinusSource1Color:		return BF_InverseSource1Color;
        case MTL::BlendFactorSource1Alpha:				return BF_Source1Alpha;
        case MTL::BlendFactorOneMinusSource1Alpha:		return BF_InverseSource1Alpha;
        case MTL::BlendFactorZero: default:				return BF_Zero;
	};
}

static EColorWriteMask TranslateWriteMask(MTL::ColorWriteMask WriteMask)
{
	uint32 Result = 0;
	Result |= (WriteMask & MTL::ColorWriteMaskRed) ? (CW_RED) : 0;
	Result |= (WriteMask & MTL::ColorWriteMaskGreen) ? (CW_GREEN) : 0;
	Result |= (WriteMask & MTL::ColorWriteMaskBlue) ? (CW_BLUE) : 0;
	Result |= (WriteMask & MTL::ColorWriteMaskAlpha) ? (CW_ALPHA) : 0;
	
	return (EColorWriteMask)Result;
}

template <typename InitializerType, typename StateType>
class FMetalStateObjectCache
{
public:
	FMetalStateObjectCache()
	{
		
	}
	
	~FMetalStateObjectCache()
	{
		
	}
	
	StateType Find(InitializerType Init)
	{
		if(IsRunningRHIInSeparateThread())
		{
			Mutex.ReadLock();
		}
		
		StateType* State = Cache.Find(Init);
		
		if(IsRunningRHIInSeparateThread())
		{
			Mutex.ReadUnlock();
		}
		
		return State ? *State : StateType(nullptr);
	}
	
	void Add(InitializerType Init, StateType const& State)
	{
		if(IsRunningRHIInSeparateThread())
		{
			Mutex.WriteLock();
		}
		
		Cache.Add(Init, State);
		
		if(IsRunningRHIInSeparateThread())
		{
			Mutex.WriteUnlock();
		}
	}
	
private:
	TMap<InitializerType, StateType> Cache;
	FRWLock Mutex;
};

static FMetalStateObjectCache<FSamplerStateInitializerRHI, MTL::SamplerState*> Samplers;
static FCriticalSection SamplersCS;

static MTL::SamplerState* FindOrCreateSamplerState(FMetalDevice& Device, const FSamplerStateInitializerRHI& Initializer)
{
	FScopeLock Lock(&SamplersCS);
    MTL::SamplerState* State = Samplers.Find(Initializer);
	if (!State)
	{
		MTL::SamplerDescriptor* Desc = MTL::SamplerDescriptor::alloc()->init();
        check(Desc);
        
		switch(Initializer.Filter)
		{
			case SF_AnisotropicLinear:
			case SF_AnisotropicPoint:
				Desc->setMinFilter(MTL::SamplerMinMagFilterLinear);
				Desc->setMagFilter(MTL::SamplerMinMagFilterLinear);
				Desc->setMipFilter(MTL::SamplerMipFilterLinear);
				break;
			case SF_Trilinear:
				Desc->setMinFilter(MTL::SamplerMinMagFilterLinear);
				Desc->setMagFilter(MTL::SamplerMinMagFilterLinear);
				Desc->setMipFilter(MTL::SamplerMipFilterLinear);
				break;
			case SF_Bilinear:
				Desc->setMinFilter(MTL::SamplerMinMagFilterLinear);
				Desc->setMagFilter(MTL::SamplerMinMagFilterLinear);
				Desc->setMipFilter(MTL::SamplerMipFilterNearest);
				break;
			case SF_Point:
				Desc->setMinFilter(MTL::SamplerMinMagFilterNearest);
				Desc->setMagFilter(MTL::SamplerMinMagFilterNearest);
				Desc->setMipFilter(MTL::SamplerMipFilterNearest);
				break;
		}
		Desc->setMaxAnisotropy(GetMetalMaxAnisotropy(Initializer.Filter, Initializer.MaxAnisotropy));
		Desc->setSAddressMode(TranslateWrapMode(Initializer.AddressU));
		Desc->setTAddressMode(TranslateWrapMode(Initializer.AddressV));
		Desc->setRAddressMode(TranslateWrapMode(Initializer.AddressW));
		Desc->setLodMinClamp(Initializer.MinMipLevel);
		Desc->setLodMaxClamp(Initializer.MaxMipLevel);
		Desc->setCompareFunction(TranslateSamplerCompareFunction(Initializer.SamplerComparisonFunction));
#if PLATFORM_MAC
		Desc->setBorderColor(Initializer.BorderColor == 0 ? MTL::SamplerBorderColorTransparentBlack : MTL::SamplerBorderColorOpaqueWhite);
#endif
		
#if METAL_USE_METAL_SHADER_CONVERTER
		Desc->setSupportArgumentBuffers(true);
#endif
		
		State = Device.GetDevice()->newSamplerState(Desc);
        Desc->release();
        
		Samplers.Add(Initializer, State);
	}
	return State;
}

FMetalSamplerState::FMetalSamplerState(FMetalDevice& MetalDevice, const FSamplerStateInitializerRHI& Initializer)
	: Device(MetalDevice)
{
	State = FindOrCreateSamplerState(Device, Initializer);
#if !PLATFORM_MAC
	if (GetMetalMaxAnisotropy(Initializer.Filter, Initializer.MaxAnisotropy))
	{
		FSamplerStateInitializerRHI Init = Initializer;
		Init.MaxAnisotropy = 1;
		NoAnisoState = FindOrCreateSamplerState(Device, Init);
	}
#endif
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
    FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
    check(BindlessDescriptorManager);

	if(IsMetalBindlessEnabled())
	{
		BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(State);
	}
#endif
}

FMetalSamplerState::~FMetalSamplerState()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if(IsMetalBindlessEnabled())
	{
		FMetalDynamicRHI::Get().DeferredDelete(BindlessHandle);
	}
#endif
}

FMetalRasterizerState::FMetalRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
	State = Initializer;
}

FMetalRasterizerState::~FMetalRasterizerState()
{
	
}

bool FMetalRasterizerState::GetInitializer(FRasterizerStateInitializerRHI& Init)
{
	Init = State;
	return true;
}

static FMetalStateObjectCache<FDepthStencilStateInitializerRHI, MTL::DepthStencilState*> DepthStencilStates;

FMetalDepthStencilState::FMetalDepthStencilState(MTL::Device* Device, const FDepthStencilStateInitializerRHI& InInitializer)
{
	Initializer = InInitializer;

	State = DepthStencilStates.Find(Initializer);
	if (!State)
	{
		MTL::DepthStencilDescriptor* Desc = MTL::DepthStencilDescriptor::alloc()->init();
        check(Desc);
        
		Desc->setDepthCompareFunction(TranslateCompareFunction(Initializer.DepthTest));
		Desc->setDepthWriteEnabled(Initializer.bEnableDepthWrite);
		
		if (Initializer.bEnableFrontFaceStencil)
		{
			// set up front face stencil operations
			MTL::StencilDescriptor* Stencil = MTL::StencilDescriptor::alloc()->init();
            check(Stencil);
            
			Stencil->setStencilCompareFunction(TranslateCompareFunction(Initializer.FrontFaceStencilTest));
			Stencil->setStencilFailureOperation(TranslateStencilOp(Initializer.FrontFaceStencilFailStencilOp));
			Stencil->setDepthFailureOperation(TranslateStencilOp(Initializer.FrontFaceDepthFailStencilOp));
			Stencil->setDepthStencilPassOperation(TranslateStencilOp(Initializer.FrontFacePassStencilOp));
			Stencil->setReadMask(Initializer.StencilReadMask);
			Stencil->setWriteMask(Initializer.StencilWriteMask);
			Desc->setFrontFaceStencil(Stencil);
            
            Stencil->release();
		}
		
		if (Initializer.bEnableBackFaceStencil)
		{
			// set up back face stencil operations
            MTL::StencilDescriptor* Stencil = MTL::StencilDescriptor::alloc()->init();
            check(Stencil);
            
			Stencil->setStencilCompareFunction(TranslateCompareFunction(Initializer.BackFaceStencilTest));
			Stencil->setStencilFailureOperation(TranslateStencilOp(Initializer.BackFaceStencilFailStencilOp));
			Stencil->setDepthFailureOperation(TranslateStencilOp(Initializer.BackFaceDepthFailStencilOp));
			Stencil->setDepthStencilPassOperation(TranslateStencilOp(Initializer.BackFacePassStencilOp));
			Stencil->setReadMask(Initializer.StencilReadMask);
			Stencil->setWriteMask(Initializer.StencilWriteMask);
			Desc->setBackFaceStencil(Stencil);
            
            Stencil->release();
		}
		else if(Initializer.bEnableFrontFaceStencil)
		{
			// set up back face stencil operations to front face in single-face mode
            MTL::StencilDescriptor* Stencil = MTL::StencilDescriptor::alloc()->init();
            check(Stencil);
            
			Stencil->setStencilCompareFunction(TranslateCompareFunction(Initializer.FrontFaceStencilTest));
			Stencil->setStencilFailureOperation(TranslateStencilOp(Initializer.FrontFaceStencilFailStencilOp));
			Stencil->setDepthFailureOperation(TranslateStencilOp(Initializer.FrontFaceDepthFailStencilOp));
			Stencil->setDepthStencilPassOperation(TranslateStencilOp(Initializer.FrontFacePassStencilOp));
			Stencil->setReadMask(Initializer.StencilReadMask);
			Stencil->setWriteMask(Initializer.StencilWriteMask);
			Desc->setBackFaceStencil(Stencil);
            
            Stencil->release();
		}
		
		// bake out the descriptor
		State = Device->newDepthStencilState(Desc);
        Desc->release();
        
		DepthStencilStates.Add(Initializer, State);
	}
	
	// cache some pipeline state info
	bIsDepthWriteEnabled = Initializer.bEnableDepthWrite;
	bIsStencilWriteEnabled = Initializer.bEnableFrontFaceStencil || Initializer.bEnableBackFaceStencil;
}

FMetalDepthStencilState::~FMetalDepthStencilState()
{
}

bool FMetalDepthStencilState::GetInitializer(FDepthStencilStateInitializerRHI& Init)
{
	Init = Initializer;
	return true;
}



// statics
static FMetalStateObjectCache<FBlendStateInitializerRHI::FRenderTarget, MTL::RenderPipelineColorAttachmentDescriptor*> BlendStates;
TMap<uint32, uint8> FMetalBlendState::BlendSettingsToUniqueKeyMap;
uint8 FMetalBlendState::NextKey = 0;
FCriticalSection FMetalBlendState::Mutex;

FMetalBlendState::FMetalBlendState(const FBlendStateInitializerRHI& Initializer)
{
	bUseIndependentRenderTargetBlendStates = Initializer.bUseIndependentRenderTargetBlendStates;
	bUseAlphaToCoverage = Initializer.bUseAlphaToCoverage;
	for(uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
	{
		// which initializer to use
		const FBlendStateInitializerRHI::FRenderTarget& Init =
			Initializer.bUseIndependentRenderTargetBlendStates
				? Initializer.RenderTargets[RenderTargetIndex]
				: Initializer.RenderTargets[0];

		// make a new blend state
		MTL::RenderPipelineColorAttachmentDescriptor* BlendState = BlendStates.Find(Init);

		if (!BlendState)
		{
			BlendState = MTL::RenderPipelineColorAttachmentDescriptor::alloc()->init();
            check(BlendState);
			
			// set values
			BlendState->setBlendingEnabled(
				Init.ColorBlendOp != BO_Add || Init.ColorDestBlend != BF_Zero || Init.ColorSrcBlend != BF_One ||
				Init.AlphaBlendOp != BO_Add || Init.AlphaDestBlend != BF_Zero || Init.AlphaSrcBlend != BF_One);
			BlendState->setSourceRGBBlendFactor(TranslateBlendFactor(Init.ColorSrcBlend));
			BlendState->setDestinationRGBBlendFactor(TranslateBlendFactor(Init.ColorDestBlend));
			BlendState->setRgbBlendOperation(TranslateBlendOp(Init.ColorBlendOp));
			BlendState->setSourceAlphaBlendFactor(TranslateBlendFactor(Init.AlphaSrcBlend));
			BlendState->setDestinationAlphaBlendFactor(TranslateBlendFactor(Init.AlphaDestBlend));
			BlendState->setAlphaBlendOperation(TranslateBlendOp(Init.AlphaBlendOp));
			BlendState->setWriteMask(TranslateWriteMask(Init.ColorWriteMask));
			
			BlendStates.Add(Init, BlendState);
		}
		
		RenderTargetStates[RenderTargetIndex].BlendState = BlendState;

		// get the unique key
		uint32 BlendBitMask =
			((uint32)BlendState->sourceRGBBlendFactor() << 0) | ((uint32)BlendState->destinationRGBBlendFactor() << 4) | ((uint32)BlendState->rgbBlendOperation() << 8) |
			((uint32)BlendState->sourceAlphaBlendFactor() << 11) | ((uint32)BlendState->destinationAlphaBlendFactor() << 15) | ((uint32)BlendState->alphaBlendOperation() << 19) |
			((uint32)BlendState->writeMask() << 22);
		
		
		if(IsRunningRHIInSeparateThread())
		{
			Mutex.Lock();
		}
		uint8* Key = BlendSettingsToUniqueKeyMap.Find(BlendBitMask);
		if (Key == NULL)
		{
			Key = &BlendSettingsToUniqueKeyMap.Add(BlendBitMask, NextKey++);

			// only giving limited bits to the key, since we need to pack 8 of them into a key
			checkf(NextKey < (1 << NumBits_BlendState), TEXT("Too many unique blend states to fit into the PipelineStateHash [%d allowed]"), 1 << NumBits_BlendState);
		}
		// set the key
		RenderTargetStates[RenderTargetIndex].BlendStateKey = *Key;
		if(IsRunningRHIInSeparateThread())
		{
			Mutex.Unlock();
		}
	}
}

FMetalBlendState::~FMetalBlendState()
{
}

bool FMetalBlendState::GetInitializer(FBlendStateInitializerRHI& Initializer)
{
	Initializer.bUseIndependentRenderTargetBlendStates = bUseIndependentRenderTargetBlendStates;
	Initializer.bUseAlphaToCoverage = bUseAlphaToCoverage;
	for(uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
	{
		// which initializer to use
		FBlendStateInitializerRHI::FRenderTarget& Init = Initializer.RenderTargets[RenderTargetIndex];
		MTL::RenderPipelineColorAttachmentDescriptor* CurrentState = RenderTargetStates[RenderTargetIndex].BlendState;
		
		if (CurrentState)
		{
			Init.ColorSrcBlend = TranslateBlendFactor(CurrentState->sourceRGBBlendFactor());
			Init.ColorDestBlend = TranslateBlendFactor(CurrentState->destinationRGBBlendFactor());
			Init.ColorBlendOp = TranslateBlendOp(CurrentState->rgbBlendOperation());
			Init.AlphaSrcBlend = TranslateBlendFactor(CurrentState->sourceAlphaBlendFactor());
			Init.AlphaDestBlend = TranslateBlendFactor(CurrentState->destinationAlphaBlendFactor());
			Init.AlphaBlendOp = TranslateBlendOp(CurrentState->alphaBlendOperation());
			Init.ColorWriteMask = TranslateWriteMask(CurrentState->writeMask());
		}
		
		if (!bUseIndependentRenderTargetBlendStates)
		{
			break;
		}
	}
	
	return true;
}

FSamplerStateRHIRef FMetalDynamicRHI::RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	return new FMetalSamplerState(*Device, Initializer);
}

FRasterizerStateRHIRef FMetalDynamicRHI::RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    return new FMetalRasterizerState(Initializer);
}

FDepthStencilStateRHIRef FMetalDynamicRHI::RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	return new FMetalDepthStencilState(Device->GetDevice(), Initializer);
}


FBlendStateRHIRef FMetalDynamicRHI::RHICreateBlendState(const FBlendStateInitializerRHI& Initializer)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	return new FMetalBlendState(Initializer);
}

