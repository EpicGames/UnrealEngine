// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalStateCache.h"

#include "MetalRHIPrivate.h"
#include "MetalRHIRenderQuery.h"
#include "MetalShaderTypes.h"
#include "MetalGraphicsPipelineState.h"
#include "MetalProfiler.h"
#include "MetalCommandBuffer.h"
#include "MetalCommandEncoder.h"
#include "MetalVertexDeclaration.h"
#include "MetalBindlessDescriptors.h"
#include "RHIShaderParametersShared.h"
#include "MetalResourceCollection.h"
#include "MetalDynamicRHI.h"
#include "MetalStaticSamplers.h"
#include "RHIUniformBufferUtilities.h"

#if PLATFORM_MAC
	#ifndef UINT128_MAX
		#define UINT128_MAX (((__uint128_t)1 << 127) - (__uint128_t)1 + ((__uint128_t)1 << 127))
	#endif
	#define FMETALTEXTUREMASK_MAX UINT128_MAX
#else
	#define FMETALTEXTUREMASK_MAX UINT32_MAX
#endif

static MTL::TriangleFillMode TranslateFillMode(ERasterizerFillMode FillMode)
{
	switch (FillMode)
	{
		case FM_Wireframe:	return MTL::TriangleFillModeLines;
		case FM_Point:		return MTL::TriangleFillModeFill;
		default:			return MTL::TriangleFillModeFill;
	};
}

static MTL::CullMode TranslateCullMode(ERasterizerCullMode CullMode)
{
	switch (CullMode)
	{
		case CM_CCW:	return MTL::CullModeFront;
		case CM_CW:		return MTL::CullModeBack;
		default:		return MTL::CullModeNone;
	}
}

static MTL::DepthClipMode TranslateDepthClipMode(ERasterizerDepthClipMode DepthClipMode)
{
	switch (DepthClipMode)
	{
	case ERasterizerDepthClipMode::DepthClip:	return MTL::DepthClipModeClip;
	case ERasterizerDepthClipMode::DepthClamp:	return MTL::DepthClipModeClamp;
	default:									return MTL::DepthClipModeClip;
	}
}

static MTL::StoreAction ConditionalOverrideStoreAction(MTL::StoreAction StoreAction, bool bIsDepth, bool bIsMSAA)
{
	if (!bIsMSAA)
	{
		return StoreAction;
	}
	
	static int Mode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.XRMSAAMode"))->GetValueOnAnyThread();
	if (Mode != 0)
	{
		// @todo visionos: this needs to be experimented on to find best mode
		return MTL::StoreActionStoreAndMultisampleResolve;
	}
	
	return StoreAction;
}

FORCEINLINE MTL::StoreAction GetMetalRTStoreAction(FMetalDevice& Device, ERenderTargetStoreAction StoreAction)
{
	switch(StoreAction)
	{
		case ERenderTargetStoreAction::ENoAction: return MTL::StoreActionDontCare;
		case ERenderTargetStoreAction::EStore: return MTL::StoreActionStore;
		//default store action in the desktop renderers needs to be MTL::StoreActionStoreAndMultisampleResolve.  Trying to express the renderer by the requested maxrhishaderplatform
        //because we may render to the same MSAA target twice in two separate passes.  BasePass, then some stuff, then translucency for example and we need to not lose the prior MSAA contents to do this properly.
		case ERenderTargetStoreAction::EMultisampleResolve:
		{
            static bool bNoMSAA = !AllowMSAA();
			static bool bSupportsMSAAStoreResolve = Device.SupportsFeature(EMetalFeaturesMSAAStoreAndResolve) && (GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5);
            if (bNoMSAA)
            {
                return MTL::StoreActionStore;
            }
			else if (bSupportsMSAAStoreResolve)
			{
				return MTL::StoreActionStoreAndMultisampleResolve;
			}
			else
			{
				return MTL::StoreActionMultisampleResolve;
			}
		}
		default: return MTL::StoreActionDontCare;
	}
}

FORCEINLINE MTL::StoreAction GetConditionalMetalRTStoreAction(FMetalDevice& Device, bool bMSAATarget)
{
	if (bMSAATarget)
	{
		//this func should only be getting called when an encoder had to abnormally break.  In this case we 'must' do StoreAndResolve because the encoder will be restarted later
		//with the original MSAA rendertarget and the original data must still be there to continue the render properly.
		check(Device.SupportsFeature(EMetalFeaturesMSAAStoreAndResolve));
		return MTL::StoreActionStoreAndMultisampleResolve;
	}
	else
	{
		return MTL::StoreActionStore;
	}	
}

FMetalStateCache::FMetalStateCache(FMetalDevice& MetalDevice)
: Device(MetalDevice)
, DepthStore(MTL::StoreActionUnknown)
, StencilStore(MTL::StoreActionUnknown)
, VisibilityResults(nullptr)
, VisibilityMode(MTL::VisibilityResultModeDisabled)
, VisibilityOffset(0)
, VisibilityWritten(0)
, DepthStencilState(nullptr)
, RasterizerState(nullptr)
, StencilRef(0)
, BlendFactor(FLinearColor::Transparent)
, FrameBufferSize(CGSizeMake(0.0, 0.0))
, RenderTargetArraySize(1)
, RenderPassDesc(nullptr)
, RasterBits(0)
, PipelineBits(0)
, bIsRenderTargetActive(false)
, bHasValidRenderTarget(false)
, bHasValidColorTarget(false)
{
	FMemory::Memzero(Viewport);
	FMemory::Memzero(Scissor);
	
	ActiveViewports = 0;
	ActiveScissors = 0;
	
	for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
	{
		ColorStore[i] = MTL::StoreActionUnknown;
	}
	
	FMemory::Memzero(RenderPassInfo);
	FMemory::Memzero(DirtyUniformBuffers);
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	// Reset Vertex Buffer Offsets.
	for (uint32 i = 0; i < UE_ARRAY_COUNT(VertexBufferVAs); i++)
	{
		VertexBufferVAs[i].addr = 0;
		VertexBufferVAs[i].stride = 0;
		VertexBufferVAs[i].length = 0;
	}
	
	// Clear CBV table
	for (uint32 Frequency = 0; Frequency < EMetalShaderStages::Num; Frequency++)
	{
		for (uint32 i = 0; i < TopLevelABNumEntry; i++)
		{
			CBVTable[Frequency][i] = 0ull;
		}
	}
#endif
}

FMetalStateCache::~FMetalStateCache()
{
	RenderPassDesc = nullptr;
	
	InternalResetStageBindings();
	
	VisibilityResults = nullptr;
}

void FMetalStateCache::InternalResetStageBindings()
{
	for (uint32 i = 0; i < MaxVertexElementCount; i++)
	{
		VertexBuffers[i].Buffer = nullptr;
		VertexBuffers[i].Bytes = nullptr;
#if METAL_RHI_RAYTRACING
		VertexBuffers[i].AccelerationStructure = nullptr;
#endif
		VertexBuffers[i].Length = 0;
		VertexBuffers[i].Offset = 0;
	}
	
	for (uint32 Frequency = 0; Frequency < EMetalShaderStages::Num; Frequency++)
	{
		FMetalBufferBindings& StageBufferBindings = ShaderBuffers[Frequency];
		FMetalSamplerBindings& StageSamplerBindings = ShaderSamplers[Frequency];
		FMetalTextureBindings& StageTextureBindings = ShaderTextures[Frequency];
		
		StageBufferBindings.Bound = 0;
		
		for (uint32 i = 0; i < ML_MaxSamplers; i++)
		{
			StageSamplerBindings.Samplers[i] = nullptr;
		}
		for (uint32 i = 0; i < ML_MaxBuffers; i++)
		{
			FMetalBufferBinding& CurrentBufferBinding = StageBufferBindings.Buffers[i];
			
			CurrentBufferBinding.Buffer = nullptr;
#if METAL_RHI_RAYTRACING
			CurrentBufferBinding.AccelerationStructure = nullptr;
#endif
			CurrentBufferBinding.Bytes = nullptr;
			CurrentBufferBinding.Length = 0;
			CurrentBufferBinding.ElementRowPitch = 0;
			CurrentBufferBinding.Offset = 0;
			CurrentBufferBinding.Usage = MTL::ResourceUsage(0);
			StageBufferBindings.Formats[i] = PF_Unknown;
		}
		ShaderBuffers[Frequency].Bound = 0;
		
		for (uint32 i = 0; i < ML_MaxTextures; i++)
		{
			StageTextureBindings.Textures[i] = nullptr;
			StageTextureBindings.Usage[i] = MTL::ResourceUsage(0);
		}
		StageTextureBindings.Bound = 0;
	}
}

void FMetalStateCache::Reset()
{
	SampleCount = 0;
	
	FMemory::Memzero(Viewport);
	FMemory::Memzero(Scissor);
	
	ActiveViewports = 0;
	ActiveScissors = 0;
	
	FMemory::Memzero(RenderPassInfo);
	bIsRenderTargetActive = false;
	bHasValidRenderTarget = false;
	bHasValidColorTarget = false;
	
	VisibilityResults = nullptr;
	VisibilityMode = MTL::VisibilityResultModeDisabled;
	VisibilityOffset = 0;
	VisibilityWritten = 0;
	
	DepthStencilState.SafeRelease();
	RasterizerState.SafeRelease();
	GraphicsPSO.SafeRelease();
	ComputeShader.SafeRelease();
	PreviousComputeShader.SafeRelease();
	DepthStencilSurface.SafeRelease();
	StencilRef = 0;
	
	if(RenderPassDesc)
	{
		FMetalRenderPassDescriptorPool::Get().ReleaseDescriptor(RenderPassDesc);
	}
	
	RenderPassDesc = nullptr;
	
	for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
	{
		ColorStore[i] = MTL::StoreActionUnknown;
	}
	DepthStore = MTL::StoreActionUnknown;
	StencilStore = MTL::StoreActionUnknown;
	
	BlendFactor = FLinearColor::Transparent;
	FrameBufferSize = CGSizeMake(0.0, 0.0);
	RenderTargetArraySize = 0;
	
	RasterBits = EMetalRenderFlagMask;
	PipelineBits = EMetalPipelineFlagMask;
	
	FMemory::Memzero(DirtyUniformBuffers);
	FMemory::Memzero(BoundUniformBuffers);
	ActiveUniformBuffers.Empty();
	
	InternalResetStageBindings();
	
	SetRenderTargetsActive(false);
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if(GIsMetalInitialized)
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		
		if(BindlessDescriptorManager->IsSupported())
		{
			for (uint32 i = 0; i < UE_ARRAY_COUNT(VertexBufferVAs); i++)
			{
				VertexBufferVAs[i].addr = 0;
				VertexBufferVAs[i].stride = 0;
				VertexBufferVAs[i].length = 0;
			}
			
			// Clear CBV table
			for (uint32 Frequency = 0; Frequency < EMetalShaderStages::Num; Frequency++)
			{
				for (uint32 i = 0; i < TopLevelABNumEntry; i++)
				{
					CBVTable[Frequency][i] = 0ull;
				}
			}
		}
	}
#endif
	
	ResetBindings();
}

void FMetalStateCache::ResetBindings()
{
	if(GIsMetalInitialized)
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		
		if(BindlessDescriptorManager->IsSupported())
		{
			DirtyUniformBuffers[EMetalShaderStages::Vertex] = 0xffffffff;
			DirtyUniformBuffers[EMetalShaderStages::Pixel] = 0xffffffff;
	#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			DirtyUniformBuffers[EMetalShaderStages::Geometry] = 0xffffffff;
	#endif
	#if PLATFORM_SUPPORTS_MESH_SHADERS
			DirtyUniformBuffers[EMetalShaderStages::Mesh] = 0xffffffff;
			DirtyUniformBuffers[EMetalShaderStages::Amplification] = 0xffffffff;
	#endif
		}
#endif
	}
	SetRenderTargetsActive(false);
}

static bool MTLScissorRectEqual(MTL::ScissorRect const& Left, MTL::ScissorRect const& Right)
{
	return Left.x == Right.x && Left.y == Right.y && Left.width == Right.width && Left.height == Right.height;
}

void FMetalStateCache::SetScissorRect(bool const bEnable, MTL::ScissorRect const& Rect)
{
	if (bEnable)
	{
		Scissor[0] = Rect;
	}
	else
	{
		Scissor[0].x = Viewport[0].originX;
		Scissor[0].y = Viewport[0].originY;
		Scissor[0].width = Viewport[0].width;
		Scissor[0].height = Viewport[0].height;
	}
	
	// FrameBufferSize will be 0 if using RT-less rasterization, so ignore
	if(FrameBufferSize.width != 0 && FrameBufferSize.height != 0)
	{
		Scissor[0].width = FMath::Max((Scissor[0].x + Scissor[0].width <= FMath::RoundToInt32(FrameBufferSize.width)) ? Scissor[0].width : FMath::RoundToInt32(FrameBufferSize.width) - Scissor[0].x, (NS::UInteger)0u);
		Scissor[0].height = FMath::Max((Scissor[0].y + Scissor[0].height <= FMath::RoundToInt32(FrameBufferSize.height)) ? Scissor[0].height : FMath::RoundToInt32(FrameBufferSize.height) - Scissor[0].y, (NS::UInteger)0u);
	}

	RasterBits |= EMetalRenderFlagScissorRect;
	
	ActiveScissors = 1;
}

void FMetalStateCache::SetBlendFactor(FLinearColor const& InBlendFactor)
{
	if(BlendFactor != InBlendFactor)
	{
		BlendFactor = InBlendFactor;
		RasterBits |= EMetalRenderFlagBlendColor;
	}
}

void FMetalStateCache::SetStencilRef(uint32 const InStencilRef)
{
	if(StencilRef != InStencilRef)
	{
		StencilRef = InStencilRef;
		RasterBits |= EMetalRenderFlagStencilReferenceValue;
	}
}

void FMetalStateCache::SetDepthStencilState(FMetalDepthStencilState* InDepthStencilState)
{
	if(DepthStencilState != InDepthStencilState)
	{
		DepthStencilState = InDepthStencilState;
		RasterBits |= EMetalRenderFlagDepthStencilState;
	}
}

void FMetalStateCache::SetRasterizerState(FMetalRasterizerState* InRasterizerState)
{
	if(RasterizerState != InRasterizerState)
	{
		RasterizerState = InRasterizerState;
#if PLATFORM_VISIONOS
		RasterBits |= EMetalRenderFlagFrontFacingWinding|EMetalRenderFlagCullMode|EMetalRenderFlagDepthBias|EMetalRenderFlagTriangleFillMode;
#else
		RasterBits |= EMetalRenderFlagFrontFacingWinding|EMetalRenderFlagCullMode|EMetalRenderFlagDepthBias|EMetalRenderFlagTriangleFillMode|EMetalRenderFlagDepthClipMode;
#endif
	}
}

void FMetalStateCache::SetComputeShader(FMetalComputeShader* InComputeShader)
{
	if(ComputeShader != InComputeShader)
	{
		ComputeShader = InComputeShader;
		
		PipelineBits |= EMetalPipelineFlagComputeShader;
		
		DirtyUniformBuffers[EMetalShaderStages::Compute] = 0xffffffff;

		for (uint32 Index = 0; Index < ML_MaxTextures; ++Index)
		{
			ShaderTextures[EMetalShaderStages::Compute].Textures[Index] = nullptr;
			ShaderTextures[EMetalShaderStages::Compute].Usage[Index] = MTL::ResourceUsage(0);
		}
		ShaderTextures[EMetalShaderStages::Compute].Bound = 0;

		for (const auto& PackedGlobalArray : InComputeShader->Bindings.PackedGlobalArrays)
		{
			ShaderParameters[EMetalShaderStages::Compute].PrepareGlobalUniforms(CrossCompiler::PackedTypeNameToTypeIndex(PackedGlobalArray.TypeName), PackedGlobalArray.Size);
		}
	}
}

bool FMetalStateCache::SetRenderPassInfo(FRHIRenderPassInfo const& InRenderTargets, FMetalQueryBufferRef QueryBuffer)
{
	bool bNeedsSet = false;
	
	// see if our new Info matches our previous Info
	if (NeedsToSetRenderTarget(InRenderTargets))
	{
		bool bNeedsClear = false;
		
		//Create local store action states if we support deferred store
		MTL::StoreAction NewColorStore[MaxSimultaneousRenderTargets];
		for (uint32 i = 0; i < MaxSimultaneousRenderTargets; ++i)
		{
			NewColorStore[i] = MTL::StoreActionUnknown;
		}
		
		MTL::StoreAction NewDepthStore = MTL::StoreActionUnknown;
		MTL::StoreAction NewStencilStore = MTL::StoreActionUnknown;
		
		// back this up for next frame
		RenderPassInfo = InRenderTargets;
		
		if(RenderPassDesc)
		{
			FMetalRenderPassDescriptorPool::Get().ReleaseDescriptor(RenderPassDesc);
		}
		
		// at this point, we need to fully set up an encoder/command buffer, so make a new one (autoreleased)
		MTL::RenderPassDescriptor* RenderPass = FMetalRenderPassDescriptorPool::Get().CreateDescriptor();
	
		// if we need to do queries, write to the supplied query buffer
		{
			VisibilityResults = QueryBuffer;
			RenderPass->setVisibilityResultBuffer(QueryBuffer ? QueryBuffer->Buffer->GetMTLBuffer() : nullptr);
		}
		
		if (QueryBuffer != VisibilityResults)
		{
			VisibilityOffset = 0;
			VisibilityWritten = 0;
		}
	
		// default to non-msaa
	    int32 OldCount = SampleCount;
		SampleCount = 0;
	
		bIsRenderTargetActive = false;
		bHasValidRenderTarget = false;
		bHasValidColorTarget = false;
		
		uint8 ArrayTargets = 0;
		uint8 BoundTargets = 0;
		uint32 ArrayRenderLayers = UINT_MAX;
		
		bool bFramebufferSizeSet = false;
		FrameBufferSize = CGSizeMake(0.f, 0.f);

        MTL::RenderPassColorAttachmentDescriptorArray* Attachments = RenderPass->colorAttachments();
		
		uint32 NumColorRenderTargets = RenderPassInfo.GetNumColorRenderTargets();
		
		// If we don't have RT or Depth we need to set default raster sample count
		if(!NumColorRenderTargets && RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget == nullptr)
		{
			uint32_t RTWidth = FMath::Max(RenderPassInfo.ResolveRect.X2, GetViewport(0).width);
			uint32_t RTHeight = FMath::Max(RenderPassInfo.ResolveRect.Y2, GetViewport(0).height);
			
			RenderPass->setRenderTargetWidth(RTWidth);
			RenderPass->setRenderTargetHeight(RTHeight);
			RenderPass->setDefaultRasterSampleCount(1);
		}
		else
		{
			RenderPass->setRenderTargetWidth(0);
			RenderPass->setRenderTargetHeight(0);
		}
		
		for (uint32 RenderTargetIndex = 0; RenderTargetIndex < MaxSimultaneousRenderTargets; RenderTargetIndex++)
		{
			// default to invalid
			uint8 FormatKey = 0;
			// only try to set it if it was one that was set (ie less than RenderPassInfo.NumColorRenderTargets)
			if (RenderTargetIndex < NumColorRenderTargets && RenderPassInfo.ColorRenderTargets[RenderTargetIndex].RenderTarget != nullptr)
			{
				const FRHIRenderPassInfo::FColorEntry& RenderTargetView = RenderPassInfo.ColorRenderTargets[RenderTargetIndex];
				ColorTargets[RenderTargetIndex] = RenderTargetView.RenderTarget;
				ResolveTargets[RenderTargetIndex] = RenderTargetView.ResolveTarget;
				
				FMetalSurface& Surface = *GetMetalSurfaceFromRHITexture(RenderTargetView.RenderTarget);
				FormatKey = Surface.FormatKey;
				
				uint32 Width = FMath::Max((uint32)(Surface.GetDesc().Extent.X >> RenderTargetView.MipIndex), (uint32)1);
				uint32 Height = FMath::Max((uint32)(Surface.GetDesc().Extent.Y >> RenderTargetView.MipIndex), (uint32)1);
				if(!bFramebufferSizeSet)
				{
					bFramebufferSizeSet = true;
					FrameBufferSize.width = Width;
					FrameBufferSize.height = Height;
				}
				else
				{
					FrameBufferSize.width = FMath::Min(FrameBufferSize.width, (CGFloat)Width);
					FrameBufferSize.height = FMath::Min(FrameBufferSize.height, (CGFloat)Height);
				}
	
				// if this is the back buffer, make sure we have a usable drawable
				ConditionalUpdateBackBuffer(Surface);
				FMetalSurface* ResolveSurface = GetMetalSurfaceFromRHITexture(RenderTargetView.ResolveTarget);
				if (ResolveSurface)
				{
					ConditionalUpdateBackBuffer(*ResolveSurface);
				}
					
				BoundTargets |= 1 << RenderTargetIndex;

				if (Surface.Texture.get() == nullptr)
				{
#if PLATFORM_MAC
					UE_LOG(LogMetal, Warning, TEXT("Failed to acquire backbuffer drawable texture for color render target %u; skipping render-pass setup for this frame."), RenderTargetIndex);
#endif
					SampleCount = OldCount;
					return true;
				}
	
				// user code generally passes -1 as a default, but we need 0
				uint32 ArraySliceIndex = RenderTargetView.ArraySlice == 0xFFFFFFFF ? 0 : RenderTargetView.ArraySlice;
				
				switch(Surface.GetDesc().Dimension)
				{
					case ETextureDimension::Texture2DArray:
					case ETextureDimension::Texture3D:
					case ETextureDimension::TextureCube:
					case ETextureDimension::TextureCubeArray:
						if(RenderTargetView.ArraySlice == 0xFFFFFFFF)
						{
							ArrayTargets |= (1 << RenderTargetIndex);
							ArrayRenderLayers = FMath::Min(ArrayRenderLayers, Surface.GetNumFaces());
						}
						else
						{
							ArrayRenderLayers = 1;
						}
						break;
					default:
						ArrayRenderLayers = 1;
						break;
				}
	
				MTL::RenderPassColorAttachmentDescriptor* ColorAttachment = Attachments->object(RenderTargetIndex);
	
				ERenderTargetStoreAction HighLevelStoreAction = GetStoreAction(RenderTargetView.Action);
				ERenderTargetLoadAction HighLevelLoadAction = GetLoadAction(RenderTargetView.Action);
				
				// on iOS with memory-less MSAA textures we can't load them
                // in case high level code wants to load and render to MSAA target, set attachment to a resolved texture
				bool bUseResolvedTexture = false;
#if PLATFORM_IOS
				bUseResolvedTexture = (
					Surface.MSAATexture && 
					Surface.MSAATexture->storageMode() == MTL::StorageModeMemoryless &&
					HighLevelLoadAction == ERenderTargetLoadAction::ELoad);
#endif
				
				bool bMemoryless = false;
				if (Surface.MSAATexture && !bUseResolvedTexture)
				{
#if PLATFORM_IOS
					if (Surface.MSAATexture->storageMode() == MTL::StorageModeMemoryless)
					{
						bMemoryless = true;
						HighLevelLoadAction = ERenderTargetLoadAction::EClear;
					}
#endif
					// set up an MSAA attachment
					ColorAttachment->setTexture(Surface.MSAATexture.get());

					NewColorStore[RenderTargetIndex] = GetMetalRTStoreAction(Device, ERenderTargetStoreAction::EMultisampleResolve);
					MTL::StoreAction FinalStoreAction = !bMemoryless && GRHIDeviceId > 2 ? MTL::StoreActionUnknown : NewColorStore[RenderTargetIndex];
					FinalStoreAction = ConditionalOverrideStoreAction(FinalStoreAction, false, !!Surface.MSAATexture);
					
					MTL::Texture* resolveTexture = Surface.MSAAResolveTexture ? Surface.MSAAResolveTexture.get() : Surface.Texture.get();
#if PLATFORM_IOS
					if (Surface.MSAATexture->pixelFormat() == MTL::PixelFormatR32Float)
					{
						FinalStoreAction = MTL::StoreActionDontCare;
						resolveTexture = nullptr;
					}
#endif
					ColorAttachment->setStoreAction(FinalStoreAction);
					
					ColorAttachment->setResolveTexture(resolveTexture);
					SampleCount = Surface.MSAATexture->sampleCount();
				}
				else
				{
#if PLATFORM_IOS
					if (Surface.Texture->storageMode() == MTL::StorageModeMemoryless)
					{
						bMemoryless = true;
						HighLevelStoreAction = ERenderTargetStoreAction::ENoAction;
						HighLevelLoadAction = ERenderTargetLoadAction::EClear;
					}
#endif
					// set up non-MSAA attachment
					ColorAttachment->setTexture(Surface.Texture.get());
					NewColorStore[RenderTargetIndex] = GetMetalRTStoreAction(Device, HighLevelStoreAction);
					ColorAttachment->setStoreAction(!bMemoryless ? MTL::StoreActionUnknown : NewColorStore[RenderTargetIndex]);
                    SampleCount = 1;
				}
				
				ColorAttachment->setLevel(RenderTargetView.MipIndex);
				if(Surface.GetDesc().IsTexture3D())
				{
					ColorAttachment->setSlice(0);
					ColorAttachment->setDepthPlane(ArraySliceIndex);
				}
				else
				{
					ColorAttachment->setSlice(ArraySliceIndex);
				}
				
				ColorAttachment->setLoadAction(GetMetalRTLoadAction(HighLevelLoadAction));
				
				bNeedsClear |= (ColorAttachment->loadAction() == MTL::LoadActionClear);
				
				const FClearValueBinding& ClearValue = RenderPassInfo.ColorRenderTargets[RenderTargetIndex].RenderTarget->GetClearBinding();
				if (ClearValue.ColorBinding == EClearBinding::EColorBound)
				{
					const FLinearColor& ClearColor = ClearValue.GetClearColor();
					ColorAttachment->setClearColor(MTL::ClearColor(ClearColor.R, ClearColor.G, ClearColor.B, ClearColor.A));
				}

				bHasValidRenderTarget = true;
				bHasValidColorTarget = true;
			}
			else
			{
				ColorTargets[RenderTargetIndex].SafeRelease();
				ResolveTargets[RenderTargetIndex].SafeRelease();
			}
		}
		
		RenderTargetArraySize = 1;
		
		if(ArrayTargets)
		{
			if (!Device.SupportsFeature(EMetalFeaturesLayeredRendering))
			{
				METAL_FATAL_ASSERT(ArrayRenderLayers != 1, TEXT("Layered rendering is unsupported on this device (%d)."), ArrayRenderLayers);
			}
#if PLATFORM_MAC
			else
			{
				METAL_FATAL_ASSERT(ArrayTargets == BoundTargets, TEXT("All color render targets must be layered when performing multi-layered rendering under Metal (%d != %d)."), ArrayTargets, BoundTargets);
					RenderTargetArraySize = ArrayRenderLayers;
					RenderPass->setRenderTargetArrayLength(ArrayRenderLayers);
			}
#endif
		}
	
		// default to invalid
		uint8 DepthFormatKey = 0;
		uint8 StencilFormatKey = 0;
		
		// setup depth and/or stencil
		if (RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget != nullptr)
		{
			FMetalSurface& Surface = *GetMetalSurfaceFromRHITexture(RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget);
			
			switch(Surface.GetDesc().Dimension)
			{
				case ETextureDimension::Texture2DArray:
				case ETextureDimension::Texture3D:
				case ETextureDimension::TextureCube:
				case ETextureDimension::TextureCubeArray:
					ArrayRenderLayers = Surface.GetNumFaces();
					break;
				default:
					ArrayRenderLayers = 1;
					break;
			}
			if(!ArrayTargets && ArrayRenderLayers > 1)
			{
				METAL_FATAL_ASSERT(Device.SupportsFeature(EMetalFeaturesLayeredRendering), TEXT("Layered rendering is unsupported on this device (%d)."), ArrayRenderLayers);
#if PLATFORM_MAC
					RenderTargetArraySize = ArrayRenderLayers;
					RenderPass->setRenderTargetArrayLength(ArrayRenderLayers);
#endif
			}
			
			if(!bFramebufferSizeSet)
			{
				bFramebufferSizeSet = true;
				FrameBufferSize.width  = Surface.GetDesc().Extent.X;
				FrameBufferSize.height = Surface.GetDesc().Extent.Y;
			}
			else
			{
				FrameBufferSize.width = FMath::Min(FrameBufferSize.width, (CGFloat)Surface.GetDesc().Extent.X);
				FrameBufferSize.height = FMath::Min(FrameBufferSize.height, (CGFloat)Surface.GetDesc().Extent.Y);
			}
			
			EPixelFormat DepthStencilPixelFormat = RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget->GetFormat();
			
			MTL::Texture* DepthTexture = nullptr;
			MTL::Texture* StencilTexture = nullptr;
			
            const bool bSupportSeparateMSAAResolve = FMetalCommandQueue::SupportsSeparateMSAAAndResolveTarget();
			uint32 DepthSampleCount = (Surface.MSAATexture ? Surface.MSAATexture->sampleCount() : Surface.Texture->sampleCount());
            bool bDepthStencilSampleCountMismatchFixup = false;
            DepthTexture = Surface.MSAATexture ? Surface.MSAATexture.get() : Surface.Texture.get();
			if (SampleCount == 0)
			{
				SampleCount = DepthSampleCount;
			}
			else if (SampleCount != DepthSampleCount)
            {
				static bool bLogged = false;
				if (!bSupportSeparateMSAAResolve)
				{
					//in the case of NOT support separate MSAA resolve the high level may legitimately cause a mismatch which we need to handle by binding the resolved target which we normally wouldn't do.
					DepthTexture = Surface.Texture.get();
					bDepthStencilSampleCountMismatchFixup = true;
					DepthSampleCount = 1;
				}
				else if (!bLogged)
				{
					UE_LOGF(LogMetal, Error, "If we support separate targets the high level should always give us matching counts");
					bLogged = true;
				}
            }

			if(DepthStencilPixelFormat == PF_X24_G8 || DepthStencilPixelFormat == PF_DepthStencil)
			{
				StencilTexture = DepthTexture;
			}
			
			float DepthClearValue = 0.0f;
			uint32 StencilClearValue = 0;
			const FClearValueBinding& ClearValue = RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget->GetClearBinding();
			if (ClearValue.ColorBinding == EClearBinding::EDepthStencilBound)
			{
				ClearValue.GetDepthStencil(DepthClearValue, StencilClearValue);
			}
			else if(!ArrayTargets && ArrayRenderLayers > 1)
			{
				DepthClearValue = 1.0f;
			}

           bool const bCombinedDepthStencilUsingStencil = (DepthTexture && (MTL::PixelFormat)DepthTexture->pixelFormat() != MTL::PixelFormatDepth32Float && RenderPassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingStencil());
			bool const bUsingDepth = (RenderPassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingDepth() || (bCombinedDepthStencilUsingStencil));
			if (DepthTexture && bUsingDepth)
			{
                MTL::RenderPassDepthAttachmentDescriptor* DepthAttachment = MTL::RenderPassDepthAttachmentDescriptor::alloc()->init();
                check(DepthAttachment);
                
				DepthFormatKey = Surface.FormatKey;
				
				ERenderTargetActions DepthActions = GetDepthActions(RenderPassInfo.DepthStencilRenderTarget.Action);
				ERenderTargetLoadAction DepthLoadAction = GetLoadAction(DepthActions);
				ERenderTargetStoreAction DepthStoreAction = GetStoreAction(DepthActions);

				// set up the depth attachment
				DepthAttachment->setTexture(DepthTexture);
				DepthAttachment->setLoadAction(GetMetalRTLoadAction(DepthLoadAction));
				
				bNeedsClear |= (DepthAttachment->loadAction() == MTL::LoadActionClear);
				
				ERenderTargetStoreAction HighLevelStoreAction = (Surface.MSAATexture && !bDepthStencilSampleCountMismatchFixup) ? ERenderTargetStoreAction::EMultisampleResolve : DepthStoreAction;
				if (bUsingDepth && (HighLevelStoreAction == ERenderTargetStoreAction::ENoAction || bDepthStencilSampleCountMismatchFixup))
				{
					if (DepthSampleCount > 1)
					{
						HighLevelStoreAction = ERenderTargetStoreAction::EMultisampleResolve;
					}
					else
					{
						HighLevelStoreAction = ERenderTargetStoreAction::EStore;
					}
				}
				
				const bool bSupportsMSAADepthResolve = Device.SupportsFeature(EMetalFeaturesMSAADepthResolve);
				bool bDepthTextureMemoryless = false;
#if PLATFORM_IOS
				bDepthTextureMemoryless = DepthTexture->storageMode() == MTL::StorageModeMemoryless;
				if (bDepthTextureMemoryless)
				{
					DepthAttachment->setLoadAction(MTL::LoadActionClear);
					
					if (bSupportsMSAADepthResolve && Surface.MSAATexture && DepthStoreAction == ERenderTargetStoreAction::EMultisampleResolve)
					{
						HighLevelStoreAction = ERenderTargetStoreAction::EMultisampleResolve;
					}
					else
					{
						HighLevelStoreAction = ERenderTargetStoreAction::ENoAction;
					}
				}
                else
                {
                	HighLevelStoreAction = DepthStoreAction;
                }
#endif
                //needed to quiet the metal validation that runs when you end renderpass. (it requires some kind of 'resolve' for an msaa target)
				//But with deferredstore we don't set the real one until submit time.
				NewDepthStore = !Surface.MSAATexture || bSupportsMSAADepthResolve ? GetMetalRTStoreAction(Device, HighLevelStoreAction) : MTL::StoreActionDontCare;
				MTL::StoreAction FinalStoreAction = !bDepthTextureMemoryless && Surface.MSAATexture && GRHIDeviceId > 2 ? MTL::StoreActionUnknown : NewDepthStore;
				FinalStoreAction = ConditionalOverrideStoreAction(FinalStoreAction, true, !!Surface.MSAATexture);
				DepthAttachment->setStoreAction(FinalStoreAction);
				DepthAttachment->setClearDepth(DepthClearValue);
				check(SampleCount > 0);

				if (Surface.MSAATexture && bSupportsMSAADepthResolve && DepthAttachment->storeAction() != MTL::StoreActionDontCare)
				{
                    if (!bDepthStencilSampleCountMismatchFixup)
                    {
                        DepthAttachment->setResolveTexture(Surface.MSAAResolveTexture ? Surface.MSAAResolveTexture.get() : Surface.Texture.get());
                    }
#if PLATFORM_MAC
					//would like to assert and do manual custom resolve, but that is causing some kind of weird corruption.
					//checkf(false, TEXT("Depth resolves need to do 'max' for correctness.  MacOS does not expose this yet unless the spec changed."));
#else
					DepthAttachment->setDepthResolveFilter(MTL::MultisampleDepthResolveFilterMax);
#endif
				}
				
				bHasValidRenderTarget = true;
				
				// and assign it
				RenderPass->setDepthAttachment(DepthAttachment);
                DepthAttachment->release();
			}
			
            //if we're dealing with a samplecount mismatch we just bail on stencil entirely as stencil
            //doesn't have an autoresolve target to use.
			bool const bCombinedDepthStencilUsingDepth =  RenderPassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingDepth();
			bool const bUsingStencil = RenderPassInfo.DepthStencilRenderTarget.ExclusiveDepthStencil.IsUsingStencil() || (bCombinedDepthStencilUsingDepth);
			
			if (StencilTexture && bUsingStencil)
			{
                MTL::RenderPassStencilAttachmentDescriptor* StencilAttachment = MTL::RenderPassStencilAttachmentDescriptor::alloc()->init();
				
				StencilFormatKey = Surface.FormatKey;
				
				ERenderTargetActions StencilActions = GetStencilActions(RenderPassInfo.DepthStencilRenderTarget.Action);
				ERenderTargetLoadAction StencilLoadAction = GetLoadAction(StencilActions);
				ERenderTargetStoreAction StencilStoreAction = GetStoreAction(StencilActions);
	
				// set up the stencil attachment
				StencilAttachment->setTexture(StencilTexture);
				StencilAttachment->setLoadAction(GetMetalRTLoadAction(StencilLoadAction));
				
				bNeedsClear |= (StencilAttachment->loadAction() == MTL::LoadActionClear);
				
				ERenderTargetStoreAction HighLevelStoreAction = StencilStoreAction;
				if (bUsingStencil && (HighLevelStoreAction == ERenderTargetStoreAction::ENoAction || bDepthStencilSampleCountMismatchFixup))
				{
					HighLevelStoreAction = ERenderTargetStoreAction::EStore;
				}
				
				bool bStencilMemoryless = false;
#if PLATFORM_IOS
				if (StencilTexture->storageMode() == MTL::StorageModeMemoryless)
				{
					bStencilMemoryless = true;
					HighLevelStoreAction = ERenderTargetStoreAction::ENoAction;
					StencilAttachment->setLoadAction(MTL::LoadActionClear);
				}
				else
				{
					HighLevelStoreAction = StencilStoreAction;
				}
#endif
				
				// For the case where Depth+Stencil is MSAA we can't Resolve depth and Store stencil - we can only Resolve + DontCare or StoreResolve + Store (on newer H/W and iOS).
				// We only allow use of StoreResolve in the Desktop renderers as the mobile renderer does not and should not assume hardware support for it.
				NewStencilStore = (StencilTexture->sampleCount() == 1  || GetMetalRTStoreAction(Device, ERenderTargetStoreAction::EMultisampleResolve) == MTL::StoreActionStoreAndMultisampleResolve) ? GetMetalRTStoreAction(Device, HighLevelStoreAction) : MTL::StoreActionDontCare;
                
                bool bStoreAction = !bStencilMemoryless && StencilTexture->sampleCount() > 1 && GRHIDeviceId > 2;
				StencilAttachment->setStoreAction(bStoreAction ? MTL::StoreActionUnknown : NewStencilStore);
				StencilAttachment->setClearStencil(StencilClearValue);

				if (SampleCount == 0)
				{
					SampleCount = StencilAttachment->texture()->sampleCount();
				}
				
				bHasValidRenderTarget = true;
				
				// and assign it
				RenderPass->setStencilAttachment(StencilAttachment);
                
                StencilAttachment->release();
			}
		}
		
		//Update deferred store states if required otherwise they're already set directly on the Metal Attachement Descriptors
		{
			for (uint32 i = 0; i < MaxSimultaneousRenderTargets; ++i)
			{
				ColorStore[i] = NewColorStore[i];
			}
			DepthStore = NewDepthStore;
			StencilStore = NewStencilStore;
		}
		
		if (SampleCount == 0)
		{
			SampleCount = 1;
		}
		
		bIsRenderTargetActive = bHasValidRenderTarget;
		
		// Only start encoding if the render target state is valid
		if (bHasValidRenderTarget)
		{
			// Retain and/or release the depth-stencil surface in case it is a temporary surface for a draw call that writes to depth without a depth/stencil buffer bound.
			DepthStencilSurface = RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget;
			DepthStencilResolve = RenderPassInfo.DepthStencilRenderTarget.ResolveTarget;
		}
		else
		{
			DepthStencilSurface.SafeRelease();
			DepthStencilResolve.SafeRelease();
		}
		
		RenderPassDesc = RenderPass;
		
		bNeedsSet = true;
	}

	return bNeedsSet;
}

void FMetalStateCache::InvalidateRenderTargets(void)
{
	bHasValidRenderTarget = false;
	bHasValidColorTarget = false;
	bIsRenderTargetActive = false;
}

void FMetalStateCache::SetRenderTargetsActive(bool const bActive)
{
	bIsRenderTargetActive = bActive;
}

static bool MTLViewportEqual(MTL::Viewport const& Left, MTL::Viewport const& Right)
{
	return FMath::IsNearlyEqual(Left.originX, Right.originX) &&
			FMath::IsNearlyEqual(Left.originY, Right.originY) &&
			FMath::IsNearlyEqual(Left.width, Right.width) &&
			FMath::IsNearlyEqual(Left.height, Right.height) &&
			FMath::IsNearlyEqual(Left.znear, Right.znear) &&
			FMath::IsNearlyEqual(Left.zfar, Right.zfar);
}

void FMetalStateCache::SetViewport(const MTL::Viewport& InViewport)
{
	if (!MTLViewportEqual(Viewport[0], InViewport))
	{
		Viewport[0] = InViewport;
	
		RasterBits |= EMetalRenderFlagViewport;
	}
	
	ActiveViewports = 1;
	
	MTL::ScissorRect Rect;
	
	Rect.x = InViewport.originX;
	Rect.y = InViewport.originY;
	Rect.width = InViewport.width;
	Rect.height = InViewport.height;
	
	SetScissorRect(false, Rect);
}

void FMetalStateCache::SetViewport(uint32 Index, const MTL::Viewport& InViewport)
{
	check(Index < ML_MaxViewports);
	
	if (!MTLViewportEqual(Viewport[Index], InViewport))
	{
		Viewport[Index] = InViewport;
		
		RasterBits |= EMetalRenderFlagViewport;
	}
	
	// There may not be gaps in the viewport array.
	ActiveViewports = Index + 1;
	
	// This always sets the scissor rect because the RHI doesn't bother to expose proper scissor states for multiple viewports.
	// This will have to change if we want to guarantee correctness in the mid to long term.
	{
        MTL::ScissorRect Rect;
		Rect.x = InViewport.originX;
		Rect.y = InViewport.originY;
		Rect.width = InViewport.width;
		Rect.height = InViewport.height;
		SetScissorRect(Index, false, Rect);
	}
}

void FMetalStateCache::SetScissorRect(uint32 Index, bool const bEnable, MTL::ScissorRect const& Rect)
{
	check(Index < ML_MaxViewports);
	if (!MTLScissorRectEqual(Scissor[Index], Rect))
	{
		// There's no way we can setup the bounds correctly - that must be done by the caller or incorrect rendering & crashes will ensue.
		Scissor[Index] = Rect;
		RasterBits |= EMetalRenderFlagScissorRect;
	}
	
	ActiveScissors = Index + 1;
}

void FMetalStateCache::SetViewports(const MTL::Viewport InViewport[], uint32 Count)
{
	check(Count >= 1 && Count < ML_MaxViewports);
	
	// Check if the count has changed first & if so mark for a rebind
	if (ActiveViewports != Count)
	{
		RasterBits |= EMetalRenderFlagViewport;
		RasterBits |= EMetalRenderFlagScissorRect;
	}
	
	for (uint32 i = 0; i < Count; i++)
	{
		SetViewport(i, InViewport[i]);
	}
	
	ActiveViewports = Count;
}

void FMetalStateCache::SetVertexStream(uint32 const Index, FMetalBufferPtr Buffer, FMetalBufferData* Bytes, uint32 const Offset, uint32 const Length)
{
	check(Index < MaxVertexElementCount);
	check(UNREAL_TO_METAL_BUFFER_INDEX(Index) < MaxMetalStreams);

	if (Buffer)
	{
		VertexBuffers[Index].Buffer = Buffer;
	}
	else
	{
		VertexBuffers[Index].Buffer = nullptr;
	}
	VertexBuffers[Index].Offset = 0;
	VertexBuffers[Index].Bytes = Bytes;
	VertexBuffers[Index].Length = Length;
#if METAL_RHI_RAYTRACING
	VertexBuffers[Index].AccelerationStructure = nullptr;
#endif

#if METAL_USE_METAL_SHADER_CONVERTER
	FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
	
	if(IsMetalBindlessEnabled())
	{
		// Update GPU VA (assuming the offset has changed since last time).
		if (Buffer || Bytes)
		{
			if (Bytes != nil)
			{
				uint8 const* BytesWithOffset = (((uint8 const*)VertexBuffers[Index].Bytes->Data) + Offset);
				uint32 Len = VertexBuffers[Index].Bytes->Len - Offset;
				
				FMetalBufferPtr SideBuffer = IRSideUploadToBuffer(BytesWithOffset, Len);
				CacheOrSkipResourceResidencyUpdate(SideBuffer->GetMTLBuffer(), EMetalShaderStages::Vertex, true);
				
				VertexBufferVAs[Index].addr = SideBuffer->GetGPUAddress();
				VertexBufferVAs[Index].length = Length;
			}
			else
			{
				VertexBufferVAs[Index].addr = VertexBuffers[Index].Buffer->GetGPUAddress() + Offset;
				VertexBufferVAs[Index].length = Length;
				CacheOrSkipResourceResidencyUpdate(VertexBuffers[Index].Buffer->GetMTLBuffer(), EMetalShaderStages::Vertex, true);
			}
		}
		else
		{
			VertexBufferVAs[Index].addr = 0;
			VertexBufferVAs[Index].length = 0;
		}
	}
	else
#endif
	{
		SetShaderBuffer(EMetalShaderStages::Vertex, VertexBuffers[Index].Buffer, Bytes, Offset, Length, UNREAL_TO_METAL_BUFFER_INDEX(Index), MTL::ResourceUsageRead);
	}
}

uint32 FMetalStateCache::GetVertexBufferSize(uint32 const Index)
{
	check(Index < MaxVertexElementCount);
	check(UNREAL_TO_METAL_BUFFER_INDEX(Index) < MaxMetalStreams);
	return VertexBuffers[Index].Length;
}

void FMetalStateCache::SetGraphicsPipelineState(FMetalGraphicsPipelineState* State)
{
	if (GraphicsPSO != State)
	{
		GraphicsPSO = State;
				
		DirtyUniformBuffers[EMetalShaderStages::Vertex] = 0xffffffff;
		DirtyUniformBuffers[EMetalShaderStages::Pixel] = 0xffffffff;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		DirtyUniformBuffers[EMetalShaderStages::Geometry] = 0xffffffff;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
        DirtyUniformBuffers[EMetalShaderStages::Mesh] = 0xffffffff;
        DirtyUniformBuffers[EMetalShaderStages::Amplification] = 0xffffffff;
#endif

		PipelineBits |= EMetalPipelineFlagPipelineState;
		
        if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelResetOnBind)
        {
            for (uint32 i = 0; i < EMetalShaderStages::Num; i++)
            {
                ShaderBuffers[i].Bound = UINT32_MAX;
                ShaderTextures[i].Bound = FMETALTEXTUREMASK_MAX;
                ShaderSamplers[i].Bound = UINT16_MAX;
            }
        }
		
		SetDepthStencilState(State->DepthStencilState);
		SetRasterizerState(State->RasterizerState);

#if PLATFORM_SUPPORTS_MESH_SHADERS
        if (State->MeshShader)
        {
            for (const auto& PackedGlobalArray : State->MeshShader->Bindings.PackedGlobalArrays)
            {
                ShaderParameters[EMetalShaderStages::Mesh].PrepareGlobalUniforms(CrossCompiler::PackedTypeNameToTypeIndex(PackedGlobalArray.TypeName), PackedGlobalArray.Size);
            }
            
            if (State->AmplificationShader)
            {
                for (const auto& PackedGlobalArray : State->AmplificationShader->Bindings.PackedGlobalArrays)
                {
                    ShaderParameters[EMetalShaderStages::Amplification].PrepareGlobalUniforms(CrossCompiler::PackedTypeNameToTypeIndex(PackedGlobalArray.TypeName), PackedGlobalArray.Size);
                }
            }
        }
        else if (State->VertexShader)
#endif
		{
			for (const auto& PackedGlobalArray : State->VertexShader->Bindings.PackedGlobalArrays)
			{
				ShaderParameters[EMetalShaderStages::Vertex].PrepareGlobalUniforms(CrossCompiler::PackedTypeNameToTypeIndex(PackedGlobalArray.TypeName), PackedGlobalArray.Size);
			}
		}

		if (State->PixelShader)
		{
			for (const auto& PackedGlobalArray : State->PixelShader->Bindings.PackedGlobalArrays)
			{
				ShaderParameters[EMetalShaderStages::Pixel].PrepareGlobalUniforms(CrossCompiler::PackedTypeNameToTypeIndex(PackedGlobalArray.TypeName), PackedGlobalArray.Size);
			}
		}
	}
}

FMetalShaderPipeline* FMetalStateCache::GetPipelineState() const
{
	return GraphicsPSO->GetPipeline().Get();
}

EPrimitiveType FMetalStateCache::GetPrimitiveType()
{
	check(IsValidRef(GraphicsPSO));
	return GraphicsPSO->GetPrimitiveType();
}

void FMetalStateCache::BindUniformBuffer(EMetalShaderStages const Freq, uint32 const BufferIndex, FRHIUniformBuffer* BufferRHI)
{
	check(BufferIndex < ML_MaxBuffers);
	if (BoundUniformBuffers[Freq][BufferIndex] != BufferRHI)
	{
		ActiveUniformBuffers.Add(BufferRHI);
		BoundUniformBuffers[Freq][BufferIndex] = BufferRHI;
		DirtyUniformBuffers[Freq] |= 1 << BufferIndex;
	}
}

void FMetalStateCache::SetVisibilityResultMode(MTL::VisibilityResultMode const Mode, NS::UInteger const Offset)
{
	if (VisibilityMode != Mode || VisibilityOffset != Offset)
	{
		VisibilityMode = Mode;
		VisibilityOffset = Offset;
		
		RasterBits |= EMetalRenderFlagVisibilityResultMode;
	}
}

void FMetalStateCache::ConditionalUpdateBackBuffer(FMetalSurface& Surface)
{
	// are we setting the back buffer? if so, make sure we have the drawable
	if (EnumHasAnyFlags(Surface.GetDesc().Flags, TexCreate_Presentable))
	{
		// update the back buffer texture the first time used this frame
		if (Surface.Texture.get() == nullptr)
		{
			// set the texture into the backbuffer
			Surface.GetDrawableTexture();
		}
	}
}

bool FMetalStateCache::NeedsToSetRenderTarget(const FRHIRenderPassInfo& InRenderPassInfo)
{
	// see if our new Info matches our previous Info
	uint32 CurrentNumColorRenderTargets = RenderPassInfo.GetNumColorRenderTargets();
	uint32 NewNumColorRenderTargets = InRenderPassInfo.GetNumColorRenderTargets();
	
	// basic checks
	bool bAllChecksPassed = GetHasValidRenderTarget() && bIsRenderTargetActive && CurrentNumColorRenderTargets == NewNumColorRenderTargets &&
		(InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget == RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget);

	// now check each color target if the basic tests passe
	if (bAllChecksPassed)
	{
		for (int32 RenderTargetIndex = 0; RenderTargetIndex < NewNumColorRenderTargets; RenderTargetIndex++)
		{
			const FRHIRenderPassInfo::FColorEntry& RenderTargetView = InRenderPassInfo.ColorRenderTargets[RenderTargetIndex];
			const FRHIRenderPassInfo::FColorEntry& PreviousRenderTargetView = RenderPassInfo.ColorRenderTargets[RenderTargetIndex];

			// handle simple case of switching textures or mip/slice
			if (RenderTargetView.RenderTarget != PreviousRenderTargetView.RenderTarget ||
				RenderTargetView.ResolveTarget != PreviousRenderTargetView.ResolveTarget ||
				RenderTargetView.MipIndex != PreviousRenderTargetView.MipIndex ||
				RenderTargetView.ArraySlice != PreviousRenderTargetView.ArraySlice)
			{
				bAllChecksPassed = false;
				break;
			}
			
			// it's non-trivial when we need to switch based on load/store action:
			// LoadAction - it only matters what we are switching to in the new one
			//    If we switch to Load, no need to switch as we can re-use what we already have
			//    If we switch to Clear, we have to always switch to a new RT to force the clear
			//    If we switch to DontCare, there's definitely no need to switch
			//    If we switch *from* Clear then we must change target as we *don't* want to clear again.
            if (GetLoadAction(RenderTargetView.Action) == ERenderTargetLoadAction::EClear)
            {
                bAllChecksPassed = false;
                break;
            }
            // StoreAction - this matters what the previous one was **In Spirit**
            //    If we come from Store, we need to switch to a new RT to force the store
            //    If we come from DontCare, then there's no need to switch
            //    @todo metal: However, we basically only use Store now, and don't
            //        care about intermediate results, only final, so we don't currently check the value
            //			if (PreviousRenderTargetView.StoreAction == ERenderTTargetStoreAction::EStore)
            //			{
            //				bAllChecksPassed = false;
            //				break;
            //			}
        }
        
        if (InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget && (GetLoadAction(GetDepthActions(InRenderPassInfo.DepthStencilRenderTarget.Action)) == ERenderTargetLoadAction::EClear || GetLoadAction(GetStencilActions(InRenderPassInfo.DepthStencilRenderTarget.Action)) == ERenderTargetLoadAction::EClear))
        {
            bAllChecksPassed = false;
		}
		
		if (InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget && (GetStoreAction(GetDepthActions(InRenderPassInfo.DepthStencilRenderTarget.Action)) > GetStoreAction(GetDepthActions(RenderPassInfo.DepthStencilRenderTarget.Action)) || GetStoreAction(GetStencilActions(InRenderPassInfo.DepthStencilRenderTarget.Action)) > GetStoreAction(GetStencilActions(RenderPassInfo.DepthStencilRenderTarget.Action))))
		{
			// Don't break the encoder if we can just change the store actions.
            MTL::StoreAction NewDepthStore = DepthStore;
            MTL::StoreAction NewStencilStore = StencilStore;
			if (GetStoreAction(GetDepthActions(InRenderPassInfo.DepthStencilRenderTarget.Action)) > GetStoreAction(GetDepthActions(RenderPassInfo.DepthStencilRenderTarget.Action)))
			{
				if (RenderPassDesc->depthAttachment()->texture())
				{
					FMetalSurface& Surface = *GetMetalSurfaceFromRHITexture(RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget);
					
					const uint32 DepthSampleCount = (Surface.MSAATexture ? Surface.MSAATexture->sampleCount() : Surface.Texture->sampleCount());
					bool const bDepthStencilSampleCountMismatchFixup = (SampleCount != DepthSampleCount);

					ERenderTargetStoreAction HighLevelStoreAction = (Surface.MSAATexture && !bDepthStencilSampleCountMismatchFixup) ? ERenderTargetStoreAction::EMultisampleResolve : GetStoreAction(GetDepthActions(RenderPassInfo.DepthStencilRenderTarget.Action));
					
#if PLATFORM_IOS
					MTLTexturePtr Tex = Surface.MSAATexture ? Surface.MSAATexture : Surface.Texture;
					if (Tex->storageMode() == MTL::StorageModeMemoryless)
					{
						HighLevelStoreAction = ERenderTargetStoreAction::ENoAction;
					}
#endif
					
					NewDepthStore = GetMetalRTStoreAction(Device, HighLevelStoreAction);
				}
				else
				{
					bAllChecksPassed = false;
				}
			}
			
			if (GetStoreAction(GetStencilActions(InRenderPassInfo.DepthStencilRenderTarget.Action)) > GetStoreAction(GetStencilActions(RenderPassInfo.DepthStencilRenderTarget.Action)))
			{
				if (RenderPassDesc->stencilAttachment()->texture())
				{
					NewStencilStore = GetMetalRTStoreAction(Device, GetStoreAction(GetStencilActions(RenderPassInfo.DepthStencilRenderTarget.Action)));
#if PLATFORM_IOS
					if (RenderPassDesc->stencilAttachment()->texture()->storageMode() == MTL::StorageModeMemoryless)
					{
						NewStencilStore = GetMetalRTStoreAction(Device, ERenderTargetStoreAction::ENoAction);
					}
#endif
				}
				else
				{
					bAllChecksPassed = false;
				}
			}
			
			if (bAllChecksPassed)
			{
				DepthStore = NewDepthStore;
				StencilStore = NewStencilStore;
			}
		}
	}

	// if we are setting them to nothing, then this is probably end of frame, and we can't make a framebuffer
	// with nothng, so just abort this (only need to check on single MRT case)
	if (NewNumColorRenderTargets == 1 && InRenderPassInfo.ColorRenderTargets[0].RenderTarget == nullptr &&
		InRenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget == nullptr)
	{
		bAllChecksPassed = true;
	}

	return bAllChecksPassed == false;
}

void FMetalStateCache::SetShaderBuffer(
	  EMetalShaderStages const Frequency
	, FMetalBufferPtr Buffer
	, FMetalBufferData* const Bytes
	, NS::UInteger const Offset
	, NS::UInteger const Length
	, NS::UInteger const Index
	, MTL::ResourceUsage const Usage
	, EPixelFormat const Format
	, NS::UInteger const ElementRowPitch
	, TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources
)
{
	check(Frequency < EMetalShaderStages::Num);
	check(Index < ML_MaxBuffers);
	
	FMetalBufferBindings& StageBufferBindings = ShaderBuffers[Frequency]; 
	FMetalBufferBinding& CurrentBufferBinding = StageBufferBindings.Buffers[Index];
	
	if (CurrentBufferBinding.Buffer != Buffer ||
		CurrentBufferBinding.Bytes != Bytes ||
		CurrentBufferBinding.Offset != Offset ||
		CurrentBufferBinding.Length != Length ||
		CurrentBufferBinding.ElementRowPitch != ElementRowPitch ||
		CurrentBufferBinding.Usage != Usage ||
		StageBufferBindings.Formats[Index] != Format)
	{
		CurrentBufferBinding.Buffer = Buffer;
		CurrentBufferBinding.Bytes = Bytes;
#if METAL_RHI_RAYTRACING
		CurrentBufferBinding.AccelerationStructure = nullptr;
#endif
		CurrentBufferBinding.ReferencedResources = ReferencedResources;
		CurrentBufferBinding.Offset = Offset;
		CurrentBufferBinding.Length = Length;
		CurrentBufferBinding.ElementRowPitch = ElementRowPitch;
		CurrentBufferBinding.Usage = Usage;
		
		StageBufferBindings.Formats[Index] = Format;
		
		if (Buffer || Bytes)
		{
			StageBufferBindings.Bound |= (1 << Index);
		}
		else
		{
			StageBufferBindings.Bound &= ~(1 << Index);
		}
	}
}

#if METAL_RHI_RAYTRACING
void FMetalStateCache::SetShaderBuffer(EMetalShaderStages const Frequency, FMetalAccelerationStructure* AccelerationStructure, NS::UInteger const Index, TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources)
{
	check(Frequency < EMetalShaderStages::Num);
	check(Index < ML_MaxBuffers);

	FMetalBufferBindings& StageBufferBindings = ShaderBuffers[Frequency]; 
	FMetalBufferBinding& CurrentBufferBinding = StageBufferBindings.Buffers[Index];
	
	if (CurrentBufferBinding.AccelerationStructure != AccelerationStructure)
	{
		CurrentBufferBinding.AccelerationStructure = AccelerationStructure;
		CurrentBufferBinding.Buffer = nullptr;
		CurrentBufferBinding.Bytes = nullptr;
		CurrentBufferBinding.ReferencedResources = ReferencedResources;
		CurrentBufferBinding.Offset = 0;
		CurrentBufferBinding.Length = 0;
		CurrentBufferBinding.Usage = MTL::ResourceUsage(0);

		StageBufferBindings.Formats[Index] = PF_Unknown;

		if (AccelerationStructure)
		{
			StageBufferBindings.Bound |= (1 << Index);
		}
		else
		{
			StageBufferBindings.Bound &= ~(1 << Index);
		}
	}
}
#endif // METAL_RHI_RAYTRACING

#if METAL_USE_METAL_SHADER_CONVERTER

void FMetalStateCache::CacheHeapResidency(MTL::Heap* InHeap, EMetalShaderStages const Frequency)
{
	HeapsUsedByStage[Frequency].Add(InHeap);
}

void FMetalStateCache::CacheOrSkipResourceResidencyUpdate(MTL::Resource* InResource, EMetalShaderStages const Frequency, bool bReadOnly, bool bForceUseResource)
{
	bool bAlreadyInSet = false;
	if (!bForceUseResource && bReadOnly && InResource->heap())
	{
		HeapsUsedByStage[Frequency].Add(InResource->heap(), &bAlreadyInSet);
	}
	else
	{
		TSet<MTL::Resource*>& StageResources = bReadOnly ? ROResourcesByStage[Frequency] : RWResourcesByStage[Frequency];

		if (!StageResources.Contains(InResource))
		{
			StageResources.Add(InResource);
		}
	}
}

void FMetalStateCache::IRMakeSRVResident(EMetalShaderStages const Frequency, FMetalShaderResourceView* SRV)
{
    FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();

	if(!IsMetalBindlessEnabled())
	{
		return;
	}
	
    switch (SRV->GetMetalType())
    {
        case FMetalResourceViewBase::EMetalType::Null:
		{
			checkf(false, TEXT("Attempt to bind a null SRV."));
			break;
		}
            
        case FMetalResourceViewBase::EMetalType::TextureView:
        {
            auto const& View = SRV->GetTextureView();
			IRMakeTextureResident(Frequency, View.get());
			
			break;
        }
        case FMetalResourceViewBase::EMetalType::BufferView:
        {
            auto const& View = SRV->GetBufferView();
			CacheOrSkipResourceResidencyUpdate(View.Buffer->GetMTLBuffer(), Frequency, true);
			
			break;
        }            
#if METAL_RHI_RAYTRACING
        case FMetalResourceViewBase::EMetalType::AccelerationStructure:
        {
            FMetalAccelerationStructure* AccelerationStructure = SRV->GetAccelerationStructure();
			CacheOrSkipResourceResidencyUpdate(AccelerationStructure->GetIndirectArgumentBuffer()->GetMTLBuffer(), Frequency, true);
			CacheOrSkipResourceResidencyUpdate(AccelerationStructure->GetHitContributionsBuffer()->GetMTLBuffer(), Frequency, true);
			CacheOrSkipResourceResidencyUpdate(AccelerationStructure->GetAccelerationStructure().get(), Frequency, true);
			
			break;
        }
#endif
    };
}

void FMetalStateCache::IRMakeUAVResident(EMetalShaderStages const Frequency, FMetalUnorderedAccessView* UAV)
{
    FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
	if(!IsMetalBindlessEnabled())
	{
		return;
	}

    switch (UAV->GetMetalType())
    {
        case FMetalResourceViewBase::EMetalType::Null:
            checkf(false, TEXT("Attempt to bind a null UAV. %s"), *UAV->GetOwnerName().ToString());
            break;
            
        case FMetalResourceViewBase::EMetalType::TextureView:
        {
            auto const& View = UAV->GetTextureView();
            
			if (View->buffer())
			{
				CacheOrSkipResourceResidencyUpdate(View->buffer(), Frequency, false);
			}
			else if (View->parentTexture())
			{
				CacheOrSkipResourceResidencyUpdate(View->parentTexture(), Frequency, false);
			}
			
			CacheOrSkipResourceResidencyUpdate(View.get(), Frequency, false);
			break;
        }
            
        case FMetalResourceViewBase::EMetalType::BufferView:
        {
            auto const& View = UAV->GetBufferView();
			CacheOrSkipResourceResidencyUpdate(View.Buffer->GetMTLBuffer(), Frequency, false);
			
			break;
        }
            
        case FMetalResourceViewBase::EMetalType::TextureBufferBacked:
        {
            auto const& View = UAV->GetTextureBufferBacked();
            
			CacheOrSkipResourceResidencyUpdate(View.Buffer->GetMTLBuffer(), Frequency, false);
			CacheOrSkipResourceResidencyUpdate(View.Texture.get(),Frequency, false);

			break;
        }
            
#if METAL_RHI_RAYTRACING
        case FMetalResourceViewBase::EMetalType::AccelerationStructure:
            checkNoEntry(); // not implemented
            break;
#endif
        default:
            checkNoEntry();
            break;
    }
}

void FMetalStateCache::IRMakeTextureResident(EMetalShaderStages const Frequency, MTL::Texture* Texture)
{
    FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
    
	if (Texture->buffer())
	{
		CacheOrSkipResourceResidencyUpdate(Texture->buffer(), Frequency, true);
	}
	else if (Texture->parentTexture())
	{
		CacheOrSkipResourceResidencyUpdate(Texture->parentTexture(), Frequency, true, true);
	}
	
	CacheOrSkipResourceResidencyUpdate(Texture, Frequency, true, true);
}

void FMetalStateCache::IRBindPackedUniforms(EMetalShaderStages const Frequency, int32 Index, uint8 const* Bytes, const uint32 Size, FMetalBufferPtr& Buffer)
{
	uint64 PackedUniformsVA;
	if(!Buffer)
	{
		Buffer = IRSideUploadToBuffer(Bytes, Size);
	}
	
	CacheOrSkipResourceResidencyUpdate(Buffer->GetMTLBuffer(), Frequency, true);
	check((uint32)Index < TopLevelABNumEntry);
	CBVTable[Frequency][Index] = Buffer->GetGPUAddress();
}

void FMetalStateCache::IRBindUniformBuffer(EMetalShaderStages const Frequency, int32 Index, FMetalUniformBuffer* UB)
{
	FMetalBufferPtr Buffer = UB->BackingBuffer;
	CacheOrSkipResourceResidencyUpdate(Buffer->GetMTLBuffer(), Frequency, true);

	check((uint32)Index < TopLevelABNumEntry);
	CBVTable[Frequency][Index] = Buffer->GetGPUAddress();
}
#endif

void FMetalStateCache::SetShaderTexture(EMetalShaderStages const Frequency, MTL::Texture* Texture, NS::UInteger const Index, MTL::ResourceUsage const Usage)
{
	check(Frequency < EMetalShaderStages::Num);
	check(Index < ML_MaxTextures);

#if (PLATFORM_IOS || PLATFORM_TVOS)
	UE_CLOGF(Texture->storageMode() == MTL::StorageModeMemoryless, LogMetal, Fatal, "FATAL: Attempting to bind a memoryless texture. Stage %u Index %llu Texture %ls", Frequency, Index, *NSStringToFString(Texture->description()));
#endif
	
	if (ShaderTextures[Frequency].Textures[Index] != Texture
		|| ShaderTextures[Frequency].Usage[Index] != Usage)
	{
		ShaderTextures[Frequency].Textures[Index] = Texture;
		ShaderTextures[Frequency].Usage[Index] = Usage;
		
		if (Texture)
		{
			ShaderTextures[Frequency].Bound |= (FMetalTextureMask(1) << FMetalTextureMask(Index));
		}
		else
		{
			ShaderTextures[Frequency].Bound &= ~(FMetalTextureMask(1) << FMetalTextureMask(Index));
		}
	}
}

void FMetalStateCache::SetShaderSamplerState(EMetalShaderStages const Frequency, FMetalSamplerState* const Sampler, NS::UInteger const Index)
{
	check(Frequency < EMetalShaderStages::Num);
	check(Index < ML_MaxSamplers);
	
	if (ShaderSamplers[Frequency].Samplers[Index] != (Sampler ? Sampler->State : nullptr))
	{
		if (Sampler)
		{
#if !PLATFORM_MAC
			ShaderSamplers[Frequency].Samplers[Index] = ((Frequency == EMetalShaderStages::Vertex || Frequency == EMetalShaderStages::Compute) && Sampler->NoAnisoState) ? Sampler->NoAnisoState : Sampler->State;
#else
			ShaderSamplers[Frequency].Samplers[Index] = Sampler->State;
#endif
			ShaderSamplers[Frequency].Bound |= (1 << Index);
		}
		else
		{
			ShaderSamplers[Frequency].Samplers[Index] = nullptr;
			ShaderSamplers[Frequency].Bound &= ~(1 << Index);
		}
	}
}

static EMetalShaderStages TranslateShaderStage(CrossCompiler::EShaderStage ShaderStage)
{
	switch (ShaderStage)
	{
	default: checkNoEntry(); [[fallthrough]];
	case CrossCompiler::SHADER_STAGE_PIXEL  : return EMetalShaderStages::Pixel;
	case CrossCompiler::SHADER_STAGE_VERTEX : return EMetalShaderStages::Vertex;
	case CrossCompiler::SHADER_STAGE_COMPUTE: return EMetalShaderStages::Compute;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    case CrossCompiler::SHADER_STAGE_GEOMETRY: return EMetalShaderStages::Geometry;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
    case CrossCompiler::SHADER_STAGE_MESH: return EMetalShaderStages::Mesh;
    case CrossCompiler::SHADER_STAGE_AMPLIFICATION: return EMetalShaderStages::Amplification;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
	}
}

void FMetalStateCache::SetShaderResourceView(EMetalShaderStages ShaderStage, uint32 BindIndex, FMetalShaderResourceView* SRV)
{
	if (SRV)
	{
		switch (SRV->GetMetalType())
		{
			case FMetalResourceViewBase::EMetalType::Null:
			{
				checkf(false, TEXT("Attempt to bind a null SRV."));
				break;
			}
			case FMetalResourceViewBase::EMetalType::TextureView:
			{
				SetShaderTexture(ShaderStage, SRV->GetTextureView().get(), BindIndex, MTL::ResourceUsage(MTL::ResourceUsageRead | MTL::ResourceUsageSample));
				break;
			}
			case FMetalResourceViewBase::EMetalType::BufferView:
			{
				auto const& View = SRV->GetBufferView();
				SetShaderBuffer(ShaderStage, View.Buffer, nullptr, View.Offset, View.Size, BindIndex, MTL::ResourceUsageRead);
				break;
			}
			
	#if METAL_RHI_RAYTRACING
			case FMetalResourceViewBase::EMetalType::AccelerationStructure:
			{
				SetShaderBuffer(ShaderStage, SRV->GetAccelerationStructure(), BindIndex, SRV->ReferencedResources);
				break;
			}
	#endif
		}
	}
}

void FMetalStateCache::SetShaderUnorderedAccessView(EMetalShaderStages ShaderStage, uint32 BindIndex, FMetalUnorderedAccessView* UAV)
{
	if (UAV)
	{
        MTL::ResourceUsage const Usage = MTL::ResourceUsage(MTL::ResourceUsageRead |
                                                            MTL::ResourceUsageWrite);

		switch (UAV->GetMetalType())
		{
			case FMetalResourceViewBase::EMetalType::Null:
				checkf(false, TEXT("Attempt to bind a null UAV."));
				break;

			case FMetalResourceViewBase::EMetalType::TextureView:
			{
				SetShaderTexture(ShaderStage, UAV->GetTextureView().get(), BindIndex, Usage);
				break;
			}
			case FMetalResourceViewBase::EMetalType::BufferView:
			{
				auto const& View = UAV->GetBufferView();
				SetShaderBuffer(ShaderStage, View.Buffer, nullptr, View.Offset, View.Size, BindIndex, Usage);
				break;
			}
			case FMetalResourceViewBase::EMetalType::TextureBufferBacked:
			{
				auto const& View = UAV->GetTextureBufferBacked();
				uint32 BytesPerRow = View.Texture->bufferBytesPerRow();
				uint32 ElementsPerRow = BytesPerRow / GPixelFormats[View.Format].BlockBytes;
				
				SetShaderBuffer(ShaderStage, View.Buffer, nullptr, View.Offset, View.Size,
								BindIndex, Usage, static_cast<EPixelFormat>(View.Format), ElementsPerRow);
				SetShaderTexture(ShaderStage, View.Texture.get(), BindIndex, Usage);
				break;
			}
				
	#if METAL_RHI_RAYTRACING
			case FMetalResourceViewBase::EMetalType::AccelerationStructure:
			{
				checkNoEntry(); // not implemented
				break;
			}
	#endif
		}
	}
}

template <class ShaderType>
void FMetalStateCache::SetResourcesFromTables(ShaderType Shader, CrossCompiler::EShaderStage ShaderStage)
{
	checkSlow(Shader);
	
	EMetalShaderStages Frequency = TranslateShaderStage(ShaderStage);

	struct FUniformResourceBinder
	{
		FMetalStateCache& StateCache;
		EMetalShaderStages Frequency;

		void SetUAV(FRHIUnorderedAccessView* UAV, uint8 Index)
		{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if(IsMetalBindlessEnabled())
			{
				StateCache.IRMakeUAVResident(Frequency, static_cast<FMetalUnorderedAccessView*>(UAV));
			}
			else
#endif
			{
				StateCache.SetShaderUnorderedAccessView(Frequency, Index, static_cast<FMetalUnorderedAccessView*>(UAV));
			}
		}

		void SetSRV(FRHIShaderResourceView* SRV, uint8 Index)
		{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if(IsMetalBindlessEnabled())
			{
				StateCache.IRMakeSRVResident(Frequency, static_cast<FMetalShaderResourceView*>(SRV));
			}
			else
#endif
			{
				StateCache.SetShaderResourceView(Frequency, Index, static_cast<FMetalShaderResourceView*>(SRV));
			}
		}

		void SetTexture(FRHITexture* Texture, uint8 Index)
		{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
			if(IsMetalBindlessEnabled())
			{
				StateCache.IRMakeTextureResident(Frequency, GetMetalSurfaceFromRHITexture(Texture)->Texture.get());
			}
			else
#endif
			{
				
				StateCache.SetShaderTexture(Frequency,
											GetMetalSurfaceFromRHITexture(Texture)->Texture.get(),
											Index,
											MTL::ResourceUsage(MTL::ResourceUsageRead | MTL::ResourceUsageSample));
			}
		}

		void SetSampler(FRHISamplerState* Sampler, uint8 Index)
		{
			if(!IsMetalBindlessEnabled())
			{
				StateCache.SetShaderSamplerState(Frequency, static_cast<FMetalSamplerState*>(Sampler), Index);
			}
		}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
		{
			FMetalResourceCollection* MetalResourceCollection = ResourceCast(ResourceCollection);
			SetSRV(MetalResourceCollection->GetShaderResourceView(), Index);
		}
#endif
	};

	UE::RHI::Private::SetUniformBufferResourcesFromTables(
		  FUniformResourceBinder { *this, Frequency }
		, *Shader
		, DirtyUniformBuffers[Frequency]
		, BoundUniformBuffers[Frequency]
#if ENABLE_RHI_VALIDATION
		, nullptr /*Tracker*/ // @todo: the current structure of the Metal RHI prevents easily passing the RHI validation layer tracker here
#endif
	);
}

#if METAL_USE_METAL_SHADER_CONVERTER
FMetalBufferPtr FMetalStateCache::IRSideUploadToBuffer(void const* Content, uint64 Size)
{
	FMetalTempAllocator* Allocator = Device.GetUniformAllocator();

	FMetalBufferPtr Buffer = Allocator->Allocate(Size);
	memcpy((uint8_t*)Buffer->Contents(), Content, Size);
	
	return Buffer;
}

template<EMetalShaderStages Frequency, MTL::FunctionType FunctionType>
void FMetalStateCache::FlushUsedResources(FMetalCommandEncoder* Encoder)
{
	MTL::RenderStages RenderStage = (MTL::RenderStages)0;
	switch (FunctionType)
	{
		case MTL::FunctionTypeVertex:
			RenderStage |= MTLRenderStageVertex;
			break;
		case MTL::FunctionTypeFragment:
			RenderStage |= MTLRenderStageFragment;
			break;
#if PLATFORM_SUPPORTS_MESH_SHADERS
		case MTL::FunctionTypeMesh:
			RenderStage |= MTLRenderStageMesh;
			break;
		case MTL::FunctionTypeObject:
			RenderStage |= MTLRenderStageObject;
			break;
#endif
		default:
			break;
	};

	if (!HeapsUsedByStage[Frequency].IsEmpty())
	{
		Encoder->UseHeaps(HeapsUsedByStage[Frequency].Array(), FunctionType);
	}

	if (!ROResourcesByStage[Frequency].IsEmpty())
	{
		Encoder->UseResources(ROResourcesByStage[Frequency].Array(), MTL::ResourceUsageRead, RenderStage);
	}
		
	if (!RWResourcesByStage[Frequency].IsEmpty())
	{
		Encoder->UseResources(RWResourcesByStage[Frequency].Array(), MTL::ResourceUsage(MTL::ResourceUsageRead | MTL::ResourceUsageWrite), RenderStage);
	}
	HeapsUsedByStage[Frequency].Reset();
	ROResourcesByStage[Frequency].Reset();
	RWResourcesByStage[Frequency].Reset();
}

template<class ShaderType, EMetalShaderStages Frequency, MTL::FunctionType FunctionType>
void FMetalStateCache::IRBindResourcesToEncoder(ShaderType Shader, FMetalCommandEncoder* Encoder)
{ 	
	FlushUsedResources<Frequency, FunctionType>(Encoder);
	
    // Bind Standard/Sampler descriptor heaps.
    FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
	
	if(!OverriddenDescriptorHeap)
	{
		BindlessDescriptorManager->BindDescriptorHeapsToEncoder(Encoder, FunctionType, Frequency);
	}
	else
	{
		OverriddenDescriptorHeap->BindToEncoder(Encoder, FunctionType, kIRStandardHeapBindPoint);
	}

	FMetalStaticSamplers* StaticSamplersTable = Device.GetStaticSamplers();
	const uint64 StaticSamplersTableVA = StaticSamplersTable->GetGPUAddress();

	check(Shader->Bindings.RSNumCBVs < TopLevelABNumEntry);
	CBVTable[Frequency][Shader->Bindings.RSNumCBVs] = StaticSamplersTableVA;
	
    // Bind CBV Table
    Encoder->SetShaderBytes(FunctionType, (const uint8*)CBVTable[Frequency], sizeof(uint64) * (Shader->Bindings.RSNumCBVs+1), kIRArgumentBufferBindPoint);
}

void FMetalStateCache::IRMapVertexBuffers(MTL::RenderCommandEncoder* Encoder, bool bBindForMeshShaders)
{
#if PLATFORM_SUPPORTS_MESH_SHADERS
    if (bBindForMeshShaders)
    {
        Encoder->setMeshBytes(VertexBufferVAs, sizeof(VertexBufferVAs), kIRVertexBufferBindPoint);
        Encoder->setObjectBytes(VertexBufferVAs, sizeof(VertexBufferVAs), kIRVertexBufferBindPoint);
    }
    else
#endif
    {
        Encoder->setVertexBytes(VertexBufferVAs, sizeof(VertexBufferVAs), kIRVertexBufferBindPoint);
    }
}
#endif

void FMetalStateCache::CommitRenderResources(FMetalCommandEncoder* Raster)
{
	check(IsValidRef(GraphicsPSO));
    
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    bool bIsUsingGeometryEmulation = IsValidRef(GraphicsPSO->GeometryShader);
#endif
        
#if PLATFORM_SUPPORTS_MESH_SHADERS
    if (IsValidRef(GraphicsPSO->MeshShader))
    {
        SetResourcesFromTables(GraphicsPSO->MeshShader, CrossCompiler::SHADER_STAGE_MESH);
        GetShaderParameters(EMetalShaderStages::Mesh).CommitPackedGlobals(this, Raster, EMetalShaderStages::Mesh, GraphicsPSO->MeshShader->Bindings);
        IRBindResourcesToEncoder<FMetalMeshShader*, EMetalShaderStages::Mesh, MTL::FunctionTypeMesh>(GraphicsPSO->MeshShader, Raster);
        
        if (IsValidRef(GraphicsPSO->AmplificationShader))
        {
            SetResourcesFromTables(GraphicsPSO->AmplificationShader, CrossCompiler::SHADER_STAGE_AMPLIFICATION);
            GetShaderParameters(EMetalShaderStages::Amplification).CommitPackedGlobals(this, Raster, EMetalShaderStages::Amplification, GraphicsPSO->AmplificationShader->Bindings);
            IRBindResourcesToEncoder<FMetalAmplificationShader*, EMetalShaderStages::Amplification, MTL::FunctionTypeObject>(GraphicsPSO->AmplificationShader, Raster);
        }
    }
    else
#endif
    {
        SetResourcesFromTables(GraphicsPSO->VertexShader, CrossCompiler::SHADER_STAGE_VERTEX);
        GetShaderParameters(EMetalShaderStages::Vertex).CommitPackedGlobals(this, Raster, EMetalShaderStages::Vertex, GraphicsPSO->VertexShader->Bindings);
#if METAL_USE_METAL_SHADER_CONVERTER
		if(IsMetalBindlessEnabled())
		{
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			if (bIsUsingGeometryEmulation)
			{
				IRBindResourcesToEncoder<FMetalVertexShader*, EMetalShaderStages::Vertex, MTL::FunctionTypeObject>(GraphicsPSO->VertexShader, Raster);
			}
			else
#endif
			{
				IRBindResourcesToEncoder<FMetalVertexShader*, EMetalShaderStages::Vertex, MTL::FunctionTypeVertex>(GraphicsPSO->VertexShader, Raster);
			}
		}
#endif
    }
    
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    if (IsValidRef(GraphicsPSO->GeometryShader))
    {
        SetResourcesFromTables(GraphicsPSO->GeometryShader, CrossCompiler::SHADER_STAGE_GEOMETRY);
        GetShaderParameters(EMetalShaderStages::Geometry).CommitPackedGlobals(this, Raster, EMetalShaderStages::Geometry, GraphicsPSO->GeometryShader->Bindings);
#if METAL_USE_METAL_SHADER_CONVERTER
		if(IsMetalBindlessEnabled())
		{
			IRBindResourcesToEncoder<FMetalGeometryShader*, EMetalShaderStages::Geometry, MTL::FunctionTypeMesh>(GraphicsPSO->GeometryShader, Raster);
		}
	}
#endif
#endif
        
    if (IsValidRef(GraphicsPSO->PixelShader))
    {
    	SetResourcesFromTables(GraphicsPSO->PixelShader, CrossCompiler::SHADER_STAGE_PIXEL);
        GetShaderParameters(EMetalShaderStages::Pixel).CommitPackedGlobals(this, Raster, EMetalShaderStages::Pixel, GraphicsPSO->PixelShader->Bindings);
#if METAL_USE_METAL_SHADER_CONVERTER
		if(IsMetalBindlessEnabled())
		{
			IRBindResourcesToEncoder<FMetalPixelShader*, EMetalShaderStages::Pixel, MTL::FunctionTypeFragment>(GraphicsPSO->PixelShader, Raster);
		}
#endif
    }
}

void FMetalStateCache::CommitComputeResources(FMetalCommandEncoder* Compute)
{
	check(IsValidRef(ComputeShader));
	SetResourcesFromTables(ComputeShader, CrossCompiler::SHADER_STAGE_COMPUTE);
	
	if(!OverriddenDescriptorHeap)
	{
		GetShaderParameters(EMetalShaderStages::Compute).CommitPackedGlobals(this, Compute, EMetalShaderStages::Compute, ComputeShader->Bindings);
	}
	
#if METAL_USE_METAL_SHADER_CONVERTER
	if(IsMetalBindlessEnabled())
	{
		IRBindResourcesToEncoder<FMetalComputeShader*, EMetalShaderStages::Compute, MTL::FunctionTypeKernel>(ComputeShader, Compute);
	}
#endif
}

void FMetalStateCache::StartRenderPass(const FRHIRenderPassInfo& InInfo, FMetalQueryBufferRef QueryBuffer, MTL::RenderPassDescriptor* InDesc, bool bInIsParallelContext)
{
	bIsParallelContext = bInIsParallelContext;
	
	FRHIRenderPassInfo Info = InInfo;
	
	ERenderTargetActions DepthActions = GetDepthActions(Info.DepthStencilRenderTarget.Action);
	ERenderTargetActions StencilActions = GetStencilActions(Info.DepthStencilRenderTarget.Action);
	ERenderTargetLoadAction DepthLoadAction = GetLoadAction(DepthActions);
	ERenderTargetStoreAction DepthStoreAction = GetStoreAction(DepthActions);
	ERenderTargetLoadAction StencilLoadAction = GetLoadAction(StencilActions);
	ERenderTargetStoreAction StencilStoreAction = GetStoreAction(StencilActions);

	if (Info.DepthStencilRenderTarget.DepthStencilTarget)
	{	
		if (DepthLoadAction != ERenderTargetLoadAction::EClear)
		{
			DepthLoadAction = ERenderTargetLoadAction::ELoad;
		}
		if (Info.DepthStencilRenderTarget.ExclusiveDepthStencil.IsDepthWrite())
		{
			DepthStoreAction = ERenderTargetStoreAction::EStore;
		}

		if (StencilLoadAction != ERenderTargetLoadAction::EClear)
		{
			StencilLoadAction = ERenderTargetLoadAction::ELoad;
		}
		if (Info.DepthStencilRenderTarget.ExclusiveDepthStencil.IsStencilWrite())
		{
			StencilStoreAction = ERenderTargetStoreAction::EStore;
		}
		
		DepthActions = MakeRenderTargetActions(DepthLoadAction, DepthStoreAction);
		StencilActions = MakeRenderTargetActions(StencilLoadAction, StencilStoreAction);
		Info.DepthStencilRenderTarget.Action = MakeDepthStencilTargetActions(DepthActions, StencilActions);
	}
	
	for (int32 RenderTargetIndex = 0; RenderTargetIndex < Info.GetNumColorRenderTargets(); RenderTargetIndex++)
	{
		FRHIRenderPassInfo::FColorEntry& RenderTargetView = Info.ColorRenderTargets[RenderTargetIndex];
		ERenderTargetLoadAction LoadAction = GetLoadAction(RenderTargetView.Action);
		ERenderTargetStoreAction StoreAction = GetStoreAction(RenderTargetView.Action);
		
		if (LoadAction == ERenderTargetLoadAction::EClear)
		{
			StoreAction = ERenderTargetStoreAction::EStore;
		}
		else
		{
			LoadAction = ERenderTargetLoadAction::ELoad;
		}
		RenderTargetView.Action = MakeRenderTargetActions(LoadAction, StoreAction);
	}
	
	check(!bIsParallelContext || InDesc);
	
	InvalidateRenderTargets();
	if(InDesc)
	{
		RenderPassDesc = InDesc;
	}
	else
	{
		SetRenderPassInfo(Info, QueryBuffer);
	}
}

void FMetalStateCache::EndRenderPass()
{
	// If we are running 
	if(bIsParallelContext)
	{
		RenderPassDesc = nullptr;
		bIsParallelContext = false;
	}
}

void FMetalStateCache::SetStateDirty(void)
{	
	RasterBits = UINT32_MAX;
    PipelineBits = EMetalPipelineFlagMask;
	for (uint32 i = 0; i < EMetalShaderStages::Num; i++)
	{
		ShaderBuffers[i].Bound = UINT32_MAX;
		ShaderTextures[i].Bound = FMETALTEXTUREMASK_MAX;
		ShaderSamplers[i].Bound = UINT16_MAX;
	}
}

void FMetalStateCache::SetShaderBufferDirty(EMetalShaderStages const Frequency, NS::UInteger const Index)
{
	ShaderBuffers[Frequency].Bound |= (1 << Index);
}

void FMetalStateCache::SetRenderStoreActions(FMetalCommandEncoder& CommandEncoder, bool const bConditionalSwitch)
{
	check(CommandEncoder.IsRenderCommandEncoderActive() || CommandEncoder.IsParallelRenderCommandEncoderActive())
	{
		if (bConditionalSwitch)
		{
			MTL::RenderPassColorAttachmentDescriptorArray* ColorAttachments = RenderPassDesc->colorAttachments();
			for (int32 RenderTargetIndex = 0; RenderTargetIndex < RenderPassInfo.GetNumColorRenderTargets(); RenderTargetIndex++)
			{
				FRHIRenderPassInfo::FColorEntry& RenderTargetView = RenderPassInfo.ColorRenderTargets[RenderTargetIndex];
				if(RenderTargetView.RenderTarget != nullptr)
				{
					const bool bMultiSampled = (ColorAttachments->object(RenderTargetIndex)->texture()->sampleCount() > 1);
					ColorStore[RenderTargetIndex] = GetConditionalMetalRTStoreAction(Device, bMultiSampled);
				}
			}
			
			if (RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget)
			{
				const bool bMultiSampled = RenderPassDesc->depthAttachment()->texture() && (RenderPassDesc->depthAttachment()->texture()->sampleCount() > 1);
				DepthStore = GetConditionalMetalRTStoreAction(Device, bMultiSampled);
				StencilStore = GetConditionalMetalRTStoreAction(Device, false);
			}
		}
		CommandEncoder.SetRenderPassStoreActions(ColorStore, DepthStore, StencilStore);
	}
}

void FMetalStateCache::FlushVisibilityResults(FMetalCommandEncoder& CommandEncoder)
{
#if PLATFORM_MAC
	if(VisibilityResults && VisibilityResults->Buffer && VisibilityResults->Buffer->GetMTLBuffer()->storageMode() == MTL::StorageModeManaged && VisibilityWritten && CommandEncoder.IsRenderCommandEncoderActive())
	{
		MTL::BlitCommandEncoder* Encoder = CommandEncoder.GetBlitCommandEncoder();

        Encoder->synchronizeResource(VisibilityResults->Buffer->GetMTLBuffer());
		VisibilityWritten = 0;
	}
#endif
}

void FMetalStateCache::SetRenderState(FMetalCommandEncoder& CommandEncoder)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalSetRenderStateTime);
	
	if (RasterBits)
	{
		if (RasterBits & EMetalRenderFlagViewport)
		{
			CommandEncoder.SetViewport(Viewport, ActiveViewports);
		}
		if (RasterBits & EMetalRenderFlagFrontFacingWinding)
		{
			CommandEncoder.SetFrontFacingWinding(MTL::WindingCounterClockwise);
		}
		if (RasterBits & EMetalRenderFlagCullMode)
		{
			check(IsValidRef(RasterizerState));
			CommandEncoder.SetCullMode(TranslateCullMode(RasterizerState->State.CullMode));
		}
		if (RasterBits & EMetalRenderFlagDepthBias)
		{
			check(IsValidRef(RasterizerState));
			CommandEncoder.SetDepthBias(RasterizerState->State.DepthBias, RasterizerState->State.SlopeScaleDepthBias, FLT_MAX);
		}
		if ((RasterBits & EMetalRenderFlagScissorRect))
		{
			CommandEncoder.SetScissorRect(Scissor, ActiveScissors);
		}
		if (RasterBits & EMetalRenderFlagTriangleFillMode)
		{
			check(IsValidRef(RasterizerState));
			CommandEncoder.SetTriangleFillMode(TranslateFillMode(RasterizerState->State.FillMode));
		}
		if (RasterBits & EMetalRenderFlagBlendColor)
		{
			CommandEncoder.SetBlendColor(BlendFactor.R, BlendFactor.G, BlendFactor.B, BlendFactor.A);
		}
		if (RasterBits & EMetalRenderFlagDepthStencilState)
		{
			check(IsValidRef(DepthStencilState));
            
            if (DepthStencilState && RenderPassDesc && Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
            {
                METAL_FATAL_ASSERT(DepthStencilState->bIsDepthWriteEnabled == false || (RenderPassDesc->depthAttachment() && RenderPassDesc->depthAttachment()->texture()) , TEXT("Attempting to set a depth-stencil state that writes depth but no depth texture is configured!\nState: %s\nRender Pass: %s"), *NSStringToFString(DepthStencilState->State->description()), *NSStringToFString(RenderPassDesc->description()));
                METAL_FATAL_ASSERT(DepthStencilState->bIsStencilWriteEnabled == false || (RenderPassDesc->stencilAttachment() && RenderPassDesc->stencilAttachment()->texture()), TEXT("Attempting to set a depth-stencil state that writes stencil but no stencil texture is configured!\nState: %s\nRender Pass: %s"), *NSStringToFString(DepthStencilState->State->description()), *NSStringToFString(RenderPassDesc->description()));
            }
            
			CommandEncoder.SetDepthStencilState(DepthStencilState ? DepthStencilState->State : nullptr);
		}
		if (RasterBits & EMetalRenderFlagStencilReferenceValue)
		{
			CommandEncoder.SetStencilReferenceValue(StencilRef);
		}
		if (RasterBits & EMetalRenderFlagVisibilityResultMode)
		{
			CommandEncoder.SetVisibilityResultMode(VisibilityMode, VisibilityOffset);
			if (VisibilityMode != MTL::VisibilityResultModeDisabled)
			{
            	VisibilityWritten = VisibilityOffset + FMetalQueryBufferPool::EQueryResultMaxSize;
			}
		}
		if (RasterBits & EMetalRenderFlagDepthClipMode)
		{
			check(IsValidRef(RasterizerState));
			CommandEncoder.SetDepthClipMode(TranslateDepthClipMode(RasterizerState->State.DepthClipMode));
		}
		RasterBits = 0;
	}
}

void FMetalStateCache::EnsureTextureAndType(EMetalShaderStages Stage, uint32 Index, const TMap<uint8, uint8>& TexTypes) const
{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	if (ShaderTextures[Stage].Textures[Index])
	{
		if (ShaderTextures[Stage].Textures[Index]->textureType() != TexTypes.FindRef(Index))
		{
			ensureMsgf(0, TEXT("Mismatched texture type: EMetalShaderStages %d, Index %d, ShaderTextureType %d != TexTypes %d"), (uint32)Stage, Index, (uint32)ShaderTextures[Stage].Textures[Index]->textureType(), (uint32)TexTypes.FindRef(Index));
		}
	}
	else
	{
		ensureMsgf(0, TEXT("NULL texture: EMetalShaderStages %d, Index %d"), (uint32)Stage, Index);
	}
#endif
}

/** Validates the pipeline/binding state */
bool FMetalStateCache::ValidateFunctionBindings(FMetalShaderPipeline* Pipeline, EMetalShaderFrequency Frequency, uint8 SideTableIndex)
{
    bool bOK = true;
    
#if METAL_DEBUG_OPTIONS
    
    if (!Pipeline->RenderPipelineReflection) 
    {
        return true;
    }
    
    if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelConditionalSubmit)
    {
        check(Pipeline);
        
        MTLRenderPipelineReflectionPtr Reflection = Pipeline->RenderPipelineReflection;
        check(Reflection);
        if (@available(macOS 13.0, iOS 16.0, *))
        {
            NS::Array* Bindings = nullptr;
            switch(Frequency)
            {
                case EMetalShaderVertex:
                {
                    Bindings = Reflection->vertexBindings();
                    break;
                }
                case EMetalShaderFragment:
                {
                    Bindings = Reflection->fragmentBindings();
                    break;
                }
                default:
                    check(false);
                    break;
            }
            
            for (uint32 i = 0; i < Bindings->count(); i++)
            {
                MTL::Binding* Binding = (MTL::Binding*)Bindings->object(i);
                check(Binding);
                switch(Binding->type())
                {
                    case MTL::BindingTypeBuffer:
                    {
                        checkf(Binding->index() < ML_MaxBuffers, TEXT("Metal buffer index exceeded!"));
                        
                        if (Pipeline->ResourceMask[Frequency].BufferMask & (1 << Binding->index()))
                        {
                            if (ShaderBuffers[Frequency].Buffers[Binding->index()].Buffer == nullptr && ShaderBuffers[Frequency].Buffers[Binding->index()].Bytes == nullptr)
                            {
                                bOK = false;
                                UE_LOGF(LogMetal, Warning, "Unbound buffer at Metal index %u which will crash the driver: %ls", (uint32)Binding->index(), *NSStringToFString(Binding->description()));
                            }
                        }
                        break;
                    }
                    case MTL::BindingTypeThreadgroupMemory:
                    {
                        break;
                    }
                    case MTL::BindingTypeTexture:
                    {
                        MTL::TextureBinding* TextureBinding = (MTL::TextureBinding*)Bindings->object(i);

                        checkf(Binding->index() < ML_MaxTextures, TEXT("Metal texture index exceeded!"));
                        if (ShaderTextures[Frequency].Textures[Binding->index()] == nullptr)
                        {
                            bOK = false;
                            UE_LOGF(LogMetal, Warning, "Unbound texture at Metal index %u which will crash the driver: %ls", (uint32)Binding->index(), *NSStringToFString(Binding->description()));
                        }
                        else if (ShaderTextures[Frequency].Textures[Binding->index()]->textureType() != TextureBinding->textureType())
                        {
                            bOK = false;
                            UE_LOGF(LogMetal, Warning, "Incorrect texture type bound at Metal index %u which will crash the driver: %ls\n%ls",
                                   (uint32)Binding->index(), *NSStringToFString(Binding->description()), *NSStringToFString(ShaderTextures[Frequency].Textures[Binding->index()]->description()));
                        }
                        break;
                    }
                    case MTL::BindingTypeSampler:
                    {
                        checkf(Binding->index() < ML_MaxSamplers, TEXT("Metal sampler index exceeded!"));
                        if (ShaderSamplers[Frequency].Samplers[Binding->index()] == nullptr)
                        {
                            bOK = false;
                            UE_LOGF(LogMetal, Warning, "Unbound sampler at Metal index %u which will crash the driver: %ls", (uint32)Binding->index(), *NSStringToFString(Binding->description()));
                        }
                        break;
                    }
                    default:
                        check(false);
                        break;
                }
            }
        }
        else
        {
            NS::Array* Arguments = nullptr;
            switch(Frequency)
            {
                case EMetalShaderVertex:
                {
                    Arguments = Reflection->vertexArguments();
                    break;
                }
                case EMetalShaderFragment:
                {
                    Arguments = Reflection->fragmentArguments();
                    break;
                }
                default:
                    check(false);
                    break;
            }
            
            for (uint32 i = 0; i < Arguments->count(); i++)
            {
                MTL::Argument* Arg = (MTL::Argument*)Arguments->object(i);
                check(Arg);
                switch(Arg->type())
                {
                    case MTL::ArgumentTypeBuffer:
                    {
                        checkf(Arg->index() < ML_MaxBuffers, TEXT("Metal buffer index exceeded!"));
                        if (NSStringToFString(Arg->name()) != TEXT("BufferSizes") && Arg->index() != SideTableIndex)
                        {
                            if (ShaderBuffers[Frequency].Buffers[Arg->index()].Buffer == nullptr && ShaderBuffers[Frequency].Buffers[Arg->index()].Bytes == nullptr)
                            {
                                bOK = false;
                                UE_LOGF(LogMetal, Warning, "Unbound buffer at Metal index %u which will crash the driver: %ls", (uint32)Arg->index(), *NSStringToFString(Arg->description()));
                            }
                        }
                        break;
                    }
                    case MTL::ArgumentTypeThreadgroupMemory:
                    {
                        break;
                    }
                    case MTL::ArgumentTypeTexture:
                    {
                        checkf(Arg->index() < ML_MaxTextures, TEXT("Metal texture index exceeded!"));
                        if (ShaderTextures[Frequency].Textures[Arg->index()] == nullptr)
                        {
                            bOK = false;
                            UE_LOGF(LogMetal, Warning, "Unbound texture at Metal index %u which will crash the driver: %ls", (uint32)Arg->index(), *NSStringToFString(Arg->description()));
                        }
                        else if (ShaderTextures[Frequency].Textures[Arg->index()]->textureType() != Arg->textureType())
                        {
                            bOK = false;
                            UE_LOGF(LogMetal, Warning, "Incorrect texture type bound at Metal index %u which will crash the driver: %ls\n%ls", (uint32)Arg->index(), *NSStringToFString(Arg->description()), *NSStringToFString(ShaderTextures[Frequency].Textures[Arg->index()]->description()));
                        }
                        break;
                    }
                    case MTL::ArgumentTypeSampler:
                    {
                        checkf(Arg->index() < ML_MaxSamplers, TEXT("Metal sampler index exceeded!"));
                        if (ShaderSamplers[Frequency].Samplers[Arg->index()] == nullptr)
                        {
                            bOK = false;
                            UE_LOGF(LogMetal, Warning, "Unbound sampler at Metal index %u which will crash the driver: %ls", (uint32)Arg->index(), *NSStringToFString(Arg->description()));
                        }
                        break;
                    }
                    default:
                        check(false);
                        break;
                }
            }
        }
    }
#endif
    return bOK;
}

void FMetalStateCache::Validate()
{
#if METAL_DEBUG_OPTIONS
    FMetalShaderPipeline* Pipeline = GetPipelineState();
	
    bool bOK = ValidateFunctionBindings(Pipeline, EMetalShaderVertex, GraphicsPSO->VertexShader->SideTableBinding);
    if (!bOK)
    {
        UE_LOGF(LogMetal, Error, "Metal Validation failures for vertex shader:\n%ls", Pipeline->VertexSource ? *NSStringToFString(Pipeline->VertexSource) : TEXT("nullptr"));
    }
    
    bOK = ValidateFunctionBindings(GetPipelineState(), EMetalShaderFragment, GraphicsPSO->PixelShader ? GraphicsPSO->PixelShader->SideTableBinding : -1);
    if (!bOK)
    {
        UE_LOGF(LogMetal, Error, "Metal Validation failures for fragment shader:\n%ls", Pipeline->FragmentSource ? *NSStringToFString(Pipeline->FragmentSource) : TEXT("nullptr"));
    }
#endif
}

void FMetalStateCache::SetRenderPipelineState(FMetalCommandEncoder& CommandEncoder)
{
	SCOPE_CYCLE_COUNTER(STAT_MetalSetRenderPipelineStateTime);
	
    if ((PipelineBits & EMetalPipelineFlagRasterMask) != 0)
    {
    	FMetalShaderPipeline* Pipeline = GetPipelineState();

		check(Pipeline);
        CommandEncoder.SetRenderPipelineState(Pipeline);
        
#if METAL_USE_METAL_SHADER_CONVERTER
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		
		if(IsMetalBindlessEnabled())
		{
#if PLATFORM_SUPPORTS_MESH_SHADERS
			if (GraphicsPSO->VertexDeclaration != nullptr)
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
			{
				// Update the stride table for Vertex input (done only once as this is constant/per pipeline).
				uint32* InputSlotStrides = GraphicsPSO->VertexDeclaration->InputDescriptorBufferStrides.GetData();
				for (uint32 i = 0; i < MaxVertexElementCount; i++)
				{
					VertexBufferVAs[i].stride = InputSlotStrides[i];
				}
			}
		}
#endif

        PipelineBits &= EMetalPipelineFlagComputeMask;
    }
	
	
#if METAL_DEBUG_OPTIONS
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
	if(!IsMetalBindlessEnabled())
#endif
	{
		Validate();
		
		if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
		{
			FMetalShaderPipeline* Pipeline = GetPipelineState();
			EMetalShaderStages VertexStage = EMetalShaderStages::Vertex;
			
			FMetalShaderResourceMask VertexMask = Pipeline->ResourceMask[EMetalShaderVertex];
			TArray<uint32>& MinVertexBufferSizes = Pipeline->BufferDataSizes[EMetalShaderVertex];
			const TMap<uint8, uint8>& VertexTexTypes = Pipeline->TextureTypes[EMetalShaderVertex];
			while(VertexMask.BufferMask)
			{
				uint32 Index = __builtin_ctz(VertexMask.BufferMask);
				VertexMask.BufferMask &= ~(1 << Index);
				
				if (VertexStage == EMetalShaderStages::Vertex)
				{
					FMetalBufferBinding const& Binding = ShaderBuffers[VertexStage].Buffers[Index];
					ensure(Binding.Buffer || Binding.Bytes);
					ensure(MinVertexBufferSizes.Num() > Index);
					ensure(Binding.Length >= MinVertexBufferSizes[Index]);
				}
			}
#if PLATFORM_MAC
			{
				uint64 LoTextures = (uint64)VertexMask.TextureMask;
				while(LoTextures)
				{
					uint32 Index = __builtin_ctzll(LoTextures);
					LoTextures &= ~(uint64(1) << uint64(Index));
					EnsureTextureAndType(VertexStage, Index, VertexTexTypes);
				}
				
				uint64 HiTextures = (uint64)(VertexMask.TextureMask >> FMetalTextureMask(64));
				while(HiTextures)
				{
					uint32 Index = __builtin_ctzll(HiTextures);
					HiTextures &= ~(uint64(1) << uint64(Index));
					EnsureTextureAndType(VertexStage, Index + 64, VertexTexTypes);
				}
			}
#else
			while(VertexMask.TextureMask)
			{
				uint32 Index = __builtin_ctz(VertexMask.TextureMask);
				VertexMask.TextureMask &= ~(1 << Index);
				
				EnsureTextureAndType(VertexStage, Index, VertexTexTypes);
			}
#endif
			while(VertexMask.SamplerMask)
			{
				uint32 Index = __builtin_ctz(VertexMask.SamplerMask);
				VertexMask.SamplerMask &= ~(1 << Index);
				ensure(ShaderSamplers[VertexStage].Samplers[Index]);
			}
			
			FMetalShaderResourceMask FragmentMask = Pipeline->ResourceMask[EMetalShaderFragment];
			TArray<uint32>& MinFragmentBufferSizes = Pipeline->BufferDataSizes[EMetalShaderFragment];
			const TMap<uint8, uint8>& FragmentTexTypes = Pipeline->TextureTypes[EMetalShaderFragment];
			while(FragmentMask.BufferMask)
			{
				uint32 Index = __builtin_ctz(FragmentMask.BufferMask);
				FragmentMask.BufferMask &= ~(1 << Index);
				
				FMetalBufferBinding const& Binding = ShaderBuffers[EMetalShaderStages::Pixel].Buffers[Index];
				ensure(Binding.Buffer || Binding.Bytes);
				ensure(MinFragmentBufferSizes.Num() > Index);
				ensure(Binding.Length >= MinFragmentBufferSizes[Index]);
			}
#if PLATFORM_MAC
			{
				uint64 LoTextures = (uint64)FragmentMask.TextureMask;
				while(LoTextures)
				{
					uint32 Index = __builtin_ctzll(LoTextures);
					LoTextures &= ~(uint64(1) << uint64(Index));
					EnsureTextureAndType(EMetalShaderStages::Pixel, Index, FragmentTexTypes);
				}
				
				uint64 HiTextures = (uint64)(FragmentMask.TextureMask >> FMetalTextureMask(64));
				while(HiTextures)
				{
					uint32 Index = __builtin_ctzll(HiTextures);
					HiTextures &= ~(uint64(1) << uint64(Index));
					EnsureTextureAndType(EMetalShaderStages::Pixel, Index + 64, FragmentTexTypes);
				}
			}
#else
			while(FragmentMask.TextureMask)
			{
				uint32 Index = __builtin_ctz(FragmentMask.TextureMask);
				FragmentMask.TextureMask &= ~(1 << Index);
				
				EnsureTextureAndType(EMetalShaderStages::Pixel, Index, FragmentTexTypes);
			}
#endif
			while(FragmentMask.SamplerMask)
			{
				uint32 Index = __builtin_ctz(FragmentMask.SamplerMask);
				FragmentMask.SamplerMask &= ~(1 << Index);
				ensure(ShaderSamplers[EMetalShaderStages::Pixel].Samplers[Index]);
			}
		}
	}
#endif // METAL_DEBUG_OPTIONS
}

void FMetalStateCache::SetComputePipelineState(FMetalCommandEncoder& CommandEncoder)
{
	if ((PipelineBits & EMetalPipelineFlagComputeMask) != 0)
	{
		if(PreviousComputeShader != ComputeShader)
		{
			FMetalShaderPipelinePtr Pipeline = ComputeShader->GetPipeline();
			check(Pipeline);
			CommandEncoder.SetComputePipelineState(Pipeline);
			
			PipelineBits &= EMetalPipelineFlagRasterMask;
			PreviousComputeShader = ComputeShader;
		}
    }
	
	if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
	{
		FMetalShaderPipelinePtr Pipeline = ComputeShader->GetPipeline();
		check(Pipeline);
		
        FMetalShaderResourceMask ComputeMask = Pipeline->ResourceMask[EMetalShaderCompute];
		TArray<uint32>& MinComputeBufferSizes = Pipeline->BufferDataSizes[EMetalShaderCompute];
		const TMap<uint8, uint8>& ComputeTexTypes = Pipeline->TextureTypes[EMetalShaderCompute];
		while(ComputeMask.BufferMask)
		{
			uint32 Index = __builtin_ctz(ComputeMask.BufferMask);
			ComputeMask.BufferMask &= ~(1 << Index);
			
			FMetalBufferBinding const& Binding = ShaderBuffers[EMetalShaderStages::Compute].Buffers[Index];
			ensure(Binding.Buffer || Binding.Bytes);
			ensure(MinComputeBufferSizes.Num() > Index);
			ensure(Binding.Length >= MinComputeBufferSizes[Index]);
		}
#if PLATFORM_MAC
		{
			uint64 LoTextures = (uint64)ComputeMask.TextureMask;
			while(LoTextures)
			{
				uint32 Index = __builtin_ctzll(LoTextures);
				LoTextures &= ~(uint64(1) << uint64(Index));
				EnsureTextureAndType(EMetalShaderStages::Compute, Index, ComputeTexTypes);
			}
			
			uint64 HiTextures = (uint64)(ComputeMask.TextureMask >> FMetalTextureMask(64));
			while(HiTextures)
			{
				uint32 Index = __builtin_ctzll(HiTextures);
				HiTextures &= ~(uint64(1) << uint64(Index));
				EnsureTextureAndType(EMetalShaderStages::Compute, Index + 64, ComputeTexTypes);
			}
		}
#else
		while(ComputeMask.TextureMask)
		{
			uint32 Index = __builtin_ctz(ComputeMask.TextureMask);
			ComputeMask.TextureMask &= ~(1 << Index);
			
			EnsureTextureAndType(EMetalShaderStages::Compute, Index, ComputeTexTypes);
		}
#endif
		while(ComputeMask.SamplerMask)
		{
			uint32 Index = __builtin_ctz(ComputeMask.SamplerMask);
			ComputeMask.SamplerMask &= ~(1 << Index);
			ensure(ShaderSamplers[EMetalShaderStages::Compute].Samplers[Index]);
		}
	}
}

void FMetalStateCache::CommitResourceTable(EMetalShaderStages const Frequency, MTL::FunctionType const Type, FMetalCommandEncoder& CommandEncoder)
{
	FMetalBufferBindings& BufferBindings = ShaderBuffers[Frequency];
	while(BufferBindings.Bound)
	{
		uint32 Index = __builtin_ctz(BufferBindings.Bound);
		BufferBindings.Bound &= ~(1 << Index);
		
		if (Index < ML_MaxBuffers)
		{
			FMetalBufferBinding& Binding = BufferBindings.Buffers[Index];
			if (Binding.Buffer)
			{
				CommandEncoder.SetShaderBuffer(Type, Binding.Buffer, Binding.Offset, Binding.Length, Index, Binding.Usage, BufferBindings.Formats[Index], Binding.ElementRowPitch, Binding.ReferencedResources);
			}
			else if (Binding.Bytes)
			{
				CommandEncoder.SetShaderData(Type, Binding.Bytes, Binding.Offset, Index, BufferBindings.Formats[Index], Binding.ElementRowPitch);
			}
#if METAL_RHI_RAYTRACING
			else if (Binding.AccelerationStructure)
			{
				CommandEncoder.SetShaderAccelerationStructure(Type, Binding.AccelerationStructure->GetAccelerationStructure(), Index);
			}
#endif // METAL_RHI_RAYTRACING
		}
	}
	
	FMetalTextureBindings& TextureBindings = ShaderTextures[Frequency];
#if PLATFORM_MAC
	uint64 LoTextures = (uint64)TextureBindings.Bound;
	while(LoTextures)
	{
		uint32 Index = __builtin_ctzll(LoTextures);
		LoTextures &= ~(uint64(1) << uint64(Index));
		
		if (Index < ML_MaxTextures && TextureBindings.Textures[Index])
		{
			CommandEncoder.SetShaderTexture(Type, TextureBindings.Textures[Index], Index, TextureBindings.Usage[Index]);
		}
	}
	
	uint64 HiTextures = (uint64)(TextureBindings.Bound >> FMetalTextureMask(64));
	while(HiTextures)
	{
		uint32 Index = __builtin_ctzll(HiTextures);
		HiTextures &= ~(uint64(1) << uint64(Index));
		Index += 64;
		
		if (Index < ML_MaxTextures && TextureBindings.Textures[Index])
		{
			CommandEncoder.SetShaderTexture(Type, TextureBindings.Textures[Index], Index, TextureBindings.Usage[Index]);
		}
	}
	
	TextureBindings.Bound = FMetalTextureMask(LoTextures) | (FMetalTextureMask(HiTextures) << FMetalTextureMask(64));
	check(TextureBindings.Bound == 0);
#else
	while(TextureBindings.Bound)
	{
		uint32 Index = __builtin_ctz(TextureBindings.Bound);
		TextureBindings.Bound &= ~(FMetalTextureMask(FMetalTextureMask(1) << FMetalTextureMask(Index)));
		
		if (Index < ML_MaxTextures && TextureBindings.Textures[Index])
		{
			CommandEncoder.SetShaderTexture(Type, TextureBindings.Textures[Index], Index, TextureBindings.Usage[Index]);
		}
	}
#endif
	
    FMetalSamplerBindings& SamplerBindings = ShaderSamplers[Frequency];
	while(SamplerBindings.Bound)
	{
		uint32 Index = __builtin_ctz(SamplerBindings.Bound);
		SamplerBindings.Bound &= ~(1 << Index);
		
		if (Index < ML_MaxSamplers && SamplerBindings.Samplers[Index])
		{
			CommandEncoder.SetShaderSamplerState(Type, SamplerBindings.Samplers[Index], Index);
		}
	}
}

void FMetalStateCache::ReleaseDescriptor(MTL::RenderPassDescriptor* Desc)
{
	FMetalRenderPassDescriptorPool::Get().ReleaseDescriptor(Desc);
}

void FMetalStateCache::ClearPreviousComputeState() 
{
	PreviousComputeShader.SafeRelease();
}
