// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHIPrivate.h"
#include "MetalRHIPrivate.h"
#include "MetalPipeline.h"
#include "MetalResources.h"
#include "MetalState.h"
#include "MetalUniformBuffer.h"
#include "Shaders/MetalShaderParameterCache.h"
#include "MetalRHIRenderQuery.h"

class FMetalGraphicsPipelineState;
class FMetalComputeShader;
class FMetalDevice;

enum EMetalPipelineFlags
{
	EMetalPipelineFlagPipelineState = 1 << 0,
    EMetalPipelineFlagComputeShader = 1 << 5,
    EMetalPipelineFlagRasterMask = 0xF,
    EMetalPipelineFlagComputeMask = 0x30,
    EMetalPipelineFlagMask = 0x3F
};

enum EMetalRenderFlags
{
    EMetalRenderFlagViewport = 1 << 0,
    EMetalRenderFlagFrontFacingWinding = 1 << 1,
    EMetalRenderFlagCullMode = 1 << 2,
    EMetalRenderFlagDepthBias = 1 << 3,
    EMetalRenderFlagScissorRect = 1 << 4,
    EMetalRenderFlagTriangleFillMode = 1 << 5,
    EMetalRenderFlagBlendColor = 1 << 6,
    EMetalRenderFlagDepthStencilState = 1 << 7,
    EMetalRenderFlagStencilReferenceValue = 1 << 8,
    EMetalRenderFlagVisibilityResultMode = 1 << 9,
	EMetalRenderFlagDepthClipMode = 1 << 10,
    EMetalRenderFlagMask = 0x3FF
};

struct FMetalDescriptorHeap;
class FMetalStateCache
{
public:
	FMetalStateCache(FMetalDevice& Device);
	~FMetalStateCache();
	
	/** Reset cached state for reuse */
	void Reset();
	
	/** Reset bindings between encoders */
	void ResetBindings();

	void SetScissorRect(bool const bEnable, MTL::ScissorRect const& Rect);
	void SetBlendFactor(FLinearColor const& InBlendFactor);
	void SetStencilRef(uint32 const InStencilRef);
	void SetComputeShader(FMetalComputeShader* InComputeShader);
	bool SetRenderPassInfo(FRHIRenderPassInfo const& InRenderTargets, FMetalQueryBufferRef QueryBuffer);
	void InvalidateRenderTargets(void);
	void SetRenderTargetsActive(bool const bActive);
	void SetViewport(const MTL::Viewport& InViewport);
	void SetViewports(const MTL::Viewport InViewport[], uint32 Count);
	void SetVertexStream(uint32 const Index, FMetalBufferPtr Buffer, FMetalBufferData* Bytes, uint32 const Offset, uint32 const Length);
	void SetGraphicsPipelineState(FMetalGraphicsPipelineState* State);
	void BindUniformBuffer(EMetalShaderStages const Freq, uint32 const BufferIndex, FRHIUniformBuffer* BufferRHI);
	
	/*
	 * Monitor if samples pass the depth and stencil tests.
	 * @param Mode Controls if the counter is disabled or moniters passing samples.
	 * @param Offset The offset relative to the occlusion query buffer provided when the command encoder was created.  offset must be a multiple of 8.
	 */
	void SetVisibilityResultMode(MTL::VisibilityResultMode const Mode, NS::UInteger const Offset);
	
#pragma mark - Public Shader Resource Mutators -
	/*
	 * Set a global buffer for the specified shader frequency at the given bind point index.
	 * @param Frequency The shader frequency to modify.
	 * @param Buffer The buffer to bind or nullptr to clear.
	 * @param Bytes The FMetalBufferData to bind or nullptr to clear.
	 * @param Offset The offset in the buffer or 0 when Buffer is nullptr.
	 * @param Offset The length of data (caller accounts for Offset) in the buffer or 0 when Buffer is nullptr.
	 * @param Index The index to modify.
	 * @param Usage The resource usage flags.
	 * @param Format The UAV pixel format.
	 * @param ReferencedResources The resources indirectly used by the bound buffer.
	 */
	void SetShaderBuffer(
		EMetalShaderStages const Frequency
		, FMetalBufferPtr Buffer
		, FMetalBufferData* const Bytes
		, NS::UInteger const Offset
		, NS::UInteger const Length
		, NS::UInteger const Index
		, MTL::ResourceUsage const Usage
		, EPixelFormat const Format = PF_Unknown
		, NS::UInteger const ElementRowPitch = 0
		, TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources = {}
	);

#if METAL_RHI_RAYTRACING
	/*
	 * Set a global acceleration structure for the specified shader frequency at the given bind point index.
	 * @param Frequency The shader frequency to modify.
	 * @param AccelerationStructure The acceleration structure to bind or nullptr to clear.
	 * @param Index The index to modify.
	 * @param BLAS The resources indirectly used by the bound buffer.
	 */
	void SetShaderBuffer(EMetalShaderStages const Frequency, FMetalAccelerationStructure* AccelerationStructure, NS::UInteger const Index, TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> BLAS);
#endif

#if METAL_USE_METAL_SHADER_CONVERTER
	void CacheHeapResidency(MTL::Heap* InHeap, EMetalShaderStages const Frequency);
	void CacheOrSkipResourceResidencyUpdate(MTL::Resource* InResource, EMetalShaderStages const Frequency, bool bReadOnly, bool bForceUseResource = false);
	
    void IRMakeSRVResident(EMetalShaderStages const Frequency, FMetalShaderResourceView* SRV);
    void IRMakeUAVResident(EMetalShaderStages const Frequency, FMetalUnorderedAccessView* UAV);
    void IRMakeTextureResident(EMetalShaderStages const Frequency, MTL::Texture* Texture);
    
    void IRBindUniformBuffer(EMetalShaderStages const Frequency, int32 Index, FMetalUniformBuffer* UB);
    void IRBindPackedUniforms(EMetalShaderStages const Frequency, int32 Index, uint8 const* Bytes, const uint32 Size, FMetalBufferPtr& Buffer);

	/*
	 * Write GPU data to ring buffer on CPU, returns GPU address of data written
	 * @param Content Data to upload
	 * @param Size Size in bytes
	 */
	FMetalBufferPtr IRSideUploadToBuffer(void const* Content, uint64 Size);
	
	template<EMetalShaderStages Frequency, MTL::FunctionType FunctionType>
	void FlushUsedResources(FMetalCommandEncoder* Encoder);
	
    template<class ShaderType, EMetalShaderStages Frequency, MTL::FunctionType FunctionType>
    void IRBindResourcesToEncoder(ShaderType Shader, FMetalCommandEncoder* Encoder);

    void IRMapVertexBuffers(MTL::RenderCommandEncoder* Encoder, bool bBindForMeshShaders = false);
#endif

	/*
	 * Set a global texture for the specified shader frequency at the given bind point index.
	 * @param Frequency The shader frequency to modify.
	 * @param Texture The texture to bind or nullptr to clear.
	 * @param Index The index to modify.
	 * @param Usage The resource usage flags.
	 */
	void SetShaderTexture(EMetalShaderStages const Frequency, MTL::Texture* Texture, NS::UInteger const Index, MTL::ResourceUsage const Usage);
	
	/*
	 * Set a global sampler for the specified shader frequency at the given bind point index.
	 * @param Frequency The shader frequency to modify.
	 * @param Sampler The sampler state to bind or nullptr to clear.
	 * @param Index The index to modify.
	 */
	void SetShaderSamplerState(EMetalShaderStages const Frequency, FMetalSamplerState* const Sampler, NS::UInteger const Index);
	
	void SetShaderResourceView(EMetalShaderStages ShaderStage, uint32 BindIndex, FMetalShaderResourceView* SRV);
	void SetShaderUnorderedAccessView(EMetalShaderStages ShaderStage, uint32 BindIndex, FMetalUnorderedAccessView* UAV);
	void SetStateDirty(void);
	void SetShaderBufferDirty(EMetalShaderStages const Frequency, NS::UInteger const Index);
	void SetRenderStoreActions(FMetalCommandEncoder& CommandEncoder, bool const bConditionalSwitch = false);
	void SetRenderState(FMetalCommandEncoder& CommandEncoder);
	
	void CommitRenderResources(FMetalCommandEncoder* Raster);	
	void CommitComputeResources(FMetalCommandEncoder* Compute);

	void CommitResourceTable(EMetalShaderStages const Frequency, MTL::FunctionType const Type, FMetalCommandEncoder& CommandEncoder);
	void StartRenderPass(const FRHIRenderPassInfo& Info, FMetalQueryBufferRef QueryBuffer, MTL::RenderPassDescriptor* InDesc, bool bInIsParallelContext);
	void EndRenderPass();
	
	FMetalShaderParameterCache& GetShaderParameters(EMetalShaderStages const Stage) { return ShaderParameters[Stage]; }
	FLinearColor const& GetBlendFactor() const { return BlendFactor; }
	uint32 GetStencilRef() const { return StencilRef; }
	FMetalDepthStencilState* GetDepthStencilState() const { return DepthStencilState; }
	FMetalRasterizerState* GetRasterizerState() const { return RasterizerState; }
	FMetalGraphicsPipelineState* GetGraphicsPSO() const { return GraphicsPSO; }
	FMetalComputeShader* GetComputeShader() const { return ComputeShader; }
	CGSize GetFrameBufferSize() const { return FrameBufferSize; }
	FRHIRenderPassInfo const& GetRenderPassInfo() const { return RenderPassInfo; }
	int32 GetNumRenderTargets() { return bHasValidColorTarget ? RenderPassInfo.GetNumColorRenderTargets() : -1; }
	bool GetHasValidRenderTarget() const { return bHasValidRenderTarget; }
	bool GetHasValidColorTarget() const { return bHasValidColorTarget; }
	const MTL::Viewport& GetViewport(uint32 const Index) const { check(Index < ML_MaxViewports); return Viewport[Index]; }
	uint32 GetVertexBufferSize(uint32 const Index);
	uint32 GetRenderTargetArraySize() const { return RenderTargetArraySize; }
	FMetalQueryBufferRef GetVisibilityResultsBuffer() const { return VisibilityResults; }
	bool NeedsToSetRenderTarget(const FRHIRenderPassInfo& RenderPassInfo);
	bool HasValidDepthStencilSurface() const { return IsValidRef(DepthStencilSurface); }

	MTL::RenderPassDescriptor* GetRenderPassDescriptor(void) const { return RenderPassDesc; }
	
	uint32 GetSampleCount(void) const { return SampleCount; }
	FMetalShaderPipeline* GetPipelineState() const;
	EPrimitiveType GetPrimitiveType();
	MTL::VisibilityResultMode GetVisibilityResultMode() { return VisibilityMode; }
	uint32 GetVisibilityResultOffset() { return VisibilityOffset; }
	
	void SetRenderPipelineState(FMetalCommandEncoder& CommandEncoder);
    void SetComputePipelineState(FMetalCommandEncoder& CommandEncoder);
	void FlushVisibilityResults(FMetalCommandEncoder& CommandEncoder);
	
	void ReleaseDescriptor(MTL::RenderPassDescriptor* Desc);
	void ClearPreviousComputeState();
	
	FMetalDescriptorHeap* OverriddenDescriptorHeap = nullptr;
	void SetOverriddenDescriptorHeap(FMetalDescriptorHeap* InHeap) 
	{ 
		OverriddenDescriptorHeap = InHeap;
	}
	
	void ConditionalUpdateBackBuffer(FMetalSurface& Surface);
private:
	void SetDepthStencilState(FMetalDepthStencilState* InDepthStencilState);
	void SetRasterizerState(FMetalRasterizerState* InRasterizerState);
	
	template <class ShaderType>
	void SetResourcesFromTables(ShaderType Shader, CrossCompiler::EShaderStage ShaderStage);
	
	void SetViewport(uint32 Index, const MTL::Viewport& InViewport);
	void SetScissorRect(uint32 Index, bool const bEnable, MTL::ScissorRect const& Rect);
    
    void Validate();
    bool ValidateFunctionBindings(FMetalShaderPipeline* Pipeline, EMetalShaderFrequency Frequency, uint8 SideTableIndex);

private:

	void EnsureTextureAndType(EMetalShaderStages Stage, uint32 Index, const TMap<uint8, uint8>& TexTypes) const;
	
private:
#pragma mark - Private Type Declarations -
	struct FMetalBufferBinding
	{
		FMetalBufferBinding() : Bytes(nullptr), Offset(0), Length(0), Usage((MTL::ResourceUsage)0), ReferencedResources{} {}
		/** The bound buffers or nullptr. */
		FMetalBufferPtr Buffer = nullptr;
		/** Optional bytes buffer used instead of an FMetalBuffer */
		FMetalBufferData* Bytes = nullptr;
		/** The bound buffer offsets or 0. */
        NS::UInteger Offset;
		/** The bound buffer lengths or 0. */
        NS::UInteger Length;
		/** The bound buffer element row pitch or 0 */
        NS::UInteger ElementRowPitch;
		/** The bound buffer usage or 0 */
		MTL::ResourceUsage Usage;
#if METAL_RHI_RAYTRACING
		/** The bound acceleration structure or nullptr. */
		FMetalAccelerationStructure* AccelerationStructure = nullptr;
#endif // METAL_RHI_RAYTRACING
		/** The resources referenced by this binding (e.g. BLAS referenced by a TLAS) */
		TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources;
	};
	
	/** A structure of arrays for the current buffer binding settings. */
	struct FMetalBufferBindings
	{
		FMetalBufferBindings() : Bound(0) {}
		/** The bound buffers/bytes or nullptr. */
		FMetalBufferBinding Buffers[ML_MaxBuffers];
		/** The pixel formats for buffers bound so that we emulate [RW]Buffer<T> type conversion */
		EPixelFormat Formats[ML_MaxBuffers];
		/** A bitmask for which buffers were bound by the application where a bit value of 1 is bound and 0 is unbound. */
		uint32 Bound;
	};
	
	/** A structure of arrays for the current texture binding settings. */
	struct FMetalTextureBindings
	{
		FMetalTextureBindings() : Bound(0) { FMemory::Memzero(Usage); }
		/** The bound textures or nullptr. */
		MTL::Texture* Textures[ML_MaxTextures];
		/** The bound texture usage or 0 */
		MTL::ResourceUsage Usage[ML_MaxTextures];
		/** A bitmask for which textures were bound by the application where a bit value of 1 is bound and 0 is unbound. */
		FMetalTextureMask Bound;
	};
	
	/** A structure of arrays for the current sampler binding settings. */
	struct FMetalSamplerBindings
	{
		FMetalSamplerBindings() : Bound(0) {}
		/** The bound sampler states or nullptr. */
		MTL::SamplerState* Samplers[ML_MaxSamplers];
		/** A bitmask for which samplers were bound by the application where a bit value of 1 is bound and 0 is unbound. */
		uint16 Bound;
	};
    
private:
	void InternalResetStageBindings();
	
	FMetalDevice& Device;
	
	FMetalShaderParameterCache ShaderParameters[EMetalShaderStages::Num];

	uint32 SampleCount;

	TSet<TRefCountPtr<FRHIUniformBuffer>> ActiveUniformBuffers;
	FRHIUniformBuffer* BoundUniformBuffers[EMetalShaderStages::Num][ML_MaxBuffers];
	
	/** Bitfield for which uniform buffers are dirty */
	uint32 DirtyUniformBuffers[EMetalShaderStages::Num];
	
	/** Vertex attribute buffers */
	FMetalBufferBinding VertexBuffers[MaxVertexElementCount];
	
	/** Bound shader resource tables. */
	FMetalBufferBindings ShaderBuffers[EMetalShaderStages::Num];
	FMetalTextureBindings ShaderTextures[EMetalShaderStages::Num];
	FMetalSamplerBindings ShaderSamplers[EMetalShaderStages::Num];
	
	MTL::StoreAction ColorStore[MaxSimultaneousRenderTargets];
    MTL::StoreAction DepthStore;
    MTL::StoreAction StencilStore;
		
#if METAL_USE_METAL_SHADER_CONVERTER
    static constexpr uint32 TopLevelABNumEntry = METAL_MAX_BUFFERS + 1; // One slot per buffer index (0..METAL_MAX_BUFFERS-1) plus one for the static sampler VA
    uint64 CBVTable[EMetalShaderStages::Num][TopLevelABNumEntry];
	
	IRRuntimeVertexBuffer VertexBufferVAs[31];
	
	TStaticArray<TSet<MTL::Heap*>, EMetalShaderStages::Num> HeapsUsedByStage;
	TStaticArray<TSet<MTL::Resource*>, EMetalShaderStages::Num> RWResourcesByStage;
	TStaticArray<TSet<MTL::Resource*>, EMetalShaderStages::Num> ROResourcesByStage;
#endif

	FMetalQueryBufferRef VisibilityResults;
    MTL::VisibilityResultMode VisibilityMode;
	NS::UInteger VisibilityOffset;
	NS::UInteger VisibilityWritten;

	TRefCountPtr<FMetalDepthStencilState> DepthStencilState;
	TRefCountPtr<FMetalRasterizerState> RasterizerState;
	TRefCountPtr<FMetalGraphicsPipelineState> GraphicsPSO;
	TRefCountPtr<FMetalComputeShader> ComputeShader;
	TRefCountPtr<FMetalComputeShader> PreviousComputeShader;
	
	uint32 StencilRef;
	
	FLinearColor BlendFactor;
	CGSize FrameBufferSize;
	
	uint32 RenderTargetArraySize;

    MTL::Viewport Viewport[ML_MaxViewports];
    MTL::ScissorRect Scissor[ML_MaxViewports];
	
	uint32 ActiveViewports;
	uint32 ActiveScissors;
	
	FRHIRenderPassInfo RenderPassInfo;
	FTextureRHIRef ColorTargets[MaxSimultaneousRenderTargets];
	FTextureRHIRef ResolveTargets[MaxSimultaneousRenderTargets];
	FTextureRHIRef DepthStencilSurface;
	FTextureRHIRef DepthStencilResolve;

    MTL::RenderPassDescriptor* RenderPassDesc;
	uint32 RasterBits;
    uint8 PipelineBits;
	bool bIsRenderTargetActive;
	bool bHasValidRenderTarget;
	bool bHasValidColorTarget;
	bool bIsParallelContext = false;
};

class FMetalRenderPassDescriptorPool
{
public:
	FMetalRenderPassDescriptorPool()
	{}
	
	~FMetalRenderPassDescriptorPool()
	{}
	
	MTL::RenderPassDescriptor* CreateDescriptor()
	{
		MTL::RenderPassDescriptor* Desc = Cache.Pop();
		if (!Desc)
		{
			Desc = MTL::RenderPassDescriptor::alloc()->init();
			check(Desc);
		}
		return Desc;
	}
	
	void ReleaseDescriptor(MTL::RenderPassDescriptor* Desc)
	{
		MTL::RenderPassColorAttachmentDescriptorArray* Attachments = Desc->colorAttachments();
		for (uint32 i = 0; i < MaxSimultaneousRenderTargets; i++)
		{
			MTL::RenderPassColorAttachmentDescriptor* Color = Attachments->object(i);
			Color->setTexture(nullptr);
			Color->setResolveTexture(nullptr);
			Color->setStoreAction(MTL::StoreActionStore);
		}
		
		MTL::RenderPassDepthAttachmentDescriptor* Depth = Desc->depthAttachment();
		Depth->setTexture(nullptr);
		Depth->setResolveTexture(nullptr);
		Depth->setStoreAction(MTL::StoreActionStore);

		MTL::RenderPassStencilAttachmentDescriptor* Stencil = Desc->stencilAttachment();
		Stencil->setTexture(nullptr);
		Stencil->setResolveTexture(nullptr);
		Stencil->setStoreAction(MTL::StoreActionStore);

		Desc->setVisibilityResultBuffer(nullptr);
		
#if PLATFORM_MAC
		Desc->setRenderTargetArrayLength(1);
#endif
		
		Cache.Push(Desc);
	}
	
	static FMetalRenderPassDescriptorPool& Get()
	{
		static FMetalRenderPassDescriptorPool sSelf;
		return sSelf;
	}
	
private:
	TLockFreePointerListLIFO<MTL::RenderPassDescriptor> Cache;
};
