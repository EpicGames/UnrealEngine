// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalStaticSamplers.cpp: Metal static samplers for bindless
=============================================================================*/

#include "MetalStaticSamplers.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

static MTL::SamplerDescriptor* MakeStaticSampler(MTL::SamplerMinMagFilter MinFilter, MTL::SamplerMinMagFilter MaxFilter, MTL::SamplerMipFilter MipFilter, MTL::SamplerAddressMode WrapMode)
{
	MTL::SamplerDescriptor* Desc = MTL::SamplerDescriptor::alloc()->init();
	Desc->setMinFilter(MinFilter);
	Desc->setMagFilter(MaxFilter);
	Desc->setMipFilter(MipFilter);
	Desc->setSAddressMode(WrapMode);
	Desc->setTAddressMode(WrapMode);
	Desc->setRAddressMode(WrapMode);
	Desc->setMaxAnisotropy(1);
	Desc->setCompareFunction(MTL::CompareFunctionNever);
	Desc->setBorderColor(MTL::SamplerBorderColorTransparentBlack);
	Desc->setLodMinClamp(0.0f);
	Desc->setLodMaxClamp(FLT_MAX);
	Desc->setSupportArgumentBuffers(true);

	return Desc;
}

FMetalStaticSamplers::FMetalStaticSamplers(FMetalDevice& Device)
	: StaticSamplersTable(nullptr)
{
	MetalStaticSamplerDescs[0] = MakeStaticSampler(MTL::SamplerMinMagFilterNearest, MTL::SamplerMinMagFilterNearest, MTL::SamplerMipFilterNearest, MTL::SamplerAddressModeRepeat);
	MetalStaticSamplerDescs[1] = MakeStaticSampler(MTL::SamplerMinMagFilterNearest, MTL::SamplerMinMagFilterNearest, MTL::SamplerMipFilterNearest, MTL::SamplerAddressModeClampToEdge);

	MetalStaticSamplerDescs[2] = MakeStaticSampler(MTL::SamplerMinMagFilterLinear, MTL::SamplerMinMagFilterLinear, MTL::SamplerMipFilterNearest, MTL::SamplerAddressModeRepeat);
	MetalStaticSamplerDescs[3] = MakeStaticSampler(MTL::SamplerMinMagFilterLinear, MTL::SamplerMinMagFilterLinear, MTL::SamplerMipFilterNearest, MTL::SamplerAddressModeClampToEdge);

	MetalStaticSamplerDescs[4] = MakeStaticSampler(MTL::SamplerMinMagFilterLinear, MTL::SamplerMinMagFilterLinear, MTL::SamplerMipFilterLinear, MTL::SamplerAddressModeRepeat);
	MetalStaticSamplerDescs[5] = MakeStaticSampler(MTL::SamplerMinMagFilterLinear, MTL::SamplerMinMagFilterLinear, MTL::SamplerMipFilterLinear, MTL::SamplerAddressModeClampToEdge);
	
	IRDescriptorTableEntry SamplerTableContent[MAX_STATIC_SAMPLERS];
	for (uint32 Idx = 0; Idx < MAX_STATIC_SAMPLERS; Idx++)
	{
		MTL::SamplerState* SamplerState = Device.GetDevice()->newSamplerState(MetalStaticSamplerDescs[Idx]);
		check(SamplerState);
		StaticSamplers.Add(SamplerState);

		IRDescriptorTableSetSampler(&SamplerTableContent[Idx], SamplerState, 0.0f);
	}

	constexpr uint64 TableSize = MAX_STATIC_SAMPLERS * sizeof(IRDescriptorTableEntry);

	StaticSamplersTable = Device.CreatePooledBuffer(FMetalPooledBufferArgs(&Device, TableSize, BUF_Static, MTL::StorageModeShared));
	
	memcpy(StaticSamplersTable->Contents(), SamplerTableContent, TableSize);
}

FMetalStaticSamplers::~FMetalStaticSamplers()
{
	for (uint32 Idx = 0; Idx < MAX_STATIC_SAMPLERS; Idx++)
	{
		FMetalDynamicRHI::Get().DeferredDelete(MetalStaticSamplerDescs[Idx]);
		FMetalDynamicRHI::Get().DeferredDelete(StaticSamplers[Idx]);	
	}
	
	FMetalDynamicRHI::Get().DeferredDelete(StaticSamplersTable);
}

uint64 FMetalStaticSamplers::GetGPUAddress() const
{
	return StaticSamplersTable->GetGPUAddress();
}

#endif
