// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalStaticSamplers.h: Metal static samplers for bindless
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING

#include "MetalRHI.h"
#include "MetalThirdParty.h"
#include "MetalResources.h"
#include "MetalDevice.h"
#include "MetalShaderResources.h"

class FMetalStaticSamplers
{
public:
	FMetalStaticSamplers(FMetalDevice& Device);
	~FMetalStaticSamplers();

	uint64 GetGPUAddress() const;

private:
	static const uint32 MAX_STATIC_SAMPLERS = 6;

	// Static sampler table must match MetalCommon.ush and MetalMSC.cpp
	MTL::SamplerDescriptor* MetalStaticSamplerDescs[MAX_STATIC_SAMPLERS];
	
	TArray<MTL::SamplerState*>      StaticSamplers;
	FMetalBufferPtr					StaticSamplersTable;
};

#endif
