// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalSubmission.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "MetalCommandBuffer.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IRenderCaptureProvider.h"
#include "MetalProfiler.h"

static TAutoConsoleVariable<int32> CVarMetalRHIUseInterruptThread(
	TEXT("MetalRHI.UseInterruptThread"),
	1,
	TEXT("Whether to enable the Metal RHI's interrupt thread.\n")
	TEXT("  0: No\n")
	TEXT("  1: Yes\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMetalRHIUseSubmissionThread(
	TEXT("MetalRHI.UseSubmissionThread"),
	1,
	TEXT("Whether to enable the Metal RHI's submission thread.\n")
	TEXT("  0: No\n")
	TEXT("  1: Yes\n"),
	ECVF_ReadOnly);

#define METAL_USE_INTERRUPT_THREAD 1
#define METAL_USE_SUBMISSION_THREAD 1

class FMetalThread final : private FRunnable
{
public:
	typedef FMetalDynamicRHI::FProcessResult(FMetalDynamicRHI::*FQueueFunc)();

	FMetalThread(TCHAR const* Name, EThreadPriority Priority, FMetalDynamicRHI* RHI, FQueueFunc QueueFunc)
		: RHI(RHI)
		, QueueFunc(QueueFunc)
		, Event(FPlatformProcess::GetSynchEventFromPool(true))
		, Thread(FRunnableThread::Create(this, Name, 0, Priority))
	{}

	virtual ~FMetalThread()
	{
		bExit = true;

		Event->Trigger();
		
		Thread->WaitForCompletion();
		delete Thread;
	}

	void Kick() const
	{
		check(Event);
		Event->Trigger();
	}

	void Join() const
	{
		Thread->WaitForCompletion();
	}

private:
	virtual uint32 Run() override
	{
		while (!bExit)
		{
			// Process the queue until no more progress is made
			FMetalDynamicRHI::FProcessResult Result;
			do { Result = (RHI->*QueueFunc)(); }
			while (EnumHasAllFlags(Result.Status, FMetalDynamicRHI::EQueueStatus::Processed));
	
			Event->Wait(1);
			Event->Reset();
		}

		// Drain any remaining work in the queue
		while (EnumHasAllFlags((RHI->*QueueFunc)().Status, FMetalDynamicRHI::EQueueStatus::Pending)) {}

		return 0;
	}

	FMetalDynamicRHI* RHI;
	FQueueFunc QueueFunc;
	FEvent* Event;
	bool bExit = false;
	
private:
	FRunnableThread* Thread = nullptr;
};

void FMetalDynamicRHI::InitializeSubmissionPipe()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
#if METAL_USE_INTERRUPT_THREAD
		bool bUseInterruptThread = CVarMetalRHIUseInterruptThread.GetValueOnAnyThread() == 1;
		if (bUseInterruptThread)
		{
			InterruptThread = new FMetalThread(TEXT("RHIInterruptThread"), TPri_Highest, this, &FMetalDynamicRHI::ProcessInterruptQueue);
		}
#endif

#if METAL_USE_SUBMISSION_THREAD
		bool bUseSubmissionThread = false;
		switch (CVarMetalRHIUseSubmissionThread.GetValueOnAnyThread())
		{
		case 1: bUseSubmissionThread = true; break;
		}
		
		if (bUseSubmissionThread)
		{
			SubmissionThread = new FMetalThread(TEXT("RHISubmissionThread"), TPri_Highest, this, &FMetalDynamicRHI::ProcessSubmissionQueue);
		}
#endif
	}

	// Initialize the timing structs in each queue, and the engine GPU profilers
	{
		TArray<FMetalPayload*> Payloads;
		TArray<UE::RHI::GPUProfiler::FQueue> ProfilerQueues;
		
		ForEachQueue([&](FMetalCommandQueue& Queue)
		{			
			FMetalPayload* Payload = Payloads.Emplace_GetRef(new FMetalPayload(Queue));
			Payload->Timing = CurrentTimingPerQueue.CreateNew(Queue);

			ProfilerQueues.Add(Queue.GetProfilerQueue());
		});

		UE::RHI::GPUProfiler::InitializeQueues(ProfilerQueues);
	
		SubmitPayloads(MoveTemp(Payloads));
	}
}

void FMetalDynamicRHI::ShutdownSubmissionPipe()
{
	delete SubmissionThread;
	SubmissionThread = nullptr;

	delete InterruptThread;
	InterruptThread = nullptr;

	if (EopTask)
	{
		ProcessInterruptQueueUntil(EopTask);
		EopTask = nullptr;
	}
}

static TLockFreePointerListUnordered<FMetalRHIUploadContext, PLATFORM_CACHE_LINE_SIZE> MetalUploadContextPool;

IRHIUploadContext* FMetalDynamicRHI::RHIGetUploadContext()
{
	FMetalRHIUploadContext* Context = MetalUploadContextPool.Pop();
	if (!Context)
	{
		Context = new FMetalRHIUploadContext(*Device);
	}
	
	return static_cast<IRHIUploadContext*>(Context);
}

FDynamicRHI::FRHIFinalizeContextsResult FMetalDynamicRHI::RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	FRHIFinalizeContextsResult Result;
	
	if (FMetalRHIUploadContext* UploadContext = static_cast<FMetalRHIUploadContext*>(Args.UploadContext))
	{
		FMetalFinalizedCommands Commands;
		UploadContext->Finalize(Commands);
		MetalUploadContextPool.Push(UploadContext);

		if (Commands.Num())
		{
			Result.UploadCommandList = new FMetalFinalizedCommands(MoveTemp(Commands));
		}
	}
	
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		if (FMetalRHICommandContext* Context = static_cast<FMetalRHICommandContext*>(Args.Contexts[Pipeline]))
		{
			FMetalFinalizedCommands Commands;
			Context->Finalize(Commands);
			Context->ResetContext();

			MetalCommandContextPool.Push(Context);
		
			Result.CommandLists[Pipeline] = new FMetalFinalizedCommands(MoveTemp(Commands));
		}
	}

	return Result;
}

void FMetalDynamicRHI::RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args)
{
	SubmitCommands(MakeArrayView(reinterpret_cast<FMetalFinalizedCommands**>(Args.CommandLists.GetData()), Args.CommandLists.Num()));
}

void FMetalDynamicRHI::SubmitCommands(TConstArrayView<FMetalFinalizedCommands*> Commands)
{	
	SCOPED_NAMED_EVENT_TEXT("CommandList_Submit", FColor::Magenta);

	TArray<FMetalPayload*> AllPayloads;
	for (FMetalFinalizedCommands* Payloads : Commands)
	{
	#if WITH_RHI_BREADCRUMBS
		TSharedPtr<FRHIBreadcrumbAllocatorArray> BreadcrumbAllocators {};
		if (Payloads->BreadcrumbAllocators.Num())
		{
			BreadcrumbAllocators = MakeShared<FRHIBreadcrumbAllocatorArray>(MoveTemp(Payloads->BreadcrumbAllocators));
		}

		for (FMetalPayload* Payload : *Payloads)
		{
			Payload->BreadcrumbRange = Payloads->BreadcrumbRange;
			if (BreadcrumbAllocators.IsValid())
			{
				check(!Payload->BreadcrumbAllocators.IsValid());
				Payload->BreadcrumbAllocators = BreadcrumbAllocators;
			}
		}
	#endif

		AllPayloads.Append(MoveTemp(*Payloads));
		delete Payloads;
	}

	SubmitPayloads(MoveTemp(AllPayloads));
}

void FMetalDynamicRHI::SubmitPayloads(TArray<FMetalPayload*>&& Payloads)
{
	if (Payloads.Num())
	{
		PendingPayloadsForSubmission.Enqueue(new TArray<FMetalPayload*>(MoveTemp(Payloads)));
	}

	if (SubmissionThread)
	{
		SubmissionThread->Kick();
	}
	else
	{
		// Since we're processing directly on the calling thread, we need to take a scope lock.
		// Multiple engine threads might be calling Submit().
		{
			FScopeLock Lock(&SubmissionCS);

			// Process the submission queue until no further progress is being made.
			while (EnumHasAnyFlags(ProcessSubmissionQueue().Status, EQueueStatus::Processed)) {}
		}
	}

	// Use this opportunity to pump the interrupt queue
	ProcessInterruptQueueUntil(nullptr);
}

static int32 GetMaxExecuteBatchSize()
{
	return
#if UE_BUILD_DEBUG
		GRHIGlobals.IsDebugLayerEnabled ? 1 :
#endif
		TNumericLimits<int32>::Max();
}

FMetalDynamicRHI::FProcessResult FMetalDynamicRHI::ProcessSubmissionQueue()
{
	SCOPED_NAMED_EVENT_TEXT("SubmissionQueue_Process", FColor::Turquoise);
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ProcessSubmissionQueue"));

	FMetalCommandQueue::FPayloadArray PayloadsToHandDown;
	FProcessResult Result;

	auto FlushPayloads = [&PayloadsToHandDown, &Result, DynamicRHI = this]()
	{
		if (PayloadsToHandDown.Num() > 0)
		{
			Result.Status |= EQueueStatus::Processed;
			DynamicRHI->FlushBatchedPayloads(PayloadsToHandDown);
		}
	};

	bool bProgress;
	bool bKickInterruptThread = false;

	do
	{
		bProgress = false;
		EnumRemoveFlags(Result.Status, EQueueStatus::Pending);

		// Push all pending payloads into the ordered per-device, per-pipe pending queues
		{
			TArray<FMetalPayload*>* Array;
			while (PendingPayloadsForSubmission.Dequeue(Array))
			{
				for (FMetalPayload* Payload : *Array)
				{
					Payload->Queue.PendingSubmission.Enqueue(Payload);
				}
				delete Array;
			}
		}

		//
		// Fence values for FMetalSyncPoint are determined on the submission thread,
		// where each queue has a monotonically incrementing fence value.
		//
		// We might receive work that waits on a sync point which has not yet been submitted
		// to the queue that will signal it, so we need to delay processing of those
		// payloads until the fence value is known.
		//

		// Process all queues (across all devices and adapters) to flush work.
		// Any sync point waits where the fence value is unknown will be left in the 
		// appropriate queue, to be processed the next time commands are submitted.
		ForEachQueue([&](FMetalCommandQueue& CurrentQueue)
		{
			while (true)
			{
				TArray<FMetalCommandQueue*, TInlineAllocator<GMetalMaxNumQueues>> QueuesWithPayloads;
				{
					FMetalPayload* Payload = CurrentQueue.PendingSubmission.Peek();
					if (!Payload)
						return;

					// Accumulate the list of fences to await, and their maximum values
					while (Payload->SyncPointsToWait.Index < Payload->SyncPointsToWait.Num())
					{
						FMetalSyncPointRef& SyncPoint = Payload->SyncPointsToWait[Payload->SyncPointsToWait.Index];
						if (!SyncPoint->ResolvedFence.IsSet())
						{
							// Need to wait on a sync point, but the fence value has not been resolved yet
							// (no other payloads have signaled the sync point yet).
						
							// Skip processing this queue, and move on to the next. We will retry later when
							// further work is submitted, which may contain the sync point we need.
							Result.Status |= EQueueStatus::Pending;
							return;
						}

						Payload->AddQueueFenceWait(
							SyncPoint->ResolvedFence->Fence,
							SyncPoint->ResolvedFence->Value
						);

						Payload->SyncPointsToWait.Index++;
						bProgress = true;
					}
				
					// All necessary sync points have been resolved.
					Payload->SyncPointsToWait = {};
					CurrentQueue.PendingSubmission.Pop();
					bProgress = true;

					check(!CurrentQueue.PayloadToSubmit);
					CurrentQueue.PayloadToSubmit = Payload;
					QueuesWithPayloads.Add(&CurrentQueue);
					Result.Status |= EQueueStatus::Processed;
					bKickInterruptThread = true;

					for (int32 Index = 0; Index < Payload->CommandBuffersToExecute.Num(); Index++)
					{
						FMetalCommandBuffer* CurrentCommandBuffer = Payload->CommandBuffersToExecute[Index];
						
#if !UE_BUILD_SHIPPING
						MTL::CommandBuffer* MTLCmdBuffer = CurrentCommandBuffer->GetMTLCmdBuffer();
						
						id<MTLLogContainer> logs = (__bridge id<MTLLogContainer>)MTLCmdBuffer->logs();
						for (id<MTLFunctionLog> log : logs)
						{
							MTL::FunctionLog* FunctionLog = (__bridge MTL::FunctionLog*)log;
							if(FunctionLog)
							{
								MTL::FunctionLogDebugLocation* DebugLocation = FunctionLog->debugLocation();
								
								if(DebugLocation)
								{
									UE_LOGF(LogMetal, Warning, "Shader Validation Log:\n functionName: %ls\nencoder: %ls\nline: %lld\ncolumn: %lld", *NSStringToFString(DebugLocation->functionName()), *NSStringToFString(FunctionLog->encoderLabel()), DebugLocation->line(), DebugLocation->column());
								}
							}
						}
#endif
						
						CurrentQueue.OcclusionQueries.Append(MoveTemp(CurrentCommandBuffer->OcclusionQueries));
						CurrentQueue.TimestampQueries.Append(MoveTemp(CurrentCommandBuffer->TimestampQueries));
					}
				}

				// Queues with work to submit other than the current one (CurrentQueue) are performing barrier operations.
				// Submit this work first, followed by a fence signal + enqueued wait.
				for (FMetalCommandQueue* OtherQueue : QueuesWithPayloads)
				{
					if (OtherQueue != &CurrentQueue)
					{
						uint64 ValueSignaled = OtherQueue->FinalizePayload(true, PayloadsToHandDown);
						CurrentQueue.PayloadToSubmit->AddQueueFenceWait(OtherQueue->GetSignalEvent(), ValueSignaled);
					}
					FlushPayloads();
				}

				// Now submit the original payload
				CurrentQueue.FinalizePayload(false, PayloadsToHandDown);
				FlushPayloads();
			}
		});
	} while (bProgress);

	FlushPayloads();

	if (InterruptThread && bKickInterruptThread)
	{
		InterruptThread->Kick();
	}

	return Result;
}

uint64 FMetalCommandQueue::FinalizePayload(bool bRequiresSignal, FPayloadArray& PayloadsToHandDown)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ExecuteCommandList);
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ExecuteCommandLists"));

	check(PayloadToSubmit && this == &PayloadToSubmit->Queue);
	check(PayloadToSubmit->SyncPointsToWait.Num() == 0);
	check(!PayloadToSubmit->SignalCommandBuffer);
	
	// Keep the latest fence value in the submitted payload.
	// The interrupt thread uses this to determine when work has completed.
	PayloadToSubmit->CompletionFenceValue = ++SignalEvent.NextCompletionValue;
	PayloadToSubmit->bAlwaysSignal |= bRequiresSignal;

	// Set the fence/value pair into any sync points we need to signal.
	for (FMetalSyncPointRef& SyncPoint : PayloadToSubmit->SyncPointsToSignal)
	{
		check(!SyncPoint->ResolvedFence.IsSet());
		SyncPoint->ResolvedFence.Emplace(SignalEvent, PayloadToSubmit->CompletionFenceValue);
	}

	PayloadToSubmit->BatchedObjects.OcclusionQueries = MoveTemp(OcclusionQueries);
	PayloadToSubmit->BatchedObjects.TimestampQueries = MoveTemp(TimestampQueries);
	
	PayloadsToHandDown.Add(PayloadToSubmit);
	PayloadToSubmit = nullptr;

	return SignalEvent.NextCompletionValue;
}

void FMetalDynamicRHI::FlushBatchedPayloads(FMetalCommandQueue::FPayloadArray& PayloadsToSubmit)
{
	uint32 FirstPayload = 0, LastPayload = 0;

	auto Wait = [this](FMetalPayload* Payload)
	{
		FMetalCommandQueue& Queue = Payload->Queue;

		// Wait for queue fences
		for (auto& [LocalFence, Value] : Payload->QueueFencesToWait)
		{
			FMetalCommandBuffer* CurrentCommandBuffer = Queue.CreateCommandBuffer();
			CurrentCommandBuffer->GetMTLCmdBuffer()->encodeWait(LocalFence.MetalEvent, Value);
			Queue.CommitCommandBuffer(CurrentCommandBuffer);
			
			DeferredDelete([CurrentCommandBuffer]() {
				delete CurrentCommandBuffer;
			});
		}
	};

	auto Flush = [&]()
	{
		if (FirstPayload == LastPayload)
			return;

		FMetalCommandQueue& Queue = PayloadsToSubmit[FirstPayload]->Queue;

		uint64 Time = FPlatformTime::Cycles64();

		TArray<FMetalCommandBuffer*> CommandBuffers;
		
		// Accumulate the command lists from the payload
		for (uint32 Index = FirstPayload; Index < LastPayload; ++Index)
		{
			FMetalPayload* Payload = PayloadsToSubmit[Index];
			check(&Payload->Queue == &Queue);

			for (FMetalCommandBuffer* CommandBuffer : Payload->CommandBuffersToExecute)
			{
				CommandBuffer->FlushProfilerEvents(Payload->EventStream, Time);
				Payload->Queue.CommitCommandBuffer(CommandBuffer);
			}
		}

		FirstPayload = LastPayload;
	};

	auto Signal = [this](FMetalPayload* Payload)
	{
		FMetalCommandQueue& Queue = Payload->Queue;

		// Signal the queue fence
		if (Payload->RequiresQueueFenceSignal())
		{
			check(Queue.GetSignalEvent().LastSignaledValue < Payload->CompletionFenceValue);

			FMetalCommandBuffer* CommandBuffer = Queue.CreateCommandBuffer();
			
			MTL::HandlerFunction CompletionHandler = [this](MTL::CommandBuffer* CompletedBuffer)
			{
				if(InterruptThread)
				{
					InterruptThread->Kick();
				}
			};
			
			CommandBuffer->GetMTLCmdBuffer()->addCompletedHandler(CompletionHandler);
			CommandBuffer->GetMTLCmdBuffer()->encodeSignalEvent(Queue.GetSignalEvent().MetalEvent, Payload->CompletionFenceValue);
			Payload->SignalCommandBuffer = CommandBuffer;
		
			Queue.CommitCommandBuffer(CommandBuffer);
			Queue.GetSignalEvent().LastSignaledValue.store(Payload->CompletionFenceValue, std::memory_order_release);
		}

		if (Payload->EndFrameEvent.IsSet())
		{
			Payload->EndFrameEvent->CPUTimestamp = FPlatformTime::Cycles64();
			Payload->EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(*Payload->EndFrameEvent);
		}

		// Submission of this payload is completed. Signal the submission event if one was provided.
		if (Payload->SubmissionEvent)
		{
			Payload->SubmissionEvent->DispatchSubsequents();
		}
	};

	FMetalCommandQueue* PrevQueue = nullptr;
	for (FMetalPayload* Payload : PayloadsToSubmit)
	{
		if (PrevQueue != &Payload->Queue)
		{
			Flush();
			PrevQueue = &Payload->Queue;
		}

		Payload->Queue.PendingInterrupt.Enqueue(Payload);

		if (Payload->HasWaitWork())
		{
			Flush();
			Wait(Payload);
		}

		if (Payload->HasPreExecuteWork())
		{
			Flush();
			Payload->PreExecute();
		}

		LastPayload++;

		if (Payload->HasSignalWork())
		{
			Flush();
			Signal(Payload);
		}
	}

	Flush();

	for (FMetalPayload* Payload : PayloadsToSubmit)
	{
		// Only set this bool to true once we'll never touch the payload again on this thread.
		// This is because the bool hands ownership to the interrupt thread, which might delete the payload.
		Payload->bSubmitted = true;
	}

	PayloadsToSubmit.Reset();	
}

void FMetalPayload::AddQueueFenceWait(FMetalSignalEvent& InFence, uint64 InValue)
{
	for (auto& [Fence, Value] : QueueFencesToWait)
	{
		if (&Fence == &InFence)
		{
			Value = FMath::Max(Value, InValue);
			return;
		}
	}

	QueueFencesToWait.Add({ InFence, InValue });
}

void FMetalSyncPoint::Wait() const
{
	checkf(GraphEvent, TEXT("This sync point was not created with a CPU event. Cannot wait for completion on the CPU."));

	if (!GraphEvent->IsComplete())
	{
		// Block the calling thread until the graph event is signaled by the interrupt thread.
		SCOPED_NAMED_EVENT_TEXT("SyncPoint_Wait", FColor::Turquoise);
		FMetalDynamicRHI::Get().ProcessInterruptQueueUntil(GraphEvent);
	}

	check(GraphEvent->IsComplete());
}

void FMetalDynamicRHI::ProcessInterruptQueueUntil(FGraphEvent* GraphEvent)
{
	if (InterruptThread)
	{
		if (GraphEvent && !GraphEvent->IsComplete())
		{
			InterruptThread->Kick();
			GraphEvent->Wait();
		}
	}
	else
	{
		// Use the current thread to process the interrupt queue until the sync point we're waiting for is signaled.
		// If GraphEvent is nullptr, process the queue until no further progress is made (assuming we can acquire the lock), then return.
		if (!GraphEvent || !GraphEvent->IsComplete())
		{
			// If we're waiting for a sync point, accumulate the idle time
			UE::Stats::FThreadIdleStats::FScopeIdle IdleScope(GraphEvent == nullptr);

		Retry:
			if (InterruptCS.TryLock())
			{
				FProcessResult Result;
				do { Result = ProcessInterruptQueue(); }
				// If we have a sync point, keep processing until the sync point is signaled.
				// Otherwise, process until no more progress is being made.
				while (GraphEvent
					? !GraphEvent->IsComplete()
					: EnumHasAllFlags(Result.Status, EQueueStatus::Processed)
				);

				InterruptCS.Unlock();
			}
			else if (GraphEvent && !GraphEvent->IsComplete())
			{
				// Failed to get the lock. Another thread is processing the interrupt queue. Try again...
				FPlatformProcess::SleepNoStats(0);
				goto Retry;
			}
		}
	}
}

void FMetalDynamicRHI::ProcessPendingCommandBuffers(TArrayView<FMetalCommandBuffer*> CommandBuffers)
{
	double SecondsPerCycle = FPlatformTime::GetSecondsPerCycle64();

	for (FMetalCommandBuffer* CommandBuffer : CommandBuffers)
	{
		MTL::CommandBuffer* CompletedBuffer = CommandBuffer->GetMTLCmdBuffer();

#if WITH_RHI_BREADCRUMBS
		FMetalBreadcrumbProfiler::GetInstance()->ResolveBreadcrumbTrackerStream(CommandBuffer);
#endif

		uint64_t* Start = CommandBuffer->BeginWorkTimestamp;
		uint64_t* End = CommandBuffer->EndWorkTimestamp;

		if(CommandBuffer->CounterSamples.Num())
		{
			for(FMetalCounterSamplePtr Sample : CommandBuffer->CounterSamples)
			{
				uint64_t StartTime, EndTime;
				Sample->ResolveStageCounters(StartTime, EndTime);

				*Start = *Start > 0 ? FMath::Min(StartTime, *Start) : StartTime;
				*End = *End > 0 ? FMath::Max(EndTime, *End) : EndTime;
			}
		}
		else
		{
			*Start = CompletedBuffer->GPUStartTime() / SecondsPerCycle;
			*End = CompletedBuffer->GPUEndTime() / SecondsPerCycle;
		}

		delete CommandBuffer;
	}
}

FMetalDynamicRHI::FProcessResult FMetalDynamicRHI::ProcessInterruptQueue()
{
	SCOPED_NAMED_EVENT_TEXT("InterruptQueue_Process", FColor::Yellow);
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/ProcessInterruptQueue"));

	// Timer that clamps each tick to prevent false positive GPU timeouts
	// when a debugger is attached and the process is broken.
	struct FTimer
	{
		uint64 Elapsed;
		uint64 Last;

		FTimer()
			: Elapsed(0)
			, Last(FPlatformTime::Cycles64())
		{}

		void Tick()
		{
			static const uint64 MaxDeltaCycles = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64()); // 1 second
			uint64 Current = FPlatformTime::Cycles64();
			Elapsed += FMath::Min(MaxDeltaCycles, Current - Last);
			Last = Current;
		}
	} static Timer;

	Timer.Tick();

	FProcessResult Result;
	ForEachQueue([&](FMetalCommandQueue& CurrentQueue)
	{
		while (FMetalPayload* Payload = CurrentQueue.PendingInterrupt.Peek())
		{
			if (!Payload->bSubmitted)
				break;

			// Check for GPU completion
			FMetalSignalEvent& CurrentEvent = CurrentQueue.GetSignalEvent();

			uint64 LastSignaledFenceValue = CurrentEvent.LastSignaledValue.load(std::memory_order_acquire);
			
			// Handle command buffer errors and check that every work command buffer has finished on the GPU. 
			bool bWorkCommandBuffersFinished = true;
			for(FMetalCommandBuffer* CommandBuffer : Payload->CommandBuffersToExecute)
			{
				MTL::CommandBuffer* CompletedBuffer = CommandBuffer->GetMTLCmdBuffer();
				MTL::CommandBufferStatus CBStatus = CompletedBuffer->status();
				if (CBStatus == MTL::CommandBufferStatusError)
				{
					FMetalCommandQueue::HandleMetalCommandBufferFailure(CompletedBuffer);
				}
				else if (CBStatus != MTL::CommandBufferStatusCompleted)
				{
					bWorkCommandBuffersFinished = false;
				}
			}

			MTL::CommandBufferStatus Status = MTL::CommandBufferStatusCompleted;

			if(Payload->SignalCommandBuffer)
			{
				MTL::CommandBuffer* SignalBuffer = Payload->SignalCommandBuffer->GetMTLCmdBuffer();
				Status = SignalBuffer->status();
				if(Status == MTL::CommandBufferStatusError)
				{
					FMetalCommandQueue::HandleMetalCommandBufferFailure(SignalBuffer);
				}
			}

			// Remove Completed status check when we remove completion handlers
			bool bCommandBufferFinished = bWorkCommandBuffersFinished
				&& (Status == MTL::CommandBufferStatusCompleted || Status == MTL::CommandBufferStatusError);
			if (!bCommandBufferFinished)
			{
				// Skip processing this queue and move on to the next. Retried on the next interrupt
				// tick (the thread self-wakes on a 1ms timeout) once the work has completed.
				Result.Status |= EQueueStatus::Pending;
				break;
			}

			if(Payload->SignalCommandBuffer)
			{
				delete Payload->SignalCommandBuffer;
			}

			// Resolve query results
			{
				for (FMetalRHIRenderQuery* OcclusionQuery : Payload->BatchedObjects.OcclusionQueries)
				{
					OcclusionQuery->SampleOcclusionResult();
				}
				
				for (FMetalRHIRenderQuery* TimestampQuery : Payload->BatchedObjects.TimestampQueries)
				{
					MTL::CommandBuffer* CmdBuffer = TimestampQuery->CommandBuffer->GetMTLCmdBuffer();
					
					// If there are no commands in the command buffer then this can be zero
					// In this case GPU start time is also not correct - we need to fall back standard behaviour
					// Only seen empty command buffers at the very end of a frame
					
					// Convert seconds to microseconds
					TimestampQuery->Result = uint64(CmdBuffer->GPUEndTime()) * 1000000;
					
					if(TimestampQuery->Result == 0)
					{
						TimestampQuery->Result = (FPlatformTime::ToMilliseconds64(mach_absolute_time()) * 1000.0);
					}
					
					TimestampQuery->Release();
				}
			}

			// Resolve breadcrumb/work timestamps into the payload's EventStream chunks before they are
			// appended to Queue.Timing, so no write targets a chunk ProcessEvents has already freed.
			ProcessPendingCommandBuffers(Payload->CommandBuffersToExecute);

			if (!Payload->EventStream.IsEmpty())
			{
				check(CurrentQueue.Timing);
				CurrentQueue.Timing->EventStream.Append(MoveTemp(Payload->EventStream));
			}

			if (Payload->Timing.IsSet())
			{
				// Switch the new timing struct into the queue. This redirects timestamp results to separate each frame's work.
				CurrentQueue.Timing = Payload->Timing.GetValue();
			}

			// Signal the CPU events of all sync points associated with this batch.
			for (FMetalSyncPointRef& SyncPoint : Payload->SyncPointsToSignal)
			{
				if (SyncPoint->GraphEvent)
				{
					SyncPoint->GraphEvent->DispatchSubsequents();
				}
			}
			
			// At this point, the current command list has completed on the GPU.
			CurrentQueue.PendingInterrupt.Pop();
			Result.Status |= EQueueStatus::Processed;
			
			// GPU resources the payload is holding a reference to will be cleaned up here.
			// E.g. command list allocators, which get recycled on the parent device.
			delete Payload;
		}
	});

	return Result;
}

FMetalPayload::FMetalPayload(FMetalCommandQueue& Queue)
	: Queue(Queue)
	, EventStream(Queue.GetProfilerQueue())
{}

FMetalPayload::~FMetalPayload()
{
}

void FMetalPayload::PreExecute()
{
	if (PreExecuteCallback)
	{
		PreExecuteCallback(Queue.GetQueue());
	}
}
