// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "MetalProfiler.h"
#include "RHIBreadcrumbs.h"
#include "GPUProfiler.h"
#include "MetalThirdParty.h"

static constexpr uint32 GMetalMaxNumQueues = 1;

class FMetalSyncPoint;
class FMetalCommandQueue;
class FMetalCommandBuffer;
class FMetalDynamicRHI;
class FMetalRHICommandContext;

using FMetalSyncPointRef = TRefCountPtr<FMetalSyncPoint>;

enum class EMetalSyncPointType
{
	// Sync points of this type do not include an FGraphEvent, so cannot
	// report completion to the CPU (via either IsComplete() or Wait())
	GPUOnly,

	// Sync points of this type include an FGraphEvent. The IsComplete() and Wait() functions
	// can be used to poll for completion from the CPU, or block the CPU, respectively.
	GPUAndCPU,
};

// Fence type used by the device queues to manage GPU completion
struct FMetalSignalEvent
{
	MTL::Event* MetalEvent;
	uint64 NextCompletionValue = 0;
	std::atomic<uint64> LastSignaledValue = 0;
};

// Used by FMetalSyncPoint and the submission thread to fix up signaled fence values at the end-of-pipe
struct FMetalResolvedFence
{
	FMetalSignalEvent& Fence;
	uint64 Value = 0;

	FMetalResolvedFence(FMetalSignalEvent& Fence, uint64 Value)
		: Fence(Fence)
		, Value(Value)
	{}
};

//
// A sync point is a logical point on a GPU queue's timeline that can be awaited by other queues, or the CPU.
// These are used throughout the RHI as a way to abstract the underlying Metal fences. The submission thread 
// manages the underlying fences and signaled values, and reports completion to the relevant sync points via 
// an FGraphEvent.
//
// Sync points are one-shot, meaning they represent a single timeline point, and are released after use, via ref-counting.
// Use FMetalSyncPoint::Create() to make a new sync point and hold a reference to it via a FMetalSyncPointRef object.
//
class FMetalSyncPoint final : public FRefCountedObject
{
	friend FMetalDynamicRHI;
	friend FMetalCommandQueue;
	friend FMetalRHICommandContext;
	
	// No copying or moving
	FMetalSyncPoint(FMetalSyncPoint const&) = delete;
	FMetalSyncPoint(FMetalSyncPoint&&) = delete;

	TOptional<FMetalResolvedFence> ResolvedFence;
	FGraphEventRef GraphEvent;

	FMetalSyncPoint(EMetalSyncPointType Type)
	{
		if (Type == EMetalSyncPointType::GPUAndCPU)
		{
			GraphEvent = FGraphEvent::CreateGraphEvent();
		}
	}

public:
	static FMetalSyncPointRef Create(EMetalSyncPointType Type)
	{
		LLM_SCOPE_BYNAME(TEXT("RHIMisc/CreateSyncPoint"));
		return new FMetalSyncPoint(Type);
	}

	bool IsComplete() const
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event. Cannot check completion on the CPU."));
		return GraphEvent->IsComplete();
	}

	void Wait() const;
	
	void OnCompletionCallback(TUniqueFunction<void()>&& Func)
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event."));

		FGraphEventRef CompletionFence = FFunctionGraphTask::CreateAndDispatchWhenReady([Callback=MoveTemp(Func)]()
		{
			Callback();
		}, TStatId(), GraphEvent, ENamedThreads::AnyThread);
	}

	FGraphEvent* GetGraphEvent() const
	{
		checkf(GraphEvent, TEXT("This sync point was not created with a CPU event."));
		return GraphEvent;
	}

	EMetalSyncPointType GetType() const
	{
		return GraphEvent != nullptr
			? EMetalSyncPointType::GPUAndCPU
			: EMetalSyncPointType::GPUOnly;
	}
};

class FMetalRHIRenderQuery;

struct FMetalBatchedPayloadObjects
{
	TArray<FMetalRHIRenderQuery*> OcclusionQueries;
	TArray<FMetalRHIRenderQuery*> TimestampQueries;

	bool IsEmpty() const
	{
		return TimestampQueries.Num() == 0 && OcclusionQueries.Num() == 0;
	}
};

// Base class to avoid 8 bytes of padding after the vtable
struct FMetalPayloadBaseFixLayout
{
	virtual ~FMetalPayloadBaseFixLayout() = default;
};

class FMetalTiming;

// A single unit of work (specific to a single GPU node and queue type) to be processed by the submission thread.
struct FMetalPayload : public FMetalPayloadBaseFixLayout
{
	friend FMetalDynamicRHI;
	friend FMetalCommandQueue;
	friend FMetalRHICommandContext;
	
	// Constants
	FMetalCommandQueue& Queue;

	// Wait
	struct : public TArray<FMetalSyncPointRef>
	{
		// Used to pause / resume iteration of the sync point array on the
		// submission thread when we find a sync point that is unresolved.
		int32 Index = 0;

	} SyncPointsToWait;

	struct FQueueFence
	{
		FMetalSignalEvent& Fence;
		uint64 Value;
	};
	TArray<FQueueFence, TInlineAllocator<GMetalMaxNumQueues>> QueueFencesToWait;

	void AddQueueFenceWait(FMetalSignalEvent& Fence, uint64 Value);

	// Flags.
	bool bAlwaysSignal = false;
	std::atomic<bool> bSubmitted { false };

	// Used by RHIRunOnQueue
	TFunction<void(MTL::CommandQueue*)> PreExecuteCallback;

	// Execute
	TArray<FMetalCommandBuffer*> CommandBuffersToExecute;
	FMetalCommandBuffer* SignalCommandBuffer = nullptr;

	// Signal
	TArray<FMetalSyncPointRef> SyncPointsToSignal;
	uint64 CompletionFenceValue = 0;

	FGraphEventRef SubmissionEvent;
	TOptional<uint64> SubmissionTime;

	FMetalBatchedPayloadObjects BatchedObjects;

#if WITH_RHI_BREADCRUMBS
	FRHIBreadcrumbRange BreadcrumbRange {};
	TSharedPtr<FRHIBreadcrumbAllocatorArray> BreadcrumbAllocators {};
#endif

	TOptional<FMetalTiming*> Timing;
	UE::RHI::GPUProfiler::FEventStream EventStream;
	TOptional<UE::RHI::GPUProfiler::FEvent::FFrameBoundary> EndFrameEvent;

	virtual ~FMetalPayload();

	virtual void PreExecute();

	virtual bool HasPreExecuteWork() const
	{
		return PreExecuteCallback != nullptr;
	}

	virtual bool RequiresQueueFenceSignal() const
	{
		return bAlwaysSignal || SyncPointsToSignal.Num() > 0 || HasPreExecuteWork();
	}

	virtual bool HasWaitWork() const
	{
		return QueueFencesToWait.Num() > 0;
	}

	virtual bool HasSignalWork() const
	{
		return RequiresQueueFenceSignal() || SubmissionEvent != nullptr;
	}

protected:
	FMetalPayload(FMetalCommandQueue&);
};
