// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalTempAllocator.h"
#include "MetalDevice.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "MetalProfiler.h"

FMetalTempAllocator::FMetalTempAllocator(FMetalDevice& InDevice, uint32_t InMinAllocationSize, uint32_t InTargetAllocationLimit, uint32_t InAlignment)
    : Device(InDevice),
	MinAllocationSize(InMinAllocationSize),
	TargetAllocationLimit(InTargetAllocationLimit),
	Alignment(InAlignment)
{
	TotalAllocationStat = GET_STATID(STAT_MetalTempAllocatorAllocatedMemory);
}

FMetalBufferPtr FMetalTempAllocator::Allocate(const uint32_t Size)
{
    FScopeLock lock(&AllocatorLock);

	uint32 AlignedSize = Align(Size, Alignment);
	
	FTempBufferInfo* TempBuffer = nullptr;
	for(FTempBufferInfo& Buffer : Buffers)
	{
		if(Buffer.Size - Buffer.Offset >= AlignedSize)
		{
			TempBuffer = &Buffer;
			break;
		}
	}
	
	if(TempBuffer == nullptr)
	{
		uint32_t BufferSize = FMath::Max((uint32_t)MinAllocationSize, AlignedSize);
		TempBuffer = &Buffers.AddDefaulted_GetRef();
								  
		TotalAllocated += BufferSize;
		INC_MEMORY_STAT_BY_FName(TotalAllocationStat.GetName(), BufferSize);
		
		TempBuffer->Offset = 0;
		TempBuffer->Size = BufferSize; 
		TempBuffer->Buffer = Device.GetDevice()->newBuffer(BufferSize, 
														MTL::ResourceCPUCacheModeWriteCombined |
														MTL::ResourceStorageModeShared);
		
		if(!TempBuffer->Buffer)
		{
			UE_LOGF(LogMetal, Fatal, "Failed to allocate MTL::Buffer in MetalTempAllocator::Allocate");
		}
	}
	
	FMetalBufferPtr Buffer = FMetalBufferPtr(new FMetalBuffer(TempBuffer->Buffer, NS::Range(TempBuffer->Offset, AlignedSize), this));
	TempBuffer->Offset += AlignedSize;
	
	if(!Buffer)
	{
		UE_LOGF(LogMetal, Fatal, "Failed to allocate FMetalBuffer in MetalTempAllocator::Allocate");
	}
	
	return Buffer;
}

void FMetalTempAllocator::Cleanup()
{
	FScopeLock lock(&AllocatorLock);
	
	TArray<FTempBufferInfo> OldBuffers;
	
	// Move all buffers that have been used in this window
	for(FTempBufferInfo TempBuffer : Buffers)
	{
		if(TempBuffer.Offset != 0)
		{
			TempBuffer.Offset = 0;
			OldBuffers.Add(TempBuffer);
		}
	}
	
	Buffers.RemoveAll([](const FTempBufferInfo& TempBuffer) { return TempBuffer.Offset != 0; });
	
	// Ensure that buffers are re-added to the pool when fences are complete (if we are below the target limit)
	FMetalDynamicRHI::Get().DeferredDelete([this, OldBuffers]()
	{
		FScopeLock lock(&AllocatorLock);
		for(const FTempBufferInfo& Buffer : OldBuffers)
		{
			if((TotalAllocated + Buffer.Size) <= TargetAllocationLimit)
			{
				Buffers.Add(Buffer);
			}
			else
			{
				Buffer.Buffer->setPurgeableState(MTL::PurgeableStateEmpty);
				Buffer.Buffer->release();
				TotalAllocated -= Buffer.Size;
				DEC_MEMORY_STAT_BY_FName(TotalAllocationStat.GetName(), Buffer.Size);
			}
		}
	});
}
