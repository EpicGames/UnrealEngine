// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 MetalTexture.cpp: Metal texture RHI implementation.
 =============================================================================*/

#include "MetalRHIPrivate.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalDynamicRHI.h"
#include "MetalProfiler.h" // for STAT_MetalTexturePageOffTime
#include "MetalCommandBuffer.h"
#include "RenderUtils.h"
#include "Containers/ResourceArray.h"
#include "Misc/ScopeRWLock.h"
#include "MetalLLM.h"
#include "RHILockTracker.h"
#include "RHICoreStats.h"
#include "RHICoreTextureInitializer.h"
#include "MetalBindlessDescriptors.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RHICoreTexture.h"
#include <CoreVideo/CVMetalTexture.h>

int32 GMetalMaxOutstandingAsyncTexUploads = 100 * 1024 * 1024;
FAutoConsoleVariableRef CVarMetalMaxOutstandingAsyncTexUploads(
															   TEXT("rhi.Metal.MaxOutstandingAsyncTexUploads"),
															   GMetalMaxOutstandingAsyncTexUploads,
															   TEXT("The maximum number of outstanding asynchronous texture uploads allowed to be pending in Metal. After the limit is reached the next upload will wait for all outstanding operations to complete and purge the waiting free-lists in order to reduce peak memory consumption. Defaults to 0 (infinite), set to a value > 0 limit the number."),
															   ECVF_ReadOnly|ECVF_RenderThreadSafe
															   );

int32 GMetalForceIOSTexturesShared = !WITH_IOS_SIMULATOR;
FAutoConsoleVariableRef CVarMetalForceIOSTexturesShared(
														TEXT("rhi.Metal.ForceIOSTexturesShared"),
														GMetalForceIOSTexturesShared,
														TEXT("If true, forces all textures to be Shared on iOS"),
														ECVF_RenderThreadSafe);

int32 GMetalDisableIOSMemoryless = 0;
FAutoConsoleVariableRef CVarMetalDisableIOSMemoryless(
													  TEXT("rhi.Metal.DisableIOSMemoryless"),
													  GMetalDisableIOSMemoryless,
													  TEXT("If true, disabled the use of Memoryless textures on iOS"),
													  ECVF_ReadOnly|ECVF_RenderThreadSafe);

bool ShouldUseMemoryless(ETextureCreateFlags Flags)
{
	constexpr bool bIsMobile = PLATFORM_IOS;
	const bool bWantsMemoryless = EnumHasAnyFlags(Flags, TexCreate_Memoryless);

	constexpr bool bIsVisionOS = PLATFORM_VISIONOS;
	const bool bIsDepth = EnumHasAnyFlags(Flags, TexCreate_DepthStencilTargetable | TexCreate_DepthStencilResolveTarget);
	static int MSAAMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.XRMSAAMode"))->GetValueOnAnyThread();

	// allow memoryless on mobile if not disabled
	return bIsMobile && bWantsMemoryless && !GMetalDisableIOSMemoryless && !(bIsVisionOS && bIsDepth && MSAAMode == 1);
}

bool AllowMSAA()
{
	static const bool bAllowMSAA = !FParse::Param(FCommandLine::Get(), TEXT("nomsaa"));	
	return bAllowMSAA;
}

/** Given a pointer to a RHI texture that was created by the Metal RHI, returns a pointer to the FMetalTextureBase it encapsulates. */
FMetalSurface* GetMetalSurfaceFromRHITexture(FRHITexture* Texture)
{
	return Texture
		? static_cast<FMetalSurface*>(Texture->GetTextureBaseRHI())
		: nullptr;
}

static bool IsRenderTarget(ETextureCreateFlags Flags)
{
	return EnumHasAnyFlags(Flags, TexCreate_RenderTargetable | TexCreate_ResolveTargetable | TexCreate_DepthStencilTargetable | TexCreate_DepthStencilResolveTarget);
}

static MTL::TextureUsage ConvertDescToUsage(const FRHITextureCreateDesc & InDesc)
{
	ETextureCreateFlags Flags = InDesc.Flags;

	NS::UInteger Usage = MTL::TextureUsageUnknown;
	if (EnumHasAnyFlags(Flags, TexCreate_ShaderResource | TexCreate_ResolveTargetable | TexCreate_DepthStencilTargetable))
	{
		Usage |= MTL::TextureUsageShaderRead;
	}

	if (EnumHasAnyFlags(Flags, TexCreate_UAV))
	{
		Usage |= MTL::TextureUsageShaderRead;
		Usage |= MTL::TextureUsageShaderWrite;
	}

	if (InDesc.AliasableFormats.Num() > 0)
	{
		Usage |= MTL::TextureUsagePixelFormatView;
	}
	
	// offline textures are normal shader read textures
	if (EnumHasAnyFlags(Flags, TexCreate_OfflineProcessed))
	{
		Usage |= MTL::TextureUsageShaderRead;
	}

	if(IsMetalBindlessEnabled())
	{
		if (EnumHasAllFlags(Flags, TexCreate_AtomicCompatible) || EnumHasAllFlags(Flags, ETextureCreateFlags::Atomic64Compatible))
		{
			Usage |= MTL::TextureUsageShaderAtomic;
		}
	}

	//if the high level is doing manual resolves then the textures specifically markes as resolve targets
	//are likely to be used in a manual shader resolve by the high level and must be bindable as rendertargets.
	const bool bSeparateResolveTargets = FMetalCommandQueue::SupportsSeparateMSAAAndResolveTarget();
	const bool bResolveTarget = EnumHasAnyFlags(Flags, TexCreate_ResolveTargetable);
	if (EnumHasAnyFlags(Flags, TexCreate_RenderTargetable|TexCreate_DepthStencilTargetable|TexCreate_DepthStencilResolveTarget) || (bResolveTarget && bSeparateResolveTargets))
	{
		Usage |= MTL::TextureUsageRenderTarget;
		Usage |= MTL::TextureUsageShaderRead;
	}
	
	// If we haven't specified any usage then default to read.
	if(Usage == MTL::TextureUsageUnknown)
	{
		Usage |= MTL::TextureUsageShaderRead;
	}
	
	return (MTL::TextureUsage)Usage;
}

static bool IsPixelFormatCompressed(EPixelFormat Format)
{
	switch (Format)
	{
		case PF_DXT1:
		case PF_DXT3:
		case PF_DXT5:
		case PF_PVRTC2:
		case PF_PVRTC4:
		case PF_BC4:
		case PF_BC5:
		case PF_ETC2_RGB:
		case PF_ETC2_RGBA:
		case PF_ASTC_4x4:
		case PF_ASTC_6x6:
		case PF_ASTC_8x8:
		case PF_ASTC_10x10:
		case PF_ASTC_12x12:
		case PF_BC6H:
		case PF_BC7:
			return true;
		default:
			return false;
	}
}

static bool IsPixelFormatASTCCompressed(EPixelFormat Format)
{
	switch (Format)
	{
		case PF_ASTC_4x4:
		case PF_ASTC_6x6:
		case PF_ASTC_8x8:
		case PF_ASTC_10x10:
		case PF_ASTC_12x12:
			return true;
		default:
			return false;
	}
}

static bool IsPixelFormatPVRTCCompressed(EPixelFormat Format)
{
	switch (Format)
	{
		case PF_PVRTC2:
		case PF_PVRTC4:
		case PF_ETC2_RGB:
		case PF_ETC2_RGBA:
			return true;
		default:
			return false;
	}
}

MTL::PixelFormat UEToMetalFormat(FMetalDevice& Device, EPixelFormat UEFormat, bool bSRGB)
{
	bool bAppleGPU = Device.GetDevice()->supportsFamily(MTL::GPUFamilyApple1);
    MTL::PixelFormat MTLFormat = (MTL::PixelFormat)GPixelFormats[UEFormat].PlatformFormat;

	if (bSRGB)
	{
        if (!bAppleGPU && UEFormat == PF_G8)
        {
            MTLFormat = MTL::PixelFormatRGBA8Unorm;
        }
        
		switch (MTLFormat)
		{
		default: break;
		case MTL::PixelFormatRGBA8Unorm   : MTLFormat = MTL::PixelFormatRGBA8Unorm_sRGB   ; break;
		case MTL::PixelFormatBGRA8Unorm   : MTLFormat = MTL::PixelFormatBGRA8Unorm_sRGB   ; break;
#if PLATFORM_MAC
		// Fix for Apple silicon M1 macs that can support BC pixel formats even though they are Apple family GPUs.
		case MTL::PixelFormatBC1_RGBA     : MTLFormat = MTL::PixelFormatBC1_RGBA_sRGB     ; break;
		case MTL::PixelFormatBC2_RGBA     : MTLFormat = MTL::PixelFormatBC2_RGBA_sRGB     ; break;
		case MTL::PixelFormatBC3_RGBA     : MTLFormat = MTL::PixelFormatBC3_RGBA_sRGB     ; break;
		case MTL::PixelFormatBC7_RGBAUnorm: MTLFormat = MTL::PixelFormatBC7_RGBAUnorm_sRGB; break;
#endif
		}

		if (bAppleGPU)
		{
			switch (MTLFormat)
			{
			default: break;
#if WITH_IOS_SIMULATOR
			case MTL::PixelFormatR8Unorm        : MTLFormat = MTL::PixelFormatR8Unorm    		  ; break;
#else
			case MTL::PixelFormatR8Unorm        : MTLFormat = MTL::PixelFormatR8Unorm_sRGB        ; break;
#endif
			case MTL::PixelFormatPVRTC_RGBA_2BPP: MTLFormat = MTL::PixelFormatPVRTC_RGBA_2BPP_sRGB; break;
			case MTL::PixelFormatPVRTC_RGBA_4BPP: MTLFormat = MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB; break;
			case MTL::PixelFormatASTC_4x4_LDR   : MTLFormat = MTL::PixelFormatASTC_4x4_sRGB       ; break;
			case MTL::PixelFormatASTC_6x6_LDR   : MTLFormat = MTL::PixelFormatASTC_6x6_sRGB       ; break;
			case MTL::PixelFormatASTC_8x8_LDR   : MTLFormat = MTL::PixelFormatASTC_8x8_sRGB       ; break;
			case MTL::PixelFormatASTC_10x10_LDR : MTLFormat = MTL::PixelFormatASTC_10x10_sRGB     ; break;
			case MTL::PixelFormatASTC_12x12_LDR : MTLFormat = MTL::PixelFormatASTC_12x12_sRGB     ; break;
			}
		}
	}

	return MTLFormat;
}

MTLTexturePtr FMetalSurface::Reallocate(MTLTexturePtr InTexture, MTL::TextureUsage UsageModifier)
{
	MTL::TextureDescriptor* Desc = MTL::TextureDescriptor::alloc()->init();
    check(Desc);
    
	Desc->setTextureType(InTexture->textureType());
	Desc->setPixelFormat(InTexture->pixelFormat());
	Desc->setWidth(InTexture->width());
	Desc->setHeight(InTexture->height());
	Desc->setDepth(InTexture->depth());
	Desc->setMipmapLevelCount(InTexture->mipmapLevelCount());
	Desc->setSampleCount(InTexture->sampleCount());
	Desc->setArrayLength(InTexture->arrayLength());
	
	MTL::ResourceOptions HazardTrackingMode = MTL::ResourceHazardTrackingModeUntracked;
	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	if(bSupportsHeaps)
	{
		HazardTrackingMode = MTL::ResourceHazardTrackingModeTracked;
	}
	
	static MTL::ResourceOptions GeneralResourceOption = (MTL::ResourceOptions)FMetalCommandQueue::GetCompatibleResourceOptions(HazardTrackingMode);
	
	Desc->setResourceOptions(MTL::ResourceOptions(((NS::UInteger)InTexture->cpuCacheMode() << MTL::ResourceCpuCacheModeShift) | ((NS::UInteger)Texture->storageMode() << MTL::ResourceStorageModeShift) | GeneralResourceOption));
	Desc->setCpuCacheMode(InTexture->cpuCacheMode());
	Desc->setStorageMode(InTexture->storageMode());
	Desc->setUsage(MTL::TextureUsage(InTexture->usage() | UsageModifier));
	
	MTLTexturePtr NewTex = Device.CreateTexture(this, Desc);
    
    Desc->release();
	check(NewTex);
	return NewTex;
}

void FMetalSurface::MakeAliasable(void)
{
	static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
	if (bSupportsHeaps && Texture->storageMode() == MTL::StorageModePrivate && Texture->heap())
	{
		if (MSAATexture && (MSAATexture != Texture) && !MSAATexture->isAliasable())
		{
			MSAATexture->makeAliasable();
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
			MetalLLM::LogAliasTexture(MSAATexture.get());
#endif
		}
		if (!Texture->isAliasable())
		{
			Texture->makeAliasable();
#if STATS || ENABLE_LOW_LEVEL_MEM_TRACKER
			MetalLLM::LogAliasTexture(Texture.get());
#endif
		}
	}
}

uint8 GetMetalPixelFormatKey(MTL::PixelFormat Format)
{
	struct FMetalPixelFormatKeyMap
	{
		FRWLock Mutex;
		uint8 NextKey = 1; // 0 is reserved for MTL::PixelFormatInvalid
		TMap<uint64, uint8> Map;

		uint8 Get(MTL::PixelFormat Format)
		{
			FRWScopeLock Lock(Mutex, SLT_ReadOnly);
			uint8* Key = Map.Find((uint64)Format);
			if (Key == nullptr)
			{
				Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
				Key = Map.Find((uint64)Format);
				if (Key == nullptr)
				{
					Key = &Map.Add((uint64)Format, NextKey++);
					// only giving 6 bits to the key
					checkf(NextKey < 64, TEXT("Too many unique pixel formats to fit into the PipelineStateHash"));
				}
			}
			return *Key;
		}

		FMetalPixelFormatKeyMap()
		{
			// Add depth stencil formats first, so we don't have to use 6 bits for them in the pipeline hash
			Get(MTL::PixelFormatDepth32Float);
			Get(MTL::PixelFormatDepth32Float_Stencil8);
	#if PLATFORM_MAC
			Get(MTL::PixelFormatDepth24Unorm_Stencil8);
			Get(MTL::PixelFormatDepth16Unorm);
	#endif
		}
	} static PixelFormatKeyMap;

	return PixelFormatKeyMap.Get(Format);
}

FMetalTextureCreateDesc::FMetalTextureCreateDesc(FMetalDevice& Device, FRHITextureCreateDesc const& InDesc)
	: FRHITextureCreateDesc(InDesc)
	, bIsRenderTarget(IsRenderTarget(InDesc.Flags))
{
	MTLFormat = UEToMetalFormat(Device, InDesc.Format, EnumHasAnyFlags(InDesc.Flags, TexCreate_SRGB));

	// get a unique key for this surface's format
	FormatKey = GetMetalPixelFormatKey(MTLFormat);

	if (InDesc.IsTextureCube())
	{
		Desc = NS::RetainPtr(MTL::TextureDescriptor::textureCubeDescriptor(MTLFormat, InDesc.Extent.X, (InDesc.NumMips > 1)));
	}
	else if (InDesc.IsTexture3D())
	{
        Desc = NS::TransferPtr(MTL::TextureDescriptor::alloc()->init());
        check(Desc);
        
		Desc->setTextureType(MTL::TextureType3D);
		Desc->setWidth(InDesc.Extent.X);
		Desc->setHeight(InDesc.Extent.Y);
		Desc->setDepth(InDesc.Depth);
		Desc->setPixelFormat(MTLFormat);
		Desc->setArrayLength(1);
		Desc->setMipmapLevelCount(1);
		Desc->setSampleCount(1);
	}
	else
	{
		Desc = NS::RetainPtr(MTL::TextureDescriptor::texture2DDescriptor(MTLFormat, InDesc.Extent.X, InDesc.Extent.Y, (InDesc.NumMips > 1)));
		Desc->setArrayLength(InDesc.ArraySize);
	}

	// flesh out the descriptor
	if (InDesc.IsTextureArray())
	{
		Desc->setArrayLength(InDesc.ArraySize);
		if (InDesc.IsTextureCube())
		{
			if (Device.SupportsFeature(EMetalFeaturesCubemapArrays))
			{
				Desc->setTextureType(MTL::TextureTypeCubeArray);
			}
			else
			{
				Desc->setTextureType(MTL::TextureType2DArray);
				Desc->setArrayLength(InDesc.ArraySize * 6);
			}
		}
		else
		{
			Desc->setTextureType(MTL::TextureType2DArray);
		}
	}
	Desc->setMipmapLevelCount(InDesc.NumMips);

	{
		Desc->setUsage(ConvertDescToUsage(InDesc));
		
#if WITH_IOS_SIMULATOR
		const bool bAppleGPU = false;
#else
		const bool bAppleGPU = Device.GetDevice()->supportsFamily(MTL::GPUFamilyApple1);
#endif

		if (EnumHasAnyFlags(InDesc.Flags, TexCreate_CPUReadback) && !EnumHasAnyFlags(InDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_FastVRAM))
		{
			Desc->setCpuCacheMode(MTL::CPUCacheModeDefaultCache);
			
			if(bAppleGPU)
			{
				Desc->setStorageMode(MTL::StorageModeShared);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared));
			}
#if PLATFORM_MAC
			else
			{
				Desc->setStorageMode(MTL::StorageModeManaged);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeManaged));
			}
#endif
		}
		else if (EnumHasAnyFlags(InDesc.Flags, TexCreate_NoTiling) && !EnumHasAnyFlags(InDesc.Flags, TexCreate_FastVRAM | TexCreate_DepthStencilTargetable | TexCreate_RenderTargetable | TexCreate_UAV))
		{
			Desc->setCpuCacheMode(MTL::CPUCacheModeDefaultCache);
			
			if(bAppleGPU)
			{
				Desc->setStorageMode(MTL::StorageModeShared);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared));
			}
#if PLATFORM_MAC
			else
			{
				Desc->setStorageMode(MTL::StorageModeManaged);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeManaged));
			}
#endif
		}
		else if (EnumHasAnyFlags(InDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable | TexCreate_ResolveTargetable | TexCreate_DepthStencilResolveTarget))
		{
			check(!(InDesc.Flags & TexCreate_CPUReadback));
			Desc->setCpuCacheMode(MTL::CPUCacheModeDefaultCache);
#if PLATFORM_MAC
			Desc->setStorageMode(MTL::StorageModePrivate);
			Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModePrivate));
#else
			if (GMetalForceIOSTexturesShared)
			{
				Desc->setStorageMode(MTL::StorageModeShared);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared));
			}
			else
			{
				Desc->setStorageMode(MTL::StorageModePrivate);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModePrivate));
			}
#endif
		}
		else
		{
			check(!(InDesc.Flags & TexCreate_CPUReadback));
			Desc->setCpuCacheMode(MTL::CPUCacheModeDefaultCache);
#if PLATFORM_MAC
			Desc->setStorageMode(MTL::StorageModePrivate);
			Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModePrivate));
#else
			if (GMetalForceIOSTexturesShared)
			{
				Desc->setStorageMode(MTL::StorageModeShared);
				Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared));
			}
			// No private storage for PVRTC as it messes up the blit-encoder usage.
			// note: this is set to always be on and will be re-addressed in a future release
			else
			{
				if (IsPixelFormatPVRTCCompressed(InDesc.Format))
				{
					Desc->setStorageMode(MTL::StorageModeShared);
					Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared));
				}
				else
				{
					Desc->setStorageMode(MTL::StorageModePrivate);
					Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModePrivate));
				}
			}
#endif
		}

		if (ShouldUseMemoryless(InDesc.Flags))
		{
			ensure(EnumHasAnyFlags(InDesc.Flags, (TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable)));
			ensure(!EnumHasAnyFlags(InDesc.Flags, (TexCreate_CPUReadback | TexCreate_CPUWritable)));
			ensure(!EnumHasAnyFlags(InDesc.Flags, TexCreate_UAV));
			Desc->setStorageMode(MTL::StorageModeMemoryless);
			Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeMemoryless));
		}

		MTL::ResourceOptions HazardTrackingMode = MTL::ResourceHazardTrackingModeUntracked;
		static bool bSupportsHeaps = Device.SupportsFeature(EMetalFeaturesHeaps);
		if(bSupportsHeaps)
		{
			HazardTrackingMode = MTL::ResourceHazardTrackingModeTracked;
		}
		
		static MTL::ResourceOptions GeneralResourceOption = FMetalCommandQueue::GetCompatibleResourceOptions(HazardTrackingMode);
		Desc->setResourceOptions((MTL::ResourceOptions)(Desc->resourceOptions() | GeneralResourceOption));
	}
}

FMetalTextureCreateDesc::FMetalTextureCreateDesc(FMetalTextureCreateDesc const& Other)
{
    *this = Other;
}

FMetalTextureCreateDesc& FMetalTextureCreateDesc::operator =(FMetalTextureCreateDesc const& Other)
{
    FRHITextureCreateDesc::operator=(Other);
    
    Desc = NS::TransferPtr(Other.Desc->copy());
    
    check(Desc->width() == Other.Desc->width());
    MTLFormat = Other.MTLFormat;
    bIsRenderTarget = Other.bIsRenderTarget;
    FormatKey = Other.FormatKey;
    
    return *this;
}

FMetalSurface::FMetalSurface(FMetalDevice& MetalDevice, FMetalTextureCreateDesc const& CreateDesc)
	: FRHITexture       (CreateDesc)
	, Device			(MetalDevice)
	, FormatKey         (CreateDesc.FormatKey)
	, Viewport          (nullptr)
	, TotalTextureSize  (0)
{
	check(CreateDesc.Extent.X > 0 && CreateDesc.Extent.Y > 0 && CreateDesc.NumMips > 0);
    
	// the special back buffer surface will be updated in BeginDrawingViewport - no need to set the texture here
	if (EnumHasAnyFlags(CreateDesc.Flags, TexCreate_Presentable))
	{
		return;
	}
    
    bool bIsMSAARequired = CreateDesc.NumSamples > 1 && AllowMSAA();
	bool bIsBindless = IsMetalBindlessEnabled();

#if RHI_USE_RESOURCE_DEBUG_NAME
    NS::String* DebugName = FStringToNSString(CreateDesc.DebugName);
#endif

	if (!Texture)
	{
		// Non VR/media texture case (i.e. a regular texture
		// Create the actual texture resource. Decide if we need to create from buffer backing
		const bool bBufferCompatibleOption = 	(CreateDesc.Desc->textureType() == MTL::TextureType2D || CreateDesc.Desc->textureType() == MTL::TextureTypeTextureBuffer) &&
												CreateDesc.NumMips == 1 && CreateDesc.ArraySize == 1 && CreateDesc.NumSamples == 1 && CreateDesc.Desc->depth() == 1;

        FMetalTextureCreateDesc NewCreateDesc = CreateDesc;
        
        // If we are attempting to create an MSAA texture the texture cannot be memoryless unless we are creating a depth texture
        if(ShouldUseMemoryless(CreateDesc.Flags) && bIsMSAARequired && CreateDesc.Format != PF_DepthStencil && CreateDesc.Format != PF_R32_FLOAT)
        {
            NewCreateDesc.Flags &= ~TexCreate_Memoryless;
            
            if (GMetalForceIOSTexturesShared)
            {
                NewCreateDesc.Desc->setStorageMode(MTL::StorageModeShared);
                NewCreateDesc.Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModeShared));
            }
            else
            {
                NewCreateDesc.Desc->setStorageMode(MTL::StorageModePrivate);
                NewCreateDesc.Desc->setResourceOptions((MTL::ResourceOptions)(MTL::ResourceCPUCacheModeDefaultCache | MTL::ResourceStorageModePrivate));
            }
        }

		const bool bAtomicCompatible = EnumHasAllFlags(CreateDesc.Flags, TexCreate_AtomicCompatible) || EnumHasAllFlags(CreateDesc.Flags, ETextureCreateFlags::Atomic64Compatible);
		
		bool bBufferBacked = EnumHasAllFlags(CreateDesc.Flags, TexCreate_UAV | TexCreate_NoTiling);
		if (bIsBindless)
		{
			bBufferBacked = bBufferBacked && !bAtomicCompatible;
		}
		else
		{
			bBufferBacked = bBufferBacked || bAtomicCompatible;
		}
		bBufferBacked = bBufferCompatibleOption && bBufferBacked;
		
        const bool bTextureArrayWithAtomics = !bIsBindless && NewCreateDesc.Desc->textureType() == MTL::TextureType2DArray && bAtomicCompatible;

		if (bBufferBacked)
		{
			MTL::Device* MTLDevice = Device.GetDevice();

			const uint32 MinimumByteAlignment = MTLDevice->minimumLinearTextureAlignmentForPixelFormat(CreateDesc.MTLFormat);
			const NS::UInteger BytesPerRow = Align(NewCreateDesc.Desc->width() * GPixelFormats[NewCreateDesc.Format].BlockBytes, MinimumByteAlignment);

			// Backing buffer resource options must match the texture we are going to create from it
			FMetalPooledBufferArgs Args(&Device, BytesPerRow * NewCreateDesc.Desc->height(), BUF_Dynamic, NewCreateDesc.Desc->storageMode(), NewCreateDesc.Desc->cpuCacheMode());
			BackingBuffer = Device.CreatePooledBuffer(Args);
			Texture = NS::TransferPtr(BackingBuffer->GetMTLBuffer()->newTexture(NewCreateDesc.Desc.get(), BackingBuffer->GetOffset(), BytesPerRow));
		}
        else if (bTextureArrayWithAtomics)
        {
            checkf(GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5, TEXT("Requested texture array with atomics that is unsupported on this platform"));
            
            MTL::Device* MTLDevice = Device.GetDevice();

            const uint32 MinimumByteAlignment = MTLDevice->minimumLinearTextureAlignmentForPixelFormat(CreateDesc.MTLFormat);
            const NS::UInteger BytesPerRow = Align(NewCreateDesc.Desc->width() * NewCreateDesc.Desc->arrayLength() * GPixelFormats[NewCreateDesc.Format].BlockBytes, MinimumByteAlignment);

            // Backing buffer resource options must match the texture we are going to create from it
            FMetalPooledBufferArgs Args(&Device, BytesPerRow * NewCreateDesc.Desc->height(), BUF_Dynamic, NewCreateDesc.Desc->storageMode(), NewCreateDesc.Desc->cpuCacheMode());
			BackingBuffer = Device.CreatePooledBuffer(Args);

            NewCreateDesc.Desc->setWidth(NewCreateDesc.Desc->width() * NewCreateDesc.Desc->arrayLength());
            NewCreateDesc.Desc->setArrayLength(1);
            NewCreateDesc.Desc->setTextureType(MTL::TextureType2D);
            Texture = NS::TransferPtr(BackingBuffer->GetMTLBuffer()->newTexture(NewCreateDesc.Desc.get(), BackingBuffer->GetOffset(), BytesPerRow));
        }
		else
		{
			if(!bIsBindless)
			{
				// If we are in here then either the texture description is not buffer compatable or these flags were not set
				// assert that these flag combinations are not set as they require a buffer backed texture and the texture description is not compatible with that
				checkf(!EnumHasAllFlags(CreateDesc.Flags, TexCreate_AtomicCompatible), TEXT("Requested buffer backed texture that breaks Metal linear texture limitations: %s"), *NSStringToFString(NewCreateDesc.Desc->description()));
			}

			NewCreateDesc.Desc->setUsage(ConvertDescToUsage(CreateDesc));
			Texture = Device.CreateTexture(this, NewCreateDesc.Desc.get());
		}
		
		METAL_FATAL_ASSERT(Texture, TEXT("Failed to create texture, desc %s"), *NSStringToFString(CreateDesc.Desc->description()));

#if RHI_USE_RESOURCE_DEBUG_NAME
		Texture->setLabel(DebugName);
#endif
	}

	// calculate size of the texture
	TotalTextureSize = GetMemorySize();

	if (bIsMSAARequired)
	{
		MTLTextureDescriptorPtr Desc = CreateDesc.Desc;
		check(CreateDesc.bIsRenderTarget);
		
		// In bindless texture types need to be converted to Array Types to match the generated AIR
		Desc->setTextureType(bIsBindless ? MTL::TextureType2DMultisampleArray : MTL::TextureType2DMultisample);

		// allow commandline to override
		uint32 NewNumSamples;
		if (FParse::Value(FCommandLine::Get(), TEXT("msaa="), NewNumSamples))
		{
			Desc->setSampleCount(NewNumSamples);
		}
		else
		{
			Desc->setSampleCount(CreateDesc.NumSamples);
		}

		bool bMemoryless = false;
        
		if (ShouldUseMemoryless(CreateDesc.Flags))
		{
			bMemoryless = true;
			Desc->setStorageMode(MTL::StorageModeMemoryless);
			Desc->setResourceOptions(MTL::ResourceStorageModeMemoryless);
		}

		MSAATexture = Device.CreateTexture(this, Desc.get());
			
		//device doesn't support HW depth resolve.  This case only valid on mobile renderer or
		//on Mac where RHISupportsSeparateMSAAAndResolveTextures is true.
		const bool bSupportsMSAADepthResolve = Device.SupportsFeature(EMetalFeaturesMSAADepthResolve);
		const bool bDepthButNoResolveSupported = CreateDesc.Format == PF_DepthStencil && !bSupportsMSAADepthResolve;
		if (bDepthButNoResolveSupported)
		{
			Texture = MSAATexture;
				
			// we don't have the resolve texture, so we just update the memory size with the MSAA size
			TotalTextureSize = TotalTextureSize * CreateDesc.NumSamples;
		}
		else if (!bMemoryless)
		{
			// an MSAA render target takes NumSamples more space, in addition to the resolve texture
			TotalTextureSize += TotalTextureSize * CreateDesc.NumSamples;
		}
			
		if (MSAATexture != Texture)
		{
			check(!MSAAResolveTexture);
				
			//if bSupportsSeparateMSAAAndResolve then the high level expect to binds the MSAA when binding shader params.
			const bool bSupportsSeparateMSAAAndResolve = FMetalCommandQueue::SupportsSeparateMSAAAndResolveTarget();
			if (bSupportsSeparateMSAAAndResolve)
			{
				MSAAResolveTexture = Texture;
					
				Texture = MSAATexture;
			}
			else
			{
				MSAAResolveTexture = Texture;
			}
		}
			
		//we always require an MSAAResolveTexture if MSAATexture is active.
		check(!MSAATexture || MSAAResolveTexture || bDepthButNoResolveSupported);
			
        UE_LOGF(LogMetal, Verbose, "Creating MSAA %d x %d %ls surface", CreateDesc.Extent.X, CreateDesc.Extent.Y, EnumHasAnyFlags(CreateDesc.Flags, TexCreate_RenderTargetable) ? TEXT("Color") : TEXT("Depth"));
        
		if (MSAATexture.get() == nullptr)
		{
            UE_LOGF(LogMetal, Fatal, "Failed to create MSAA texture");
		}

#if RHI_USE_RESOURCE_DEBUG_NAME
		MSAATexture->setLabel(DebugName);
#endif
	}
	
	// create a stencil buffer if needed
	if (CreateDesc.Format == PF_DepthStencil)
	{
		// 1 byte per texel
		TotalTextureSize += CreateDesc.Extent.X * CreateDesc.Extent.Y;
	}
	
	// track memory usage
	const bool bOnlyStreamableTexturesInTexturePool = false;
	UE::RHICore::UpdateGlobalTextureStats(GetDesc(), TotalTextureSize, bOnlyStreamableTexturesInTexturePool, true);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);

		BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::TextureSRV);
	}
#endif
}

FMetalExternalSurface::FMetalExternalSurface(FMetalDevice& MetalDevice, MTLTexturePtr InTexture, FRHITextureCreateDesc const& CreateDesc, TUniqueFunction<void()>&& InOnDeleteFunction)
	: 	FMetalSurface	(MetalDevice, InTexture, CreateDesc),
		OnDeleteFunction(MoveTemp(InOnDeleteFunction))
{
	check(CreateDesc.Extent.X > 0 && CreateDesc.Extent.Y > 0 && CreateDesc.NumMips > 0);
	
	FormatKey = GetMetalPixelFormatKey(Texture->pixelFormat());
	
	// calculate size of the texture
	TotalTextureSize = GetMemorySize();
	
	const bool bOnlyStreamableTexturesInTexturePool = false;
	UE::RHICore::UpdateGlobalTextureStats(GetDesc(), TotalTextureSize, bOnlyStreamableTexturesInTexturePool, true);
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);

		BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::TextureSRV, InTexture.get());
	}
#endif
}

FMetalExternalSurface::~FMetalExternalSurface()
{
	FMetalDynamicRHI::Get().DeferredDelete([DeleteFunction=MoveTemp(OnDeleteFunction), InTexture=Texture]{
		DeleteFunction();
	});
	
	// Reset the texture so the parent doesn't free it as well
	Texture.reset();
}

FTextureRHIRef FMetalDynamicRHI::RHICreateTexture2DFromCVMetalTexture(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, CVMetalTextureRef Resource)
{
	CFRetain(Resource);
	
	MTLTexturePtr MetalTexture = NS::RetainPtr((__bridge MTL::Texture*)CVMetalTextureGetTexture((CVPixelBufferRef)Resource));
	
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("MetalTexture2DFromResource"), MetalTexture->width(), MetalTexture->height(), Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(TexCreateFlags)
		.SetNumMips(MetalTexture->mipmapLevelCount())
		.SetNumSamples(MetalTexture->sampleCount())
		.DetermineInititialState();

	return new FMetalExternalSurface(*Device, MetalTexture, Desc, [Resource]() {
		CFRelease(Resource);
	});
}

void FMetalSurface::Initialize(FRHICommandListBase& RHICmdList)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);
		
		BindlessDescriptorManager->UpdateDescriptor(RHICmdList, BindlessHandle, Texture.get(), EMetalDescriptorUpdateType::Immediate);
	}
#endif
}

class FMetalDeferredStats
{
public:
    FMetalDeferredStats(){};
    ~FMetalDeferredStats()
    {
        const bool bOnlyStreamableTexturesInTexturePool = false;
        UE::RHICore::UpdateGlobalTextureStats(Flags, Dimension, TextureSize, bOnlyStreamableTexturesInTexturePool, false);
    }
    
	ETextureDimension Dimension;
	ETextureCreateFlags Flags;
	uint64 TextureSize;
};

FMetalSurface::~FMetalSurface()
{
	if (BackingBuffer) 
	{
		FMetalDynamicRHI::Get().DeferredDelete(BackingBuffer);	
	}
	
	if (MSAATexture)
	{
		if (Texture != MSAATexture)
		{
			SafeRelease(MSAATexture);
		}
	}
	
	//do the same as above.  only do a [release] if it's the same as texture.
	if (MSAAResolveTexture)
	{
		if (Texture != MSAAResolveTexture)
		{
			SafeRelease(MSAAResolveTexture);
		}
	}
	
	if (!(GetDesc().Flags & TexCreate_Presentable) && Texture)
	{
		SafeRelease(Texture);
	}
	
	MSAATexture.reset();
    MSAAResolveTexture.reset();
    Texture.reset();
	
	// track memory usage
	FMetalDeferredStats* DeferredStats = new FMetalDeferredStats;
    DeferredStats->Dimension = GetDesc().Dimension;
    DeferredStats->Flags = GetDesc().Flags;
    DeferredStats->TextureSize = TotalTextureSize;
    
	FMetalDynamicRHI::Get().DeferredDelete([DeferredStats]() {
        delete DeferredStats;
    });

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if(IsMetalBindlessEnabled())
	{
		if(BindlessHandle.IsValid())
		{
			FMetalDynamicRHI::Get().DeferredDelete(BindlessHandle);
		}
	}
#endif
}

void FMetalSurface::SafeRelease(MTLTexturePtr InTexture)
{
	if(GIsMetalInitialized && GDynamicRHI)
	{
		FMetalDynamicRHI::Get().DeferredDelete(InTexture);
	}
}

MTL::Buffer* FMetalSurface::AllocSurface(const FRHILockTextureArgs& Arguments, uint32 MipBytes, uint32 DestStride)
{
	check(IsInRenderingThread());

	// allocate some temporary memory
	// This should really be pooled and texture transfers should be their own pool
	MTL::Device* MTLDevice = Device.GetDevice();
    MTL::Buffer* Buffer = MTLDevice->newBuffer(MipBytes, MTL::ResourceStorageModeShared);
	Buffer->setLabel(NS::String::string("Temporary Surface Backing", NS::UTF8StringEncoding));
	
	// Note: while the lock is active, this map owns the backing store.
	GRHILockTracker.Lock(Arguments, Buffer, false);
	
#if PLATFORM_MAC
	// Expand R8_sRGB into RGBA8_sRGB for non Apple Silicon Mac.
	if (GetDesc().Format == PF_G8 
		&& EnumHasAnyFlags(GetDesc().Flags, TexCreate_SRGB) 
		&& Arguments.LockMode == RLM_WriteOnly
		&& Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
	{
		check(GetDesc().Dimension == ETextureDimension::Texture2D || GetDesc().Dimension == ETextureDimension::Texture3D);
		DestStride = FMath::Max<uint32>(GetDesc().Extent.X >> Arguments.MipIndex, 1);
	}
#endif
	
	check(Buffer);
	
	return Buffer;
}

void FMetalSurface::UpdateSurfaceAndDestroySourceBuffer(FMetalRHICommandContext* Context, MTL::Buffer* SourceBuffer, uint32 MipIndex, uint32 ArrayIndex)
{
#if STATS
	uint64 Start = FPlatformTime::Cycles64();
#endif
	check(SourceBuffer);
	
	uint32 Stride;
	uint32 BytesPerImage = GetMipSize(MipIndex, &Stride, true);
	
	MTL::Region Region;
	if (GetDesc().IsTexture3D())
	{
		// upload the texture to the texture slice
		Region = MTL::Region(
			0, 0, 0,
			FMath::Max<uint32>(GetDesc().Extent.X >> MipIndex, 1),
			FMath::Max<uint32>(GetDesc().Extent.Y >> MipIndex, 1),
			FMath::Max<uint32>(GetDesc().Depth    >> MipIndex, 1)
		);
	}
	else
	{
		// upload the texture to the texture slice
		Region = MTL::Region(
			0, 0,
			FMath::Max<uint32>(GetDesc().Extent.X >> MipIndex, 1),
			FMath::Max<uint32>(GetDesc().Extent.Y >> MipIndex, 1)
		);
	}

#if PLATFORM_MAC
	// Expand R8_sRGB into RGBA8_sRGB for non Apple Silicon Mac.
	if (GetDesc().Format == PF_G8 
		&& EnumHasAnyFlags(GetDesc().Flags, TexCreate_SRGB) 
		&& Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
	{
		check(GetDesc().Dimension == ETextureDimension::Texture2D || GetDesc().Dimension == ETextureDimension::Texture3D);
		
		TArray<uint8> Data;
		uint8* ExpandedMem = (uint8*) SourceBuffer->contents();
		check(ExpandedMem);
		Data.Append(ExpandedMem, BytesPerImage);
		uint32 SrcStride = FMath::Max<uint32>(GetDesc().Extent.X >> MipIndex, 1);
		
		for(uint z = 0; z < FMath::Max<uint32>(GetDesc().Depth >> MipIndex, 1); z++)
		{
			uint32_t DepthOffset = z * (GetDesc().Extent.X * GetDesc().Extent.Y);
			for(uint y = 0; y < FMath::Max<uint32>(GetDesc().Extent.Y >> MipIndex, 1); y++)
			{
				uint8* RowDest = ExpandedMem;
				for(uint x = 0; x < FMath::Max<uint32>(GetDesc().Extent.X >> MipIndex, 1); x++)
				{
					*(RowDest++) = Data[(y * SrcStride) + DepthOffset + x];
					*(RowDest++) = Data[(y * SrcStride) + DepthOffset + x];
					*(RowDest++) = Data[(y * SrcStride) + DepthOffset + x];
					*(RowDest++) = Data[(y * SrcStride) + DepthOffset + x];
				}
				ExpandedMem = (ExpandedMem + Stride);
			}
		}
	}
#endif
	
	FMetalBufferPtr Source = FMetalBufferPtr(new FMetalBuffer(SourceBuffer, FMetalBuffer::FreePolicy::Owner));
	
	if(Texture->storageMode() == MTL::StorageModePrivate)
	{
        MTL_SCOPED_AUTORELEASE_POOL;
		
		int64 Size = BytesPerImage * Region.size.depth * FMath::Max(1u, ArrayIndex);
		
		MTL::BlitOption Options = MTL::BlitOptionNone;
#if !PLATFORM_MAC
		if (Texture->pixelFormat() >= MTL::PixelFormatPVRTC_RGB_2BPP && Texture->pixelFormat() <= MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB)
		{
			Options = MTL::BlitOptionRowLinearPVRTC;
		}
#endif
		Context->CopyFromBufferToTexture(Source, 0, Stride, BytesPerImage, Region.size, Texture.get(), ArrayIndex, MipIndex, Region.origin, Options);
		
		INC_DWORD_STAT_BY(STAT_MetalTextureMemUpdate, Size);
	}
	else
	{
#if !PLATFORM_MAC
		if (Texture->pixelFormat() >= MTL::PixelFormatPVRTC_RGB_2BPP && Texture->pixelFormat() <= MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB) // @todo Calculate correct strides and byte-counts
		{
			Stride = 0;
			BytesPerImage = 0;
		}
#endif
		
        Texture->replaceRegion(Region, MipIndex, ArrayIndex, SourceBuffer->contents(), Stride, BytesPerImage);
		SourceBuffer = nullptr;
		
		INC_DWORD_STAT_BY(STAT_MetalTextureMemUpdate, BytesPerImage);
	}
	
	FMetalDynamicRHI::Get().DeferredDelete(Source);
}

FRHILockTextureResult FMetalSurface::Lock(const FRHILockTextureArgs& Arguments, bool bSingleLayer)
{
	const uint32 ResourceIndex = UE::RHICore::GetLockArrayIndex(GetDesc(), Arguments);

	FRHILockTextureResult Result;

	// get size and stride
	uint32 MipBytes = GetMipSize(Arguments.MipIndex, &Result.Stride, bSingleLayer);
	Result.ByteCount = MipBytes;

	// allocate some temporary memory
	MTL::Buffer* SourceData = AllocSurface(Arguments, MipBytes, Result.Stride);

	switch(Arguments.LockMode)
	{
		case RLM_ReadOnly:
		{
			SCOPE_CYCLE_COUNTER(STAT_MetalTexturePageOffTime);
			
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
			
			MTL::Region Region;
			if (GetDesc().IsTexture3D())
			{
				// upload the texture to the texture slice
				Region = MTL::Region(
					0, 0, 0,
					FMath::Max<uint32>(GetDesc().Extent.X >> Arguments.MipIndex, 1),
					FMath::Max<uint32>(GetDesc().Extent.Y >> Arguments.MipIndex, 1),
					FMath::Max<uint32>(GetDesc().Depth    >> Arguments.MipIndex, 1));
			}
			else
			{
				// upload the texture to the texture slice
				Region = MTL::Region(
					0, 0,
					FMath::Max<uint32>(GetDesc().Extent.X >> Arguments.MipIndex, 1),
					FMath::Max<uint32>(GetDesc().Extent.Y >> Arguments.MipIndex, 1)
				);
			}
			
			if (Texture->storageMode() == MTL::StorageModePrivate)
			{
				// If we are running with command lists or the RHI thread is enabled we have to execute GFX commands in that context.
				RHICmdList.EnqueueLambda(
					[this, ResourceIndex, MipIndex = Arguments.MipIndex, Region, SourceData, Stride = Result.Stride, MipBytes](FRHICommandListImmediate& RHICmdList)
					{
						FMetalBufferPtr Source = FMetalBufferPtr(new FMetalBuffer(SourceData, FMetalBuffer::FreePolicy::Temporary));

						FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
						Context.CopyFromTextureToBuffer(this->Texture.get(), ResourceIndex, MipIndex, Region.origin, Region.size, Source, 0, Stride, MipBytes, MTL::BlitOptionNone);
					});
				RHICmdList.SubmitAndBlockUntilGPUIdle();
			}
			else
			{
#if PLATFORM_MAC
				if (this->Texture->storageMode() == MTL::StorageModeManaged)
				{
					// Managed texture - need to sync GPU -> CPU before access as it could have been written to by the GPU
					RHICmdList.EnqueueLambda(
						[this, ResourceIndex, MipIndex = Arguments.MipIndex](FRHICommandListImmediate& RHICmdList)
						{
							FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);
							Context.SynchronizeTexture(this->Texture.get(), ResourceIndex, MipIndex);
						});
					RHICmdList.SubmitAndBlockUntilGPUIdle();
				}
#endif
				
				// This block breaks the texture atlas system in Ocean, which depends on nonzero strides coming back from compressed textures. Turning off.
#if 0
				if (GetDesc().Format == PF_PVRTC2 || GetDesc().Format == PF_PVRTC4)
				{
					// for compressed textures metal debug RT expects 0 for rowBytes and imageBytes.
					Result.Stride = 0;
					MipBytes = 0;
				}
#endif
				uint32 BytesPerRow = Result.Stride;
				if (GetDesc().Format == PF_PVRTC2 || GetDesc().Format == PF_PVRTC4)
				{
					// for compressed textures metal debug RT expects 0 for rowBytes and imageBytes.
					BytesPerRow = 0;
					MipBytes = 0;
				}
				
				uint32_t BytesPerImage = MipBytes;
				BytesPerImage = MipBytes / Region.size.depth;
				
				void * Contents = SourceData->contents();
				
				Texture->getBytes(Contents, BytesPerRow, BytesPerImage, Region, Arguments.MipIndex, ResourceIndex);
			}
			
#if PLATFORM_MAC
			// Pack RGBA8_sRGB into R8_sRGB for non Apple Silicon Mac.
			if (   GetDesc().Format == PF_G8 
				&& GetDesc().Dimension == ETextureDimension::Texture2D
				&& EnumHasAnyFlags(GetDesc().Flags, TexCreate_SRGB) 
				&& Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
			{
				TArray<uint8> Data;
				uint8* ExpandedMem = (uint8*)SourceData->contents();
				Data.Append(ExpandedMem, MipBytes);
				uint32 SrcStride = Result.Stride;
				Result.Stride = FMath::Max<uint32>(GetDesc().Extent.X >> Arguments.MipIndex, 1);
				for(uint y = 0; y < FMath::Max<uint32>(GetDesc().Extent.Y >> Arguments.MipIndex, 1); y++)
				{
					uint8* RowDest = ExpandedMem;
					for(uint x = 0; x < FMath::Max<uint32>(GetDesc().Extent.X >> Arguments.MipIndex, 1); x++)
					{
						*(RowDest++) = Data[(y * SrcStride) + (x * 4)];
					}
					ExpandedMem = (ExpandedMem + Result.Stride);
				}
			}
#endif
			
			break;
		}
		case RLM_WriteOnly:
		{
			break;
		}
		default:
			check(false);
			break;
	}
	
	Result.Data = SourceData->contents();

	return Result;
}

void FMetalSurface::Unlock(const FRHILockTextureArgs& Arguments)
{
	check(IsInRenderingThread());

	FRHILockTracker::FTextureLockParams Params = GRHILockTracker.Unlock(Arguments);
	MTL::Buffer* SourceData = reinterpret_cast<MTL::Buffer*>(Params.Data);

	const uint32 ResourceIndex = UE::RHICore::GetLockArrayIndex(GetDesc(), Arguments);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda(
		[Surface = this, SourceData, MipIndex = Arguments.MipIndex, ResourceIndex](FRHICommandListBase& ExecutingCmdList)
		{
			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(ExecutingCmdList);
			Surface->UpdateSurfaceAndDestroySourceBuffer(&Context, SourceData, MipIndex, ResourceIndex);
		});
}

FRHILockTextureResult FMetalSurface::AsyncLock(FRHICommandListBase& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	bool bDirectLock = (Arguments.LockMode == RLM_ReadOnly || !GIsRHIInitialized);
	
	// Never flush for writing, it is unnecessary
	if (bDirectLock && Arguments.bNeedsDefaultRHIFlush)
    {
        // @todo Not all read locks need to flush either, but that'll require resource use tracking
        QUICK_SCOPE_CYCLE_COUNTER(STAT_RHIMETHOD_LockTexture_Flush);
        RHICmdList.GetAsImmediate().ImmediateFlush(EImmediateFlushType::FlushRHIThread);
    }
    
	const FRHILockTextureResult Result = Lock(Arguments, false);
	
	check(Result.Data);
	
	return Result;
}

uint32 FMetalSurface::GetMipSize(uint32 MipIndex, uint32* Stride, bool bSingleLayer)
{
	EPixelFormat PixelFormat = GetDesc().Format;

	// DXT/BC formats on Mac actually do have mip-tails that are smaller than the block size, they end up being uncompressed.
	bool const bPixelFormatASTC = IsPixelFormatASTCCompressed(PixelFormat);
	
	// Calculate the dimensions of the mip-map.
	const uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
	const uint32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
	const uint32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;
	const uint32 Alignment = 1u; // Apparently we always want natural row alignment (tightly-packed) even though the docs say iOS doesn't support it - this may be because we don't upload texture data from one contiguous buffer.
	const uint32 UnalignedMipSizeX = FMath::Max<uint32>(GetDesc().Extent.X >> MipIndex, BlockSizeX);
	const uint32 UnalignedMipSizeY = FMath::Max<uint32>(GetDesc().Extent.Y >> MipIndex, BlockSizeY);
	const uint32 MipSizeX = (bPixelFormatASTC) ? AlignArbitrary(UnalignedMipSizeX, BlockSizeX) : UnalignedMipSizeX;
	const uint32 MipSizeY = (bPixelFormatASTC) ? AlignArbitrary(UnalignedMipSizeY, BlockSizeY) : UnalignedMipSizeY;
	
	const uint32 MipSizeZ = bSingleLayer ? 1 : FMath::Max<uint32>(GetDesc().Depth >> MipIndex, 1u);
	uint32 NumBlocksX = (MipSizeX + BlockSizeX - 1) / BlockSizeX;
	uint32 NumBlocksY = (MipSizeY + BlockSizeY - 1) / BlockSizeY;
	if (PixelFormat == PF_PVRTC2 || PixelFormat == PF_PVRTC4)
	{
		// PVRTC has minimum 2 blocks width and height
		NumBlocksX = FMath::Max<uint32>(NumBlocksX, 2);
		NumBlocksY = FMath::Max<uint32>(NumBlocksY, 2);
	}
#if PLATFORM_MAC
	else if (PixelFormat == PF_G8 && EnumHasAnyFlags(GetDesc().Flags, TexCreate_SRGB) && Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
	{
		// RGBA_sRGB is the closest match - so expand the data.
		NumBlocksX *= 4;
	}
#endif
	
	const uint32 MipStride = NumBlocksX * BlockBytes;
	const uint32 AlignedStride = ((MipStride - 1) & ~(Alignment - 1)) + Alignment;
	
	const uint32 MipBytes = AlignedStride * NumBlocksY * MipSizeZ;
	
	if (Stride)
	{
		*Stride = AlignedStride;
	}
	
	return MipBytes;
}

uint32 FMetalSurface::GetMemorySize()
{
	// if already calculated, no need to do it again
	if (TotalTextureSize != 0)
	{
		return TotalTextureSize;
	}
	
	if (!Texture)
	{
		return 0;
	}
	
	uint32 TotalSize = 0;
	for (uint32 MipIndex = 0; MipIndex < Texture->mipmapLevelCount(); MipIndex++)
	{
		uint32 NumFaces = (GetDesc().IsTextureCube() ? 6 : 1);
		TotalSize += GetMipSize(MipIndex, NULL, false) * GetDesc().ArraySize * NumFaces;
	}
	
	return TotalSize;
}

uint32 FMetalSurface::GetNumFaces()
{
	// UE <= 5.0 for Cube Texture FMetalSurface::SizeZ was set to 6
	// UE >= 5.1 FMetalSurface::SizeZ does not exist and extent.Depth from the create descriptor is set to 1 for cube textures
	return GetDesc().Depth * GetDesc().ArraySize * (GetDesc().IsTextureCube() ? 6 : 1);
}

MTLTexturePtr FMetalSurface::GetDrawableTexture()
{
	FScopeLock Lock(&DrawableMutex);
	if (!Texture && EnumHasAnyFlags(GetDesc().Flags, TexCreate_Presentable))
	{
		check(Viewport);
		Texture = NS::RetainPtr(Viewport->GetDrawableTexture(EMetalViewportAccessRHI, this));

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if(IsMetalBindlessEnabled())
		{
			FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
			check(BindlessDescriptorManager);
			
			BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::TextureSRV, Texture.get());
		}
#endif
	}
	return Texture;
}

void FMetalSurface::ReleaseDrawableTexture()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		FMetalDynamicRHI::Get().DeferredDelete(BindlessHandle);
		BindlessHandle = {};
	}
#endif

	Texture.reset();
}

MTLTexturePtr FMetalSurface::GetCurrentTexture()
{
    MTLTexturePtr Tex;
	if (Viewport && EnumHasAnyFlags(GetDesc().Flags, TexCreate_Presentable))
	{
		check(Viewport);
		Tex = NS::RetainPtr(Viewport->GetCurrentTexture(EMetalViewportAccessRHI));
	}
	return Tex;
}


/*-----------------------------------------------------------------------------
 Texture allocator support.
 -----------------------------------------------------------------------------*/

void FMetalDynamicRHI::RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	UE::RHICore::FillBaselineTextureMemoryStats(OutStats);

	if (MemoryStats.TotalGraphicsMemory > 0)
	{
		OutStats.DedicatedVideoMemory = MemoryStats.DedicatedVideoMemory;
		OutStats.DedicatedSystemMemory = MemoryStats.DedicatedSystemMemory;
		OutStats.SharedSystemMemory = MemoryStats.SharedSystemMemory;
		OutStats.TotalGraphicsMemory = MemoryStats.TotalGraphicsMemory;
	}

	OutStats.LargestContiguousAllocation = OutStats.StreamingMemorySize;
}

bool FMetalDynamicRHI::RHIGetTextureMemoryVisualizeData( FColor* /*TextureData*/, int32 /*SizeX*/, int32 /*SizeY*/, int32 /*Pitch*/, int32 /*PixelSize*/ )
{
	NOT_SUPPORTED("RHIGetTextureMemoryVisualizeData");
	return false;
}

uint32 FMetalDynamicRHI::RHIComputeMemorySize(FRHITexture* TextureRHI)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    if(!TextureRHI)
    {
        return 0;
    }
    
    return GetMetalSurfaceFromRHITexture(TextureRHI)->GetMemorySize();
}

/*-----------------------------------------------------------------------------
 2D texture support.
 -----------------------------------------------------------------------------*/

struct FMetalInitializerMemory
{
	FMetalInitializerMemory(MTL::Buffer* InMemory)
		: Memory(InMemory)
	{
	}
	FMetalInitializerMemory(FMetalInitializerMemory&& Other)
		: Memory(Other.Memory)
	{
		Other.Memory = nullptr;
	}

	~FMetalInitializerMemory()
	{
		if (Memory)
		{
			Memory->release();
		}
	}

	MTL::Buffer* Relinquish()
	{
		if (Memory)
		{
			MTL::Buffer* Result = Memory;
			Memory = nullptr;
			return Result;
		}

		return nullptr;
	}

	MTL::Buffer* Memory{};
};

static bool IsMetalTextureInitializingPVRTC(MTL::Texture* Texture)
{
#if !PLATFORM_MAC
	return (Texture->pixelFormat() >= MTL::PixelFormatPVRTC_RGB_2BPP && Texture->pixelFormat() <= MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB);
#else
	return false;
#endif
}

static UE::RHICore::FDefaultLayoutTextureInitializer CreateWritableTextureInitializer(FRHICommandListBase& RHICmdList, FMetalSurface* Texture, const FRHITextureCreateDesc& CreateDesc)
{
	const uint64 WritableMemorySize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);
	
	MTL::Buffer* WritableBuffer = Texture->Device.GetDevice()->newBuffer(WritableMemorySize, MTL::ResourceStorageModeShared);
	WritableBuffer->setLabel(NS::String::string("Texture Initializer Scratch", NS::UTF8StringEncoding));

	return UE::RHICore::FDefaultLayoutTextureInitializer(RHICmdList, Texture, WritableBuffer->contents(), WritableMemorySize,
		[Texture = TRefCountPtr<FMetalSurface>(Texture), WritableBuffer = FMetalInitializerMemory(WritableBuffer), WritableMemorySize](FRHICommandListBase& RHICmdList) mutable
		{
			Texture->Initialize(RHICmdList);

			RHICmdList.EnqueueLambda(
				[Texture, WritableBuffer = MoveTemp(WritableBuffer), WritableMemorySize](FRHICommandListBase& RHICmdList) mutable
				{
					FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(RHICmdList);

					MTL::Texture* const MetalTexture = Texture->Texture.get();

					const bool bPrivateStorage = MetalTexture->storageMode() == MTL::StorageModePrivate;
					const bool bPVRTC = IsMetalTextureInitializingPVRTC(MetalTexture);
					const MTL::BlitOption CopyOptions =  bPVRTC ? MTL::BlitOptionRowLinearPVRTC : MTL::BlitOptionNone;

					FMetalBufferPtr SourceBuffer = FMetalBufferPtr(new FMetalBuffer(WritableBuffer.Relinquish(), FMetalBuffer::FreePolicy::Owner));

					const FRHITextureDesc& TextureDesc = Texture->GetDesc();
					const uint32 FaceCount = TextureDesc.IsTextureCube() ? 6 : 1;

					for (uint32 FaceIndex = 0; FaceIndex < FaceCount; FaceIndex++)
					{
						for (uint32 ArrayIndex = 0; ArrayIndex < TextureDesc.ArraySize; ArrayIndex++)
						{
							for (uint32 MipIndex = 0; MipIndex < TextureDesc.NumMips; MipIndex++)
							{
								uint64 SubresourceStride = 0;
								uint64 SubresourceSize = 0;
								const uint64 Offset = UE::RHITextureUtils::CalculateSubresourceOffset(TextureDesc, FaceIndex, ArrayIndex, MipIndex, SubresourceStride, SubresourceSize);

								// The number of bytes between each 2D image of a 3D texture. This should be 0 for 2D textures.
								uint64 SourceBytesPerImage = 0;
								if (TextureDesc.IsTexture3D())
								{
									SourceBytesPerImage = Texture->GetMipSize(MipIndex, nullptr, true);
								}

								check((Offset + SubresourceSize) <= WritableMemorySize);

								const uint32 FullArrayIndex = FaceIndex + ArrayIndex * FaceCount;

								const FUintVector3 Extents = UE::RHITextureUtils::CalculateMipExtents(TextureDesc, MipIndex);
								const MTL::Region Region = MTL::Region(0, 0, 0, Extents.X, Extents.Y, Extents.Z);

								if (bPrivateStorage)
								{
									Context.CopyFromBufferToTexture(
										SourceBuffer,
										Offset,
										SubresourceStride,
										SourceBytesPerImage,
										Region.size,
										MetalTexture,
										FullArrayIndex,
										MipIndex,
										Region.origin,
										CopyOptions
									);
								}
								else
								{
									// @todo Calculate correct strides and byte-counts
									const uint64 UsedStride = bPVRTC ? 0 : SubresourceStride;
									const uint64 UsedSize = bPVRTC ? 0 : SourceBytesPerImage;

									MetalTexture->replaceRegion(
										Region,
										MipIndex,
										FullArrayIndex,
										SourceBuffer->Contents(),
										UsedStride,
										UsedSize
									);
								}
							}
						}
					}

					FMetalDynamicRHI::Get().DeferredDelete(SourceBuffer);
				}
			);

			return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
		}
	);
}

FRHITextureInitializer FMetalDynamicRHI::RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	FMetalSurface* Texture = new FMetalSurface(*Device, FMetalTextureCreateDesc(*Device, CreateDesc));

	if (CreateDesc.InitAction == ERHITextureInitAction::Default)
	{
		return UE::RHICore::FBaseTextureInitializerImplementation(RHICmdList, Texture,
			[Texture = TRefCountPtr<FMetalSurface>(Texture)](FRHICommandListBase& RHICmdList) mutable
			{
				Texture->Initialize(RHICmdList);
				return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
			}
		);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::BulkData)
	{
		check(CreateDesc.BulkData);

		UE::RHICore::FDefaultLayoutTextureInitializer Initializer = CreateWritableTextureInitializer(RHICmdList, Texture, CreateDesc);
		check(CreateDesc.BulkData->GetResourceBulkDataSize() <= Initializer.GetWritableSize());

		FMemory::Memcpy(Initializer.GetWritableData(), CreateDesc.BulkData->GetResourceBulkData(), CreateDesc.BulkData->GetResourceBulkDataSize());

		CreateDesc.BulkData->Discard();

		return MoveTemp(Initializer);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
	{
		return CreateWritableTextureInitializer(RHICmdList, Texture, CreateDesc);
	}

	return UE::RHICore::HandleUnknownTextureInitializerInitAction(RHICmdList, CreateDesc);
}

FTextureRHIRef FMetalDynamicRHI::RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent)
{
	UE_LOGF(LogMetal, Fatal, "RHIAsyncCreateTexture2D is not supported");
	return FTextureRHIRef();
}

FRHILockTextureResult FMetalDynamicRHI::RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	FMetalSurface* Texture = ResourceCast(Arguments.Texture);
	const uint32 ResourceIndex = UE::RHICore::GetLockArrayIndex(Texture->GetDesc(), Arguments);

	if (Texture->GetDesc().Dimension == ETextureDimension::Texture2D)
	{
		return Texture->AsyncLock(RHICmdList, Arguments);
	}

	return Texture->Lock(Arguments, false);
}

void FMetalDynamicRHI::RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	MTL_SCOPED_AUTORELEASE_POOL;

	FMetalSurface* Texture = ResourceCast(Arguments.Texture);

	Texture->Unlock(Arguments);
}

#if PLATFORM_MAC
static void InternalExpandR8ToStandardRGBA(uint32* pDest, const struct FUpdateTextureRegion2D& UpdateRegion, uint32& InOutSourcePitch, const uint8* pSrc)
{
	// Should only be required for non Apple Silicon Macs
	const uint32 ExpandedPitch = UpdateRegion.Width * sizeof(uint32);
	
	for(uint y = 0; y < UpdateRegion.Height; y++)
	{
		for(uint x = 0; x < UpdateRegion.Width; x++)
		{
			uint8 Value = pSrc[(y * InOutSourcePitch) + x];
			*(pDest++) = (Value | (Value << 8) | (Value << 16) | (Value << 24));
		}
	}
	
	InOutSourcePitch = ExpandedPitch;
}

static void InternalExpandR8ToStandardRGBA3D(uint32* pDest, const struct FUpdateTextureRegion3D& UpdateRegion, uint32& InOutSourcePitch, uint32& InOutSourceDepthPitch, const uint8* pSrc)
{
	// Should only be required for non Apple Silicon Macs
	const uint32 ExpandedPitch = UpdateRegion.Width * sizeof(uint32);
	
	for(uint z = 0; z < UpdateRegion.Depth; z++)
	{
		uint32_t DepthOffset = z * InOutSourceDepthPitch;
		
		for(uint y = 0; y < UpdateRegion.Height; y++)
		{
			for(uint x = 0; x < UpdateRegion.Width; x++)
			{
				uint8 Value = pSrc[(y * InOutSourcePitch) + DepthOffset + x];
				*(pDest++) = (Value | (Value << 8) | (Value << 16) | (Value << 24));
			}
		}
	}
	
	InOutSourceDepthPitch = UpdateRegion.Width * UpdateRegion.Height * sizeof(uint32);
	InOutSourcePitch = ExpandedPitch;
}
#endif

static FMetalBufferPtr Internal_CreateBufferAndCopyTexture2DUpdateRegionData(FMetalDevice& Device, FRHITexture* TextureRHI, const struct FUpdateTextureRegion2D& UpdateRegion, uint32& InOutSourcePitch, const uint8* SourceData)
{
	const EPixelFormat PixelFormat = TextureRHI->GetFormat();
	const FPixelFormatInfo& FormatInfo = GPixelFormats[PixelFormat];

	check(UpdateRegion.Width  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.Height % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.DestX  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.DestY  % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.SrcX   % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.SrcY   % FormatInfo.BlockSizeY == 0);

	const uint32 SrcXInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcX,   FormatInfo.BlockSizeX);
	const uint32 SrcYInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcY,   FormatInfo.BlockSizeY);
	const uint32 WidthInBlocks  = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Width,  FormatInfo.BlockSizeX);
	const uint32 HeightInBlocks = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

	const uint8* OffsetSourceData = SourceData + FormatInfo.BlockBytes * SrcXInBlocks + InOutSourcePitch * SrcYInBlocks;
	uint32 UpdatePitch = InOutSourcePitch;

	FMetalBufferPtr OutBuffer;

	FMetalSurface* Texture = ResourceCast(TextureRHI);

#if PLATFORM_MAC
	// Expand R8_sRGB into RGBA8_sRGB for non Apple Silicon Mac.
	if (   PixelFormat == PF_G8
		&& EnumHasAnyFlags(Texture->GetFlags(), TexCreate_SRGB)
		&& Texture->Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
	{
		const uint32 ExpandedBufferSize = UpdateRegion.Height * UpdateRegion.Width * sizeof(uint32);
		OutBuffer = Device.CreatePooledBuffer(FMetalPooledBufferArgs(&Device, ExpandedBufferSize, BUF_Static, MTL::StorageModeShared));
		InternalExpandR8ToStandardRGBA((uint32*)OutBuffer->Contents(), UpdateRegion, InOutSourcePitch, OffsetSourceData);
	}
	else
#endif
	{
		const uint32 SourcePitch = InOutSourcePitch;
		const uint32 StagingPitch = static_cast<size_t>(WidthInBlocks) * FormatInfo.BlockBytes;

		const uint32 BufferSize = HeightInBlocks * StagingPitch;
		
		check(BufferSize >= StagingPitch*HeightInBlocks);
		OutBuffer = Device.CreatePooledBuffer(FMetalPooledBufferArgs(&Device, BufferSize, BUF_Static, MTL::StorageModeShared));

		uint8* pDestRow = (uint8*)OutBuffer->Contents();
		const uint8* pSourceRow = OffsetSourceData;
		
		// Limit copy to line by line by update region pitch otherwise we can go off the end of source data on the last row
		for (uint32 BlockRow = 0; BlockRow < HeightInBlocks; BlockRow++)
		{
			FMemory::Memcpy(pDestRow, pSourceRow, StagingPitch);
			pSourceRow += SourcePitch;
			pDestRow += StagingPitch;
		}

		InOutSourcePitch = StagingPitch;
	}

	return OutBuffer;
}

static void InternalUpdateTexture2D(FMetalRHICommandContext& Context, FRHITexture* TextureRHI, uint32 MipIndex, FUpdateTextureRegion2D const& UpdateRegion, uint32 SourcePitch, FMetalBufferPtr Buffer)
{
	FMetalSurface* Texture = ResourceCast(TextureRHI);
	MTLTexturePtr Tex = Texture->Texture;
	
	MTL::Region Region = MTL::Region(UpdateRegion.DestX, UpdateRegion.DestY, UpdateRegion.Width, UpdateRegion.Height);
	
	if(Tex->storageMode() == MTL::StorageModePrivate)
	{
        MTL_SCOPED_AUTORELEASE_POOL;
		
		const FPixelFormatInfo& FormatInfo = GPixelFormats[TextureRHI->GetFormat()];
		const uint32 NumRows = FMath::DivideAndRoundUp(UpdateRegion.Height, (uint32)FormatInfo.BlockSizeY);
		uint32 BytesPerImage = SourcePitch * NumRows;
		
        MTL::BlitOption Options = MTL::BlitOptionNone;
#if !PLATFORM_MAC
		if (Tex->pixelFormat() >= MTL::PixelFormatPVRTC_RGB_2BPP && Tex->pixelFormat() <= MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB)
		{
			Options = MTL::BlitOptionRowLinearPVRTC;
		}
#endif
		Context.CopyFromBufferToTexture(Buffer, 0, SourcePitch, BytesPerImage, Region.size, Tex.get(), 0, MipIndex, Region.origin, Options);
	}
	else
	{
        Tex->replaceRegion(Region, MipIndex, 0, Buffer->Contents(), SourcePitch, 0);
	}
}

void FMetalDynamicRHI::RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
    MTL_SCOPED_AUTORELEASE_POOL;

    FMetalBufferPtr Buffer = Internal_CreateBufferAndCopyTexture2DUpdateRegionData(*Device, TextureRHI, UpdateRegion, SourcePitch, SourceData);

    RHICmdList.EnqueueLambda([TextureRHI, MipIndex, UpdateRegion, SourcePitch, Buffer, InDevice=Device](FRHICommandListBase& InRHICmdList) mutable
    {
		InternalUpdateTexture2D(FMetalRHICommandContext::Get(InRHICmdList), TextureRHI, MipIndex, UpdateRegion, SourcePitch, Buffer);
		FMetalDynamicRHI::Get().DeferredDelete(Buffer);
    });

    INC_DWORD_STAT_BY(STAT_MetalTextureMemUpdate, UpdateRegion.Height*SourcePitch);
}

static FMetalBufferPtr Internal_CreateBufferAndCopyTexture3DUpdateRegionData(FMetalDevice& Device, FRHITexture* TextureRHI, const struct FUpdateTextureRegion3D& UpdateRegion, uint32& SourceRowPitch, uint32& SourceDepthPitch, const uint8* SourceData)
{
	FMetalSurface* Texture = ResourceCast(TextureRHI);
	
	const EPixelFormat PixelFormat = TextureRHI->GetFormat();
	const FPixelFormatInfo& FormatInfo = GPixelFormats[PixelFormat];
	
	FMetalBufferPtr OutBuffer;
	
#if PLATFORM_MAC
	// Expand R8_sRGB into RGBA8_sRGB for non Apple Silicon Mac.
	if (PixelFormat == PF_G8
		&& EnumHasAnyFlags(Texture->GetFlags(), TexCreate_SRGB)
		&& Texture->Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
	{
		const uint32 ExpandedBufferSize = UpdateRegion.Height * UpdateRegion.Width * UpdateRegion.Depth * sizeof(uint32);
		
		const uint32 SrcXInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcX,   	FormatInfo.BlockSizeX);
		const uint32 SrcYInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcY,   	FormatInfo.BlockSizeY);
		const uint32 SrcZInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcZ,   	FormatInfo.BlockSizeZ);

		const uint8* OffsetSourceData = SourceData + FormatInfo.BlockBytes * SrcXInBlocks +
										SourceRowPitch * SrcYInBlocks +
										SourceDepthPitch * SrcZInBlocks;
		
		OutBuffer = Device.CreatePooledBuffer(FMetalPooledBufferArgs(&Device, ExpandedBufferSize, BUF_Static, MTL::StorageModeShared));
		InternalExpandR8ToStandardRGBA3D((uint32*)OutBuffer->Contents(), UpdateRegion, SourceRowPitch, SourceDepthPitch, OffsetSourceData);
	}
	else
#endif
	{
		const uint32 BufferSize = SourceDepthPitch * UpdateRegion.Depth;
		OutBuffer = Device.CreatePooledBuffer(FMetalPooledBufferArgs(&Device, BufferSize, BUF_Static, MTL::StorageModeShared));
		
		uint32 CopyPitch = FMath::DivideAndRoundUp(UpdateRegion.Width, (uint32)FormatInfo.BlockSizeX) * FormatInfo.BlockBytes;
		
		check(FormatInfo.BlockSizeZ == 1);
		check(CopyPitch <= SourceRowPitch);
		
		uint8_t* DestData = (uint8_t*)OutBuffer->Contents();
		const uint32 NumRows = FMath::DivideAndRoundUp(UpdateRegion.Height, (uint32)FormatInfo.BlockSizeY);
		
		// Perform safe line copy
		for (uint32 i = 0;i < UpdateRegion.Depth;++i)
		{
			const uint8* pSourceRowData = SourceData + (SourceDepthPitch * i);
			uint8* pDestRowData = DestData + (SourceDepthPitch * i);
			
			for (uint32 j = 0;j < NumRows;++j)
			{
				FMemory::Memcpy(pDestRowData, pSourceRowData, CopyPitch);
				pSourceRowData += SourceRowPitch;
				pDestRowData += SourceRowPitch;
			}
		}
	}
	
	return OutBuffer;
}


static void InternalUpdateTexture3D(FMetalRHICommandContext& Context, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, FMetalBufferPtr Buffer)
{
	FMetalSurface* Texture = ResourceCast(TextureRHI);
	MTLTexturePtr Tex = Texture->Texture;
	
	MTL::Region Region = MTL::Region(UpdateRegion.DestX, UpdateRegion.DestY, UpdateRegion.DestZ, UpdateRegion.Width, UpdateRegion.Height, UpdateRegion.Depth);
	
	if(Tex->storageMode() == MTL::StorageModePrivate)
	{
		const FPixelFormatInfo& FormatInfo = GPixelFormats[TextureRHI->GetFormat()];
		const uint32 NumRows = FMath::DivideAndRoundUp(UpdateRegion.Height, (uint32)FormatInfo.BlockSizeY);
		const uint32 BytesPerImage = SourceRowPitch * NumRows;
		
        MTL::BlitOption Options = MTL::BlitOptionNone;
#if !PLATFORM_MAC
		if (Tex->pixelFormat() >= MTL::PixelFormatPVRTC_RGB_2BPP && Tex->pixelFormat() <= MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB)
		{
			Options = MTL::BlitOptionRowLinearPVRTC;
		}
#endif
		Context.CopyFromBufferToTexture(Buffer, 0, SourceRowPitch, BytesPerImage, Region.size, Tex.get(), 0, MipIndex, Region.origin, Options);
	}
	else
	{
        Tex->replaceRegion(Region, MipIndex, 0, (uint8*)Buffer->Contents(), SourceRowPitch, SourceDepthPitch);
	}
}

FUpdateTexture3DData FMetalDynamicRHI::RHIBeginUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion)
{
	const int32 FormatSize = PixelFormatBlockBytes[Texture->GetFormat()];
	const int32 RowPitch = UpdateRegion.Width * FormatSize;
	const int32 DepthPitch = UpdateRegion.Width * UpdateRegion.Height * FormatSize;
	
	SIZE_T MemorySize = DepthPitch * UpdateRegion.Depth;
	uint8* Data = (uint8*)FMemory::Malloc(MemorySize);
	
	return FUpdateTexture3DData(Texture, MipIndex, UpdateRegion, RowPitch, DepthPitch, Data, MemorySize, GFrameNumberRenderThread);
}

void FMetalDynamicRHI::RHIEndUpdateTexture3D(FRHICommandListBase& RHICmdList, FUpdateTexture3DData& UpdateData)
{
	check(IsInParallelRenderingThread());
	check(GFrameNumberRenderThread == UpdateData.FrameNumber);
	GDynamicRHI->RHIUpdateTexture3D(RHICmdList, UpdateData.Texture, UpdateData.MipIndex, UpdateData.UpdateRegion, UpdateData.RowPitch, UpdateData.DepthPitch, UpdateData.Data);
	FMemory::Free(UpdateData.Data);
	UpdateData.Data = nullptr;
}

void FMetalDynamicRHI::RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI,uint32 MipIndex, const FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    FMetalBufferPtr Buffer = Internal_CreateBufferAndCopyTexture3DUpdateRegionData(*Device, TextureRHI, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData);
	
    RHICmdList.EnqueueLambda([TextureRHI, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, Buffer, InDevice=Device](FRHICommandListBase& InRHICmdList) mutable
    {
        InternalUpdateTexture3D(FMetalRHICommandContext::Get(InRHICmdList), TextureRHI, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, Buffer);
		FMetalDynamicRHI::Get().DeferredDelete(Buffer);
    });

    INC_DWORD_STAT_BY(STAT_MetalTextureMemUpdate, UpdateRegion.Height * UpdateRegion.Width * SourceDepthPitch);
}

/*-----------------------------------------------------------------------------
 Cubemap texture support.
 -----------------------------------------------------------------------------*/

void FMetalDynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	MTL_SCOPED_AUTORELEASE_POOL;

	TextureRHI->SetName(Name);

	FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(TextureRHI);

	NS::String* LabelString = FStringToNSString(Name);
	if (Surface->Texture)
	{
		Surface->Texture->setLabel(LabelString);
	}
	if (Surface->MSAATexture)
	{
		Surface->MSAATexture->setLabel(LabelString);
	}
#endif
}

inline bool MetalRHICopyTexutre_IsTextureFormatCompatible(EPixelFormat SrcFmt, EPixelFormat DstFmt)
{
	//
	// For now, we only support copies between textures of mismatching
	// formats if they are of size-compatible internal formats.  This allows us
	// to copy from uncompressed to compressed textures, specifically in support
	// of the runtime virtual texture system.  Note that copies of compatible
	// formats incur the cost of an extra copy, as we must copy from the source
	// texture to a temporary buffer and finally to the destination texture.
	//
	return ((SrcFmt == DstFmt) || (GPixelFormats[SrcFmt].BlockBytes == GPixelFormats[DstFmt].BlockBytes));
}

void FMetalRHICommandContext::RHICopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    check(SourceTextureRHI);
    check(DestTextureRHI);
    
    FMetalSurface* MetalSrcTexture = GetMetalSurfaceFromRHITexture(SourceTextureRHI);
    FMetalSurface* MetalDestTexture = GetMetalSurfaceFromRHITexture(DestTextureRHI);
    
	// Need to ensure that the src/dest texture are available if they are presentable
	StateCache.ConditionalUpdateBackBuffer(*MetalSrcTexture);
	StateCache.ConditionalUpdateBackBuffer(*MetalDestTexture);
	
    const bool TextureFormatExactMatch = (SourceTextureRHI->GetFormat() == DestTextureRHI->GetFormat());
    const bool TextureFormatCompatible = MetalRHICopyTexutre_IsTextureFormatCompatible(SourceTextureRHI->GetFormat(), DestTextureRHI->GetFormat());
    
    if (TextureFormatExactMatch || TextureFormatCompatible)
    {
        const FIntVector Size = CopyInfo.Size == FIntVector::ZeroValue ? MetalSrcTexture->GetDesc().GetSize() >> CopyInfo.SourceMipIndex : CopyInfo.Size;

        MTLTexturePtr SrcTexture;

        if (TextureFormatExactMatch)
        {
            MTL::TextureUsage Usage = MetalSrcTexture->Texture->usage();
            if (Usage & MTL::TextureUsagePixelFormatView)
            {
                NS::Range Slices(0, MetalSrcTexture->Texture->arrayLength() * (MetalSrcTexture->GetDesc().IsTextureCube() ? 6 : 1));
                if (MetalSrcTexture->Texture->pixelFormat() != MetalDestTexture->Texture->pixelFormat())
                {
                    SrcTexture = NS::TransferPtr(MetalSrcTexture->Texture->newTextureView(MetalDestTexture->Texture->pixelFormat(), MetalSrcTexture->Texture->textureType(), NS::Range(0, MetalSrcTexture->Texture->mipmapLevelCount()), Slices));
                }
            }
            if (!SrcTexture)
            {
                SrcTexture = MetalSrcTexture->Texture;
            }
        }
        
        for (uint32 SliceIndex = 0; SliceIndex < CopyInfo.NumSlices; ++SliceIndex)
        {
            uint32 SourceSliceIndex = CopyInfo.SourceSliceIndex + SliceIndex;
            uint32 DestSliceIndex = CopyInfo.DestSliceIndex + SliceIndex;

            for (uint32 MipIndex = 0; MipIndex < CopyInfo.NumMips; ++MipIndex)
            {
                uint32 SourceMipIndex = CopyInfo.SourceMipIndex + MipIndex;
                uint32 DestMipIndex = CopyInfo.DestMipIndex + MipIndex;
                MTL::Size SourceSize(FMath::Max(Size.X >> MipIndex, 1), FMath::Max(Size.Y >> MipIndex, 1), FMath::Max(Size.Z >> MipIndex, 1));
                MTL::Size DestSize = SourceSize;

                MTL::Origin SourceOrigin(CopyInfo.SourcePosition.X >> MipIndex, CopyInfo.SourcePosition.Y >> MipIndex, CopyInfo.SourcePosition.Z >> MipIndex);
                MTL::Origin DestinationOrigin(CopyInfo.DestPosition.X >> MipIndex, CopyInfo.DestPosition.Y >> MipIndex, CopyInfo.DestPosition.Z >> MipIndex);

                if (TextureFormatCompatible)
                {
                    DestSize.width  *= GPixelFormats[MetalDestTexture->GetDesc().Format].BlockSizeX;
                    DestSize.height *= GPixelFormats[MetalDestTexture->GetDesc().Format].BlockSizeY;
                }

                // Account for create with TexCreate_SRGB flag which could make these different
                if (TextureFormatExactMatch && (SrcTexture->pixelFormat() == MetalDestTexture->Texture->pixelFormat()))
                {
                    CopyFromTextureToTexture(SrcTexture.get(), SourceSliceIndex, SourceMipIndex, SourceOrigin,SourceSize, MetalDestTexture->Texture.get(), DestSliceIndex, DestMipIndex, DestinationOrigin);
                }
                else
                {
                    //
                    // In the case of compatible texture formats or pixel
                    // format mismatch (like linear vs. sRGB), then we must
                    // achieve the copy by going through a buffer object.
                    //
                    const bool BlockSizeMatch = (GPixelFormats[MetalSrcTexture->GetDesc().Format].BlockSizeX == GPixelFormats[MetalDestTexture->GetDesc().Format].BlockSizeX);
                    const uint32 BytesPerPixel = (MetalSrcTexture->GetDesc().Format != PF_DepthStencil) ? GPixelFormats[MetalSrcTexture->GetDesc().Format].BlockBytes : 1;
                    const uint32 Stride = BytesPerPixel * SourceSize.width;
#if PLATFORM_MAC
                    const uint32 Alignment = 1u;
#else
                    // don't mess with alignment if we copying between formats with a different block size
                    const uint32 Alignment = BlockSizeMatch ? 64u : 1u;
#endif
                    const uint32 AlignedStride = ((Stride - 1) & ~(Alignment - 1)) + Alignment;
                    const uint32 BytesPerImage = AlignedStride *  SourceSize.height;
                    const uint32 DataSize = BytesPerImage * SourceSize.depth;
                    
                    FMetalBufferPtr Buffer = Device.CreatePooledBuffer(FMetalPooledBufferArgs(&Device, DataSize, BUF_Dynamic, MTL::StorageModeShared));
                    
                    check(Buffer);
                    
                    MTL::BlitOption Options = MTL::BlitOptionNone;
#if !PLATFORM_MAC
                    if (MetalSrcTexture->Texture->pixelFormat() >= MTL::PixelFormatPVRTC_RGB_2BPP && MetalSrcTexture->Texture->pixelFormat() <= MTL::PixelFormatPVRTC_RGBA_4BPP_sRGB)
                    {
                        Options = MTL::BlitOptionRowLinearPVRTC;
                    }
#endif
                    CopyFromTextureToBuffer(MetalSrcTexture->Texture.get(), SourceSliceIndex, SourceMipIndex, SourceOrigin, SourceSize, Buffer, 0, AlignedStride, BytesPerImage, Options);
                    CopyFromBufferToTexture(Buffer, 0, Stride, BytesPerImage, DestSize, MetalDestTexture->Texture.get(), DestSliceIndex, DestMipIndex, DestinationOrigin, Options);
                    
					FMetalDynamicRHI::Get().DeferredDelete(Buffer);
                }
            }
        }
        
        if (SrcTexture && (SrcTexture != MetalSrcTexture->Texture))
        {
			FMetalDynamicRHI::Get().DeferredDelete(SrcTexture);
        }
    }
    else
    {
        UE_LOGF(LogMetal, Error, "RHICopyTexture Source (UnrealEngine %d: MTL %d) <-> Destination (UnrealEngine %d: MTL %d) texture format mismatch", (uint32)SourceTextureRHI->GetFormat(), (uint32)MetalSrcTexture->Texture->pixelFormat(), (uint32)DestTextureRHI->GetFormat(), (uint32)MetalDestTexture->Texture->pixelFormat());
    }
}

void FMetalRHICommandContext::RHICopyBufferRegion(FRHIBuffer* DstBufferRHI, uint64 DstOffset, FRHIBuffer* SrcBufferRHI, uint64 SrcOffset, uint64 NumBytes)
{
	if (!DstBufferRHI || !SrcBufferRHI || DstBufferRHI == SrcBufferRHI || !NumBytes)
	{
		return;
	}

    MTL_SCOPED_AUTORELEASE_POOL;
    
    FMetalRHIBuffer* DstBuffer = ResourceCast(DstBufferRHI);
    FMetalRHIBuffer* SrcBuffer = ResourceCast(SrcBufferRHI);

    check(DstBuffer && SrcBuffer);
    check(DstOffset + NumBytes <= DstBufferRHI->GetSize() && SrcOffset + NumBytes <= SrcBufferRHI->GetSize());

    CopyFromBufferToBuffer(SrcBuffer->GetCurrentBuffer(), SrcOffset, DstBuffer->GetCurrentBuffer(), DstOffset, NumBytes);
}

class FMetalTextureReference : public FRHITextureReference
{
public:
	FMetalTextureReference(FRHITexture* InReferencedTexture)
		: FRHITextureReference(InReferencedTexture)
	{
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FMetalTextureReference(FRHITexture* InReferencedTexture, FMetalShaderResourceView* InBindlessView)
		: FRHITextureReference(InReferencedTexture, InBindlessView->GetBindlessHandle())
		, BindlessView(InBindlessView)
	{
	}

	TRefCountPtr<FMetalShaderResourceView> BindlessView;
#endif
};

template<>
struct TMetalResourceTraits<FRHITextureReference>
{
	using TConcreteType = FMetalTextureReference;
};

FTextureReferenceRHIRef FMetalDynamicRHI::RHICreateTextureReference(FRHICommandListBase& RHICmdList, FRHITexture* InReferencedTexture)
{
	FRHITexture* ReferencedTexture = InReferencedTexture ? InReferencedTexture : FRHITextureReference::GetDefaultTexture();

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device->GetBindlessDescriptorManager();
	check(BindlessDescriptorManager);
	
	if(IsMetalBindlessEnabled())
	{
		// If the referenced texture is configured for bindless, make sure we also create an SRV to use for bindless.
		if (ReferencedTexture && ReferencedTexture->GetDefaultBindlessHandle().IsValid())
		{
			FShaderResourceViewRHIRef BindlessView = RHICmdList.CreateShaderResourceView(
				ReferencedTexture,
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(ReferencedTexture)
					.SetMipRange(0, 1));
			return new FMetalTextureReference(ReferencedTexture, ResourceCast(BindlessView.GetReference()));
		}
	}
#endif

	return new FMetalTextureReference(ReferencedTexture);
}

void FMetalDynamicRHI::RHIUpdateTextureReference(FRHICommandListBase& RHICmdList, FRHITextureReference* TextureRef, FRHITexture* InNewTexture)
{
	FRHITexture* NewTexture = InNewTexture ? InNewTexture : FRHITextureReference::GetDefaultTexture();
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (TextureRef && TextureRef->IsBindless())
	{
		RHICmdList.EnqueueLambda(TEXT("FMetalDynamicRHI::RHIUpdateTextureReference"), [NewTexture, TextureRef, InNewTexture, InDevice=Device](FRHICommandListBase& ExecutingCmdList)
		{		
			FMetalTextureReference* MetalTextureReference = ResourceCast(TextureRef);
			FMetalShaderResourceView* MetalTextureRefSRV = MetalTextureReference->BindlessView;
			
			FRHIDescriptorHandle DestHandle = MetalTextureRefSRV->GetBindlessHandle();
			if(DestHandle.IsValid())
			{
				FMetalSurface* NewSurface = GetMetalSurfaceFromRHITexture(NewTexture);
				
				MetalTextureRefSRV->SurfaceOverride = NewSurface;
				FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(ExecutingCmdList);
				MetalTextureRefSRV->UpdateView(&Context, false);
			}
		});
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

    FDynamicRHI::RHIUpdateTextureReference(RHICmdList, TextureRef, NewTexture);
}

