// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MetalTransitionData.cpp: Metal RHI Resource Transition Implementation.
==============================================================================*/

#include "MetalTransitionData.h"
#include "MetalRHIContext.h"
#include "MetalRHIPrivate.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Resource Transition Data Definitions -

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Resource Transition Data Implementation -


FMetalTransitionData::FMetalTransitionData(ERHIPipeline                         InSrcPipelines,
										   ERHIPipeline                         InDstPipelines,
										   ERHITransitionCreateFlags            InCreateFlags,
										   TArrayView<const FRHITransitionInfo> InInfos)
{
	SrcPipelines   = InSrcPipelines;
	DstPipelines   = InDstPipelines;
	CreateFlags    = InCreateFlags;

	bCrossPipeline = (SrcPipelines != DstPipelines);

	Infos.Append(InInfos.GetData(), InInfos.Num());
}

void FMetalTransitionData::BeginResourceTransitions() const
{
}

void FMetalTransitionData::EndResourceTransitions(FMetalCommandEncoder& CurrentEncoder, bool& UAVBarrier) const
{
	check(SrcPipelines == DstPipelines);

	for (const auto& Info : Infos)
	{
		if (nullptr == Info.Resource)
		{
			continue;
		}

		if (Info.AccessAfter == ERHIAccess::Discard)
		{
			// Discard as a destination is a no-op
			continue;
		}

		checkf(Info.AccessAfter != ERHIAccess::Unknown, TEXT("Transitioning a resource to an unknown state is not allowed."));

		UAVBarrier = UAVBarrier || EnumHasAnyFlags(Info.AccessBefore, ERHIAccess::UAVMask) || EnumHasAnyFlags(Info.AccessAfter, ERHIAccess::UAVMask);
		
		switch (Info.Type)
		{
			case FRHITransitionInfo::EType::UAV:
			{
				FMetalUnorderedAccessView* UAV = ResourceCast(Info.UAV);
				
				if (UAV->IsTexture())
				{
					FMetalSurface* Surface = ResourceCast(UAV->GetTexture());
					if (Surface->Texture)
					{
						CurrentEncoder.TransitionResources(Surface->Texture.get());
						if (Surface->MSAATexture)
						{
							CurrentEncoder.TransitionResources(Surface->MSAATexture.get());
						}
					}
				}
				else
				{
					FMetalRHIBuffer* Buffer = ResourceCast(UAV->GetBuffer());
					CurrentEncoder.TransitionResources(Buffer->GetCurrentBuffer()->GetMTLBuffer());
				}
				break;
			}
			case FRHITransitionInfo::EType::Buffer:
			{
				auto Resource = ResourceCast(Info.Buffer);
				if (Resource->GetCurrentBufferOrNull())
				{
					CurrentEncoder.TransitionResources(Resource->GetCurrentBuffer()->GetMTLBuffer());
				}
				
				break;
			}
			case FRHITransitionInfo::EType::Texture:
			{
				FMetalSurface* Surface = GetMetalSurfaceFromRHITexture(Info.Texture);
				
				if ((Surface != nullptr) && Surface->Texture)
				{
					CurrentEncoder.TransitionResources(Surface->Texture.get());
					if (Surface->MSAATexture)
					{
						CurrentEncoder.TransitionResources(Surface->MSAATexture.get());
					}
				}
				break;
			}
			case FRHITransitionInfo::EType::BVH:
			{
				// TODO: Carl - Metal 4
				break;
			}
			default:
			{
				checkNoEntry();
				break;
			}
		}
	}
}

void FMetalRHICommandContext::RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions)
{
	for (auto Transition : Transitions)
	{
		Transition->GetPrivateData<FMetalTransitionData>()->BeginResourceTransitions();
	}
}

void FMetalRHICommandContext::RHIEndTransitions(TArrayView<const FRHITransition*> Transitions)
{
	bool UAVBarrier = false;
	
	for (auto Transition : Transitions)
	{
		Transition->GetPrivateData<FMetalTransitionData>()->EndResourceTransitions(CurrentEncoder, UAVBarrier);
	}
	
	if (UAVBarrier)
	{
		InsertComputeMemoryBarrier();
	}
}

void FMetalRHICommandContext::InsertComputeMemoryBarrier()
{
	if (CurrentEncoder.IsComputeCommandEncoderActive())
	{
		CurrentEncoder.GetComputeCommandEncoder()->memoryBarrier(MTL::BarrierScopeBuffers | MTL::BarrierScopeTextures);
	}
}
