// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	MetalTransitionData.h: Metal RHI Resource Transition Definitions.
==============================================================================*/

#pragma once

#include "RHITransition.h"
#include "MetalFence.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Resource Transition Info Array Type Definition -


typedef TArray<FRHITransitionInfo, TInlineAllocator<4> > FMetalTransitionInfoArray;


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Resource Transition Data Class -

class FMetalCommandEncoder;
class FMetalTransitionData
{
public:
	explicit FMetalTransitionData(ERHIPipeline                         InSrcPipelines,
								  ERHIPipeline                         InDstPipelines,
								  ERHITransitionCreateFlags            InCreateFlags,
								  TArrayView<const FRHITransitionInfo> InInfos);

	// The default destructor is sufficient.
	~FMetalTransitionData() = default;

	// Disallow default, copy and move constructors.
	FMetalTransitionData()                             = delete;
	FMetalTransitionData(const FMetalTransitionData&)  = delete;
	FMetalTransitionData(const FMetalTransitionData&&) = delete;

	// Begin resource transitions.
	void BeginResourceTransitions() const;

	// End resource transitions.
	void EndResourceTransitions(FMetalCommandEncoder& CurrentEncoder, bool& UAVBarrier) const;

private:
	ERHIPipeline              SrcPipelines   = ERHIPipeline::Num;
	ERHIPipeline              DstPipelines   = ERHIPipeline::Num;
	ERHITransitionCreateFlags CreateFlags    = ERHITransitionCreateFlags::None;
	bool                      bCrossPipeline = false;
	FMetalTransitionInfoArray Infos          = {};
	TRefCountPtr<FMetalFence> Fence          = nullptr;
};
