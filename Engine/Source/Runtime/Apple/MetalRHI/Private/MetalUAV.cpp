// Copyright Epic Games, Inc. All Rights Reserved.


#include "MetalRHIPrivate.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalDynamicRHI.h"
#include "MetalRHIContext.h"
#include "MetalRHIStagingBuffer.h"
#include "MetalCommandBuffer.h"
#include "RenderUtils.h"
#include "ClearReplacementShaders.h"
#include "MetalTransitionData.h"
#include "MetalBindlessDescriptors.h"

static int32 GMetalUseEventForGPUFence = 1;
static FAutoConsoleVariableRef CVarMetalUseEventForGPUFence(
	TEXT("rhi.Metal.UseEventForGPUFence"),
	GMetalUseEventForGPUFence,
	TEXT("When enabled use an Event to signal GPUFence completion to save splitting command buffers default: on"));


void FMetalViewableResource::UpdateLinkedViews(FMetalRHICommandContext* Context)
{
	for (FMetalResourceViewBase* View = LinkedViews; View; View = View->Next())
	{
		View->UpdateView(Context, false);
	}
}


FMetalResourceViewBase::~FMetalResourceViewBase()
{
	Invalidate();
	Unlink();
}

void FMetalResourceViewBase::Invalidate()
{
	if (bOwnsResource)
	{
		// @todo - SRV/UAV refactor - is releasing objects like this safe / correct?
		switch (GetMetalType())
		{
		case EMetalType::TextureView:
			FMetalDynamicRHI::Get().DeferredDelete(Storage.Get<MTLTexturePtr>());
			break;

		case EMetalType::BufferView:
			FMetalDynamicRHI::Get().DeferredDelete(Storage.Get<FBufferView>().Buffer);
			break;
                
        case EMetalType::TextureBufferBacked:
            FTextureBufferBacked & View = Storage.Get<FTextureBufferBacked>();
			// If it is a buffer we don't own the resource
			if (View.bIsBuffer)
			{
				FMetalDynamicRHI::Get().DeferredDelete(View.Texture);
			}
			else
			{
				FMetalDynamicRHI::Get().DeferredDelete(View.Buffer);
				FMetalDynamicRHI::Get().DeferredDelete(View.Texture);
			}
            break;
		}
	}

	Storage.Emplace<FEmptyVariantState>();
	bOwnsResource = true;
}

void FMetalResourceViewBase::InitAsTextureView(MTLTexturePtr Texture)
{
	check(GetMetalType() == EMetalType::Null);
	Storage.Emplace<MTLTexturePtr>(Texture);
}

void FMetalResourceViewBase::InitAsBufferView(FMetalBufferPtr Buffer, uint32 Offset, uint32 Size)
{
	check(GetMetalType() == EMetalType::Null);
	Storage.Emplace<FBufferView>(Buffer, Offset, Size);
	bOwnsResource = false;
}

void FMetalResourceViewBase::InitAsTextureBufferBacked(MTLTexturePtr Texture, FMetalBufferPtr Buffer, uint32 Offset, uint32 Size, EPixelFormat Format, bool bIsBuffer)
{
    check(GetMetalType() == EMetalType::Null);
    Storage.Emplace<FTextureBufferBacked>(Texture, Buffer, Offset, Size, Format, bIsBuffer);
}

#if METAL_RHI_RAYTRACING
void FMetalResourceViewBase::InitAsAccelerationStructure(FMetalAccelerationStructure* AccelerationStructure)
{
	check(GetMetalType() == EMetalType::Null);
	Storage.Emplace<FMetalAccelerationStructure*>(AccelerationStructure);
}
#endif

FMetalShaderResourceView::FMetalShaderResourceView(FMetalDevice& InDevice, FRHICommandListBase& RHICmdList, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	: FRHIShaderResourceView(InResource, InViewDesc),
	  FMetalResourceViewBase(InDevice)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);
	
		BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::TextureSRV);
	}
	
	SurfaceOverride = nullptr;
#endif

	RHICmdList.EnqueueLambda([this](FRHICommandListBase& InCmdList)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(InCmdList);
		UpdateView(&Context, true);
	});
	
	RHICmdList.RHIThreadFence(true);
}

FMetalShaderResourceView::~FMetalShaderResourceView()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);

		BindlessDescriptorManager->FreeDescriptor(BindlessHandle);
	}
#endif
}

FMetalViewableResource* FMetalShaderResourceView::GetBaseResource() const
{
	return IsBuffer()
		? static_cast<FMetalViewableResource*>(ResourceCast(GetBuffer()))
		: static_cast<FMetalViewableResource*>(ResourceCast(GetTexture()));
}

MTL::TextureType UAVDimensionToMetalTextureType(FRHIViewDesc::EDimension Dimension)
{
    switch (Dimension)
    {
        case FRHIViewDesc::EDimension::Texture2D:
            return MTL::TextureType2D;
        case FRHIViewDesc::EDimension::Texture2DArray:
        case FRHIViewDesc::EDimension::TextureCube:
        case FRHIViewDesc::EDimension::TextureCubeArray:
            return MTL::TextureType2DArray;
        case FRHIViewDesc::EDimension::Texture3D:
            return MTL::TextureType3D;
        default:
            checkNoEntry();
    }
    
    return MTL::TextureType2D;
}

MTL::TextureType SRVDimensionToMetalTextureType(FMetalDevice& Device, FRHIViewDesc::EDimension Dimension)
{
    switch (Dimension)
    {
        case FRHIViewDesc::EDimension::Texture2D:
            return MTL::TextureType2D;
        case FRHIViewDesc::EDimension::Texture2DArray:
            return MTL::TextureType2DArray;
        case FRHIViewDesc::EDimension::TextureCube:
            return MTL::TextureTypeCube;
        case FRHIViewDesc::EDimension::TextureCubeArray:
			if(Device.SupportsFeature(EMetalFeaturesCubemapArrays))
			{
				return MTL::TextureTypeCubeArray;
			}
			else
			{
				return MTL::TextureType2DArray;
			}
        case FRHIViewDesc::EDimension::Texture3D:
            return MTL::TextureType3D;
        default:
            checkNoEntry();
    }
    
    return MTL::TextureType2D;
}

void FMetalShaderResourceView::UpdateView(FMetalRHICommandContext* Context, const bool bConstructing)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	Invalidate();

	if (IsBuffer())
	{
		FMetalRHIBuffer* Buffer = ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.SRV.GetViewInfo(Buffer);

		if(Info.bNullView)
		{
			return;
		}

		switch (Info.BufferType)
		{
			case FRHIViewDesc::EBufferType::Typed:
			{
				MTL::PixelFormat Format = (MTL::PixelFormat)GMetalBufferFormats[Info.Format].LinearTextureFormat;
				NS::UInteger Options = ((NS::UInteger)Buffer->Mode) << MTL::ResourceStorageModeShift;

				const uint32 MinimumByteAlignment = Device.GetDevice()->minimumLinearTextureAlignmentForPixelFormat(Format);
				const uint32 MinimumElementAlignment = MinimumByteAlignment / Info.StrideInBytes;
				uint32 NumElements = Align(Info.NumElements, MinimumElementAlignment);
				uint32 SizeInBytes = NumElements * Info.StrideInBytes;
				
				MTL::TextureDescriptor* Desc = MTL::TextureDescriptor::textureBufferDescriptor(
					  Format
					, NumElements
					, MTL::ResourceOptions(Options)
					, MTL::TextureUsageShaderRead
				);

				Desc->setAllowGPUOptimizedContents(false);

				FMetalBufferPtr TransferBuffer = Buffer->GetCurrentBuffer();
				MTLTexturePtr View = NS::TransferPtr(TransferBuffer->GetMTLBuffer()->newTexture(Desc, Info.OffsetInBytes+TransferBuffer->GetOffset(), SizeInBytes));
				
				InitAsTextureView(View);
				break;
			}
			case FRHIViewDesc::EBufferType::Raw:
			case FRHIViewDesc::EBufferType::Structured:
			{
				InitAsBufferView(Buffer->GetCurrentBuffer(), Info.OffsetInBytes, Info.SizeInBytes);
				break;
			}
#if METAL_RHI_RAYTRACING
			case FRHIViewDesc::EBufferType::AccelerationStructure:
			{
				check(Info.OffsetInBytes == 0);
				InitAsAccelerationStructure(Buffer->AccelerationStructure);
				break;
			}
#endif
			
			default:
				checkNoEntry();
				break;
		}
	}
#if METAL_USE_METAL_SHADER_CONVERTER
	else if (SurfaceOverride != nullptr)
	{
		MTLTexturePtr View = SurfaceOverride->Texture;
		InitAsTextureView(View);
		bOwnsResource = false;
	}
#endif
	else
	{
		FMetalSurface* Texture = ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.SRV.GetViewInfo(Texture);

#if PLATFORM_IOS
		// Memoryless targets can't have texture views (SRVs or UAVs)
		check(Texture->Texture->storageMode() != MTL::StorageModeMemoryless);
#endif

		MTL::PixelFormat MetalFormat = UEToMetalFormat(Device, Info.Format, Info.bSRGB);
        MTL::TextureType TextureType = Texture->Texture->textureType();

        if (EnumHasAnyFlags(Texture->GetDesc().Flags, TexCreate_SRGB) && !Info.bSRGB)
        {
#if PLATFORM_MAC
            // R8Unorm has been expanded in the source surface for sRGBA support - we need to expand to RGBA to enable compatible texture format view for non apple silicon macs
            if (Info.Format == PF_G8 && Texture->Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
            {
                MetalFormat = MTL::PixelFormatRGBA8Unorm;
            }
#endif
        }
        
        bool bUseSourceTexture = Info.bAllMips && Info.bAllSlices && MetalFormat == Texture->Texture->pixelFormat() &&
                                SRVDimensionToMetalTextureType(Device, Info.Dimension) == TextureType;
		
		check(TextureType != MTL::TextureType1D);
		
		bool bIsBindless = IsMetalBindlessEnabled();
        
		// We can use the source texture directly if the view's format / mip count etc matches.
		if (bUseSourceTexture)
		{
			// View is exactly compatible with the original texture.
			MTLTexturePtr View = Texture->Texture;
            InitAsTextureView(View);
			bOwnsResource = false;
		}
		else
		{
            uint32_t ArrayStart = Info.ArrayRange.First;
            uint32_t ArraySize = Info.ArrayRange.Num;
            
			if (Info.Dimension == FRHIViewDesc::EDimension::TextureCube || Info.Dimension == FRHIViewDesc::EDimension::TextureCubeArray)
            {
                ArrayStart = Info.ArrayRange.First * 6;
                ArraySize = Info.ArrayRange.Num * 6;	
            }
            
            if(TextureType != MTL::TextureType2DMultisample && TextureType != MTL::TextureType2DMultisampleArray)
            {
                TextureType = SRVDimensionToMetalTextureType(Device, Info.Dimension);
            }
			
			if(!bIsBindless)
			{
				// We don't support Texture2DArray with atomic compatible so
				// ensure we are creating a view on a Texture2D with the correct size
				bool bIsAtomicCompatible = EnumHasAllFlags(Texture->GetDesc().Flags, TexCreate_AtomicCompatible)  ||
											EnumHasAllFlags(Texture->GetDesc().Flags, ETextureCreateFlags::Atomic64Compatible);
				
				if(TextureType == MTL::TextureType2D && bIsAtomicCompatible)
				{
					ArrayStart = 0;
					ArraySize = 1;
				}
			}
            
            MTLTexturePtr View = NS::TransferPtr(Texture->Texture->newTextureView(
				MetalFormat,
                TextureType,
				NS::Range(Info.MipRange.First, Info.MipRange.Num),
				NS::Range(ArrayStart, ArraySize)
			));
            InitAsTextureView(View);

#if METAL_DEBUG_OPTIONS
			View->setLabel(Texture->Texture->label());
#endif
		}
	}
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);

		BindlessDescriptorManager->UpdateDescriptor(Context, BindlessHandle, this, bConstructing ? EMetalDescriptorUpdateType::Immediate : EMetalDescriptorUpdateType::GPU);
	}
#endif
}

FMetalUnorderedAccessView::FMetalUnorderedAccessView(FMetalDevice& InDevice, FRHICommandListBase& RHICmdList,
													FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	: FRHIUnorderedAccessView(InResource, InViewDesc)
	, FMetalResourceViewBase(InDevice)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);

		BindlessHandle = BindlessDescriptorManager->AllocateDescriptor(ERHIDescriptorType::TextureUAV);
	}
#endif

	RHICmdList.EnqueueLambda([this](FRHICommandListBase& InRHICmdList)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		
		FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(InRHICmdList);
		UpdateView(&Context, true);
	});
	
	if (RHICmdList.IsTopOfPipe())
	{
		RHICmdList.RHIThreadFence(true);
	}
}

FMetalUnorderedAccessView::~FMetalUnorderedAccessView()
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if(IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);

		BindlessDescriptorManager->FreeDescriptor(BindlessHandle);
	}
#endif
}

FMetalViewableResource* FMetalUnorderedAccessView::GetBaseResource() const
{
	return IsBuffer()
		? static_cast<FMetalViewableResource*>(ResourceCast(GetBuffer()))
		: static_cast<FMetalViewableResource*>(ResourceCast(GetTexture()));
}

void FMetalUnorderedAccessView::UpdateView(FMetalRHICommandContext* Context, const bool bConstructing)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	Invalidate();

	if (IsBuffer())
	{
		FMetalRHIBuffer* Buffer = ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);

		checkf(!Info.bAtomicCounter && !Info.bAppendBuffer, TEXT("UAV counters not implemented."));

		if (!Info.bNullView)
		{
			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Typed:
			{
				MTL::PixelFormat Format = (MTL::PixelFormat)GMetalBufferFormats[Info.Format].LinearTextureFormat;
                NS::UInteger Options = ((NS::UInteger)Buffer->Mode) << MTL::ResourceStorageModeShift;

                const uint32 MinimumByteAlignment = Device.GetDevice()->minimumLinearTextureAlignmentForPixelFormat(Format);
                const uint32 MinimumElementAlignment = MinimumByteAlignment / Info.StrideInBytes;
                uint32 NumElements = Align(Info.NumElements, MinimumElementAlignment);
                uint32 SizeInBytes = NumElements * Info.StrideInBytes;
                
                MTL::TextureDescriptor* Desc = MTL::TextureDescriptor::textureBufferDescriptor(
					Format
					, NumElements
					, MTL::ResourceOptions(Options)
					, MTL::TextureUsage(MTL::TextureUsageShaderRead | MTL::TextureUsageShaderWrite)
				);

				Desc->setAllowGPUOptimizedContents(false);

                MTLTexturePtr MetalTexture = NS::TransferPtr(Buffer->GetCurrentBuffer()->GetMTLBuffer()->newTexture(Desc, Info.OffsetInBytes + Buffer->GetCurrentBuffer()->GetOffset(), SizeInBytes));
                
                InitAsTextureBufferBacked(MetalTexture, Buffer->GetCurrentBuffer(), Info.OffsetInBytes, SizeInBytes, Info.Format, true);
			}
			break;

			case FRHIViewDesc::EBufferType::Raw:
			case FRHIViewDesc::EBufferType::Structured:
			{
				InitAsBufferView(Buffer->GetCurrentBuffer(), Info.OffsetInBytes, Info.SizeInBytes);
			}
			break;

			default:
				checkNoEntry();
				break;
			}
		}
	}
	else
	{
		FMetalSurface* Texture = ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.UAV.GetViewInfo(Texture);

#if PLATFORM_IOS
		// Memoryless targets can't have texture views (SRVs or UAVs)
		check(Texture->Texture->storageMode() != MTL::StorageModeMemoryless);
#endif

        MTL::PixelFormat MetalFormat = UEToMetalFormat(Device, Info.Format, false);
        MTL::TextureType TextureType = Texture->Texture->textureType();

        if (EnumHasAnyFlags(Texture->GetDesc().Flags, TexCreate_SRGB))
        {
#if PLATFORM_MAC
            // R8Unorm has been expanded in the source surface for sRGBA support - we need to expand to RGBA to enable compatible texture format view for non apple silicon macs
            if (Info.Format == PF_G8 && Texture->Texture->pixelFormat() == MTL::PixelFormatRGBA8Unorm_sRGB)
            {
                MetalFormat = MTL::PixelFormatRGBA8Unorm;
            }
#endif
        }
        
        bool bUseSourceTexture = Info.bAllMips && Info.bAllSlices &&
                                UAVDimensionToMetalTextureType(Info.Dimension) == TextureType && MetalFormat == Texture->Texture->pixelFormat();
        
		check(TextureType != MTL::TextureType1D);
		
        bool bIsAtomicCompatible = EnumHasAllFlags(Texture->GetDesc().Flags, TexCreate_AtomicCompatible) ||
                                            EnumHasAllFlags(Texture->GetDesc().Flags, ETextureCreateFlags::Atomic64Compatible);
        
		bool bIsBindless = IsMetalBindlessEnabled();
		
		bool bBufferBacked = EnumHasAllFlags(Texture->GetDesc().Flags, TexCreate_UAV | TexCreate_NoTiling);
		if (bIsBindless)
		{
			bBufferBacked = bBufferBacked && !bIsAtomicCompatible;
		}
		else
		{
			bBufferBacked = bBufferBacked || bIsAtomicCompatible;
		}
		
        // We can use the source texture directly if the view's format / mip count etc matches.
        if (bUseSourceTexture)
		{
            // If we are using texture atomics then we need to bind them as buffers because Metal lacks texture atomics
            if(bBufferBacked && Texture->Texture->buffer())
            {
                FMetalBufferPtr MetalBuffer = FMetalBufferPtr(new FMetalBuffer(Texture->Texture->buffer(), FMetalBuffer::FreePolicy::Temporary));
                InitAsTextureBufferBacked(Texture->Texture, MetalBuffer,
                                        Texture->Texture->bufferOffset(),
                                        Texture->Texture->buffer()->length(), Info.Format, false);
            }
            else
            {
                MTLTexturePtr View = Texture->Texture;
                InitAsTextureView(View);
            }
            bOwnsResource = false;
		}
		else
		{
            uint32_t ArrayStart = Info.ArrayRange.First;
            uint32_t ArraySize = Info.ArrayRange.Num;
            
            // Check the incoming texture type for whether this a cube or cube array
			if (Info.Dimension == FRHIViewDesc::EDimension::TextureCube || Info.Dimension == FRHIViewDesc::EDimension::TextureCubeArray)
			{
				ArrayStart = Info.ArrayRange.First * 6;
				ArraySize = Info.ArrayRange.Num * 6;
			}
            
            TextureType = UAVDimensionToMetalTextureType(Info.Dimension);
            
			if(!bIsBindless)
			{
				// Metal doesn't support atomic Texture2DArray
				if(bIsAtomicCompatible && Info.Dimension == FRHIViewDesc::EDimension::Texture2DArray)
				{
					TextureType = MTL::TextureType2D;
					ArraySize = 1;
				}
			}
            
            MTLTexturePtr MetalTexture = NS::TransferPtr(Texture->Texture->newTextureView(
				MetalFormat,
                TextureType,
				NS::Range(Info.MipLevel, 1),
                NS::Range(ArrayStart, ArraySize))
			);
            
            // If we are using texture atomics then we need to bind them as buffers because Metal lacks texture atomics
            if((EnumHasAllFlags(Texture->GetDesc().Flags, TexCreate_UAV | TexCreate_NoTiling) || (!bIsBindless && bIsAtomicCompatible)) && Texture->Texture->buffer())
            {
                FMetalBufferPtr MetalBuffer = FMetalBufferPtr(new FMetalBuffer(Texture->Texture->buffer(), FMetalBuffer::FreePolicy::Temporary));
                InitAsTextureBufferBacked(MetalTexture, MetalBuffer,
                                          Texture->Texture->bufferOffset(),
                                          Texture->Texture->buffer()->length(), Info.Format, false);
            }
            else
            {
                InitAsTextureView(MetalTexture);
            }
            
#if METAL_DEBUG_OPTIONS
            MetalTexture->setLabel(Texture->Texture->label());
#endif
        }
	}
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if(IsMetalBindlessEnabled())
	{
		FMetalBindlessDescriptorManager* BindlessDescriptorManager = Device.GetBindlessDescriptorManager();
		check(BindlessDescriptorManager);
		
		BindlessDescriptorManager->UpdateDescriptor(Context, BindlessHandle, this, bConstructing ? EMetalDescriptorUpdateType::Immediate : EMetalDescriptorUpdateType::GPU);
	}
#endif
}

FShaderResourceViewRHIRef FMetalDynamicRHI::RHICreateShaderResourceView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
    return new FMetalShaderResourceView(*Device, RHICmdList, Resource, ViewDesc);
}

FUnorderedAccessViewRHIRef FMetalDynamicRHI::RHICreateUnorderedAccessView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
    return new FMetalUnorderedAccessView(*Device, RHICmdList, Resource, ViewDesc);
}

#if UE_METAL_RHI_SUPPORT_CLEAR_UAV_WITH_BLIT_ENCODER
void FMetalUnorderedAccessView::ClearUAVWithBlitEncoder(TRHICommandList_RecursiveHazardous<FMetalRHICommandContext>& RHICmdList,
														uint32 Pattern)
{
	RHICmdList.RunOnContext([this, Pattern](FMetalRHICommandContext& Context)
	{
        MTL_SCOPED_AUTORELEASE_POOL;

		FMetalRHIBuffer* SourceBuffer = ResourceCast(GetBuffer());
        auto const &Info = ViewDesc.Buffer.UAV.GetViewInfo(SourceBuffer);
		FMetalBufferPtr Buffer = SourceBuffer->GetCurrentBuffer();
		uint32 Size = Info.SizeInBytes;
		uint32 AlignedSize = Align(Size, BufferOffsetAlignment);
		FMetalPooledBufferArgs Args(&Device, AlignedSize, BUF_Dynamic, MTL::StorageModeShared);

		FMetalBufferPtr Temp = Device.CreatePooledBuffer(Args);

		uint32* ContentBytes = (uint32*)Temp->Contents();
		for (uint32 Element = 0; Element < (AlignedSize >> 2); ++Element)
		{
			ContentBytes[Element] = Pattern;
		}

		Context.CopyFromBufferToBuffer(Temp, 0, Buffer, Info.OffsetInBytes, Size);
		FMetalDynamicRHI::Get().DeferredDelete(Temp);
	});
}
#endif // UE_METAL_RHI_SUPPORT_CLEAR_UAV_WITH_BLIT_ENCODER

void FMetalRHICommandContext::RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values)
{
	TRHICommandList_RecursiveHazardous<FMetalRHICommandContext> RHICmdList(this);
	ResourceCast(UnorderedAccessViewRHI)->ClearUAV(RHICmdList, &Values, true);
}

void FMetalRHICommandContext::RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values)
{
	TRHICommandList_RecursiveHazardous<FMetalRHICommandContext> RHICmdList(this);
	ResourceCast(UnorderedAccessViewRHI)->ClearUAV(RHICmdList, &Values, false);
}

void FMetalUnorderedAccessView::ClearUAV(TRHICommandList_RecursiveHazardous<FMetalRHICommandContext>& RHICmdList, const void* ClearValue, bool bFloat)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    auto GetValueType = [&](EPixelFormat InFormat)
    {
        if (bFloat)
		{
			return EClearReplacementValueType::Float;
		}

        // The Metal validation layer will complain about resources with a
        // signed format bound against an unsigned data format type as the
        // shader parameter.
        switch (InFormat)
        {
        case PF_R32_SINT:
        case PF_R16_SINT:
        case PF_R16G16B16A16_SINT:
            return EClearReplacementValueType::Int32;
        }

        return EClearReplacementValueType::Uint32;
    };

    if (IsBuffer())
    {
        FMetalRHIBuffer* Buffer = ResourceCast(GetBuffer());
        auto const Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);

        switch (Info.BufferType)
        {
#if UE_METAL_RHI_SUPPORT_CLEAR_UAV_WITH_BLIT_ENCODER
        case FRHIViewDesc::EBufferType::Raw:
            ClearUAVWithBlitEncoder(RHICmdList, *(const uint32*)ClearValue);
            break;

        case FRHIViewDesc::EBufferType::Structured:
            ClearUAVWithBlitEncoder(RHICmdList, *(const uint32*)ClearValue);
            break;
#endif // UE_METAL_RHI_SUPPORT_CLEAR_UAV_WITH_BLIT_ENCODER

        default:
            ClearUAVShader_T<EClearReplacementResourceType::Buffer, 4, false>(RHICmdList, this, Info.NumElements, 1, 1, ClearValue, GetValueType(Info.Format));
            break;
        }
    }
    else
    {
        FMetalSurface* Texture = ResourceCast(GetTexture());
        auto const Info = ViewDesc.Texture.UAV.GetViewInfo(Texture);

        FIntVector SizeXYZ = Texture->GetMipDimensions(Info.MipLevel);

        switch (Texture->GetDesc().Dimension)
        {
        case ETextureDimension::Texture2D:
            ClearUAVShader_T<EClearReplacementResourceType::Texture2D, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, SizeXYZ.Z, ClearValue, GetValueType(Info.Format));
            break;

        case ETextureDimension::Texture2DArray:
            ClearUAVShader_T<EClearReplacementResourceType::Texture2DArray, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, Info.ArrayRange.Num, ClearValue, GetValueType(Info.Format));
            break;

        case ETextureDimension::TextureCube:
        case ETextureDimension::TextureCubeArray:
            ClearUAVShader_T<EClearReplacementResourceType::Texture2DArray, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, Info.ArrayRange.Num * 6, ClearValue, GetValueType(Info.Format));
            break;

        case ETextureDimension::Texture3D:
            ClearUAVShader_T<EClearReplacementResourceType::Texture3D, 4, false>(RHICmdList, this, SizeXYZ.X, SizeXYZ.Y, SizeXYZ.Z, ClearValue, GetValueType(Info.Format));
            break;

        default:
            checkNoEntry();
            break;
        }
    }
}

void FMetalRHICommandContext::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 Offset, uint32 NumBytes)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
    check(DestinationStagingBufferRHI);

    FMetalRHIStagingBuffer* MetalStagingBuffer = ResourceCast(DestinationStagingBufferRHI);
    ensureMsgf(!MetalStagingBuffer->bIsLocked, TEXT("Attempting to Copy to a locked staging buffer. This may have undefined behavior"));
    FMetalRHIBuffer* SourceBuffer = ResourceCast(SourceBufferRHI);
    FMetalBufferPtr& ReadbackBuffer = MetalStagingBuffer->ShadowBuffer;

    // Need a shadow buffer for this read. If it hasn't been allocated in our FStagingBuffer or if
    // it's not big enough to hold our readback we need to allocate.
    if (!ReadbackBuffer || ReadbackBuffer->GetLength() < NumBytes)
    {
        if (ReadbackBuffer)
        {
			FMetalDynamicRHI::Get().DeferredDelete(ReadbackBuffer);
        }
        FMetalPooledBufferArgs ArgsCPU(&Device, NumBytes, BUF_Dynamic, MTL::StorageModeShared);
        ReadbackBuffer = Device.CreatePooledBuffer(ArgsCPU);
    }

    // Inline copy from the actual buffer to the shadow
    CopyFromBufferToBuffer(SourceBuffer->GetCurrentBuffer(), Offset, ReadbackBuffer, 0, NumBytes);
}

FMetalGPUFence::FMetalGPUFence(FMetalDevice& InDevice, FName InName)
	: FRHIGPUFence(InName),
	  Device(&InDevice)
{
	if(GMetalUseEventForGPUFence)
	{
		SharedEvent = Device->CreateSharedEvent();
	}
}

FMetalGPUFence::~FMetalGPUFence()
{
	if(GMetalUseEventForGPUFence)
	{
		FMetalDynamicRHI::Get().DeferredDelete([Event=SharedEvent]{});
	}
}

void FMetalGPUFence::Clear()
{
	SyncPoint = nullptr;

	if(GMetalUseEventForGPUFence)
	{
		FMetalDynamicRHI::Get().DeferredDelete([Event=SharedEvent]{});
		SharedEvent = Device->CreateSharedEvent();
	}
}

bool FMetalGPUFence::Poll() const
{
	bool bHasAnySyncPoint = false;
	
	if (SyncPoint && SyncPoint->IsComplete())
	{
		return true;
	}
	
	if (SharedEvent && SharedEvent->signaledValue() > 0)
	{
		return true;
	}
		
	return false;
}

void FMetalGPUFence::Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const
{
	if(!Poll())
	{
		bool bSuccess = false;
		if (SharedEvent)
		{
			static const uint64_t FENCE_EVENT_WAIT_TIMEOUT = 1000000;
			bSuccess = SharedEvent->waitUntilSignaledValue(1, FENCE_EVENT_WAIT_TIMEOUT);
		}
		if(!bSuccess && SyncPoint)
		{
			SyncPoint->Wait();
		}
	}	
}

void FMetalDynamicRHI::RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI)
{
	FMetalGPUFence* Fence = ResourceCast(FenceRHI);
	check(Fence);

	checkf(Fence->SyncPoint == nullptr, TEXT("The fence for the current GPU node has already been issued."));
	
	if(!Fence->SharedEvent)
	{
		Fence->SyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);
	}

	Fence->NumPendingWriteCommands.Increment();
	RHICmdList.EnqueueLambda([Fence, SyncPoint = Fence->SyncPoint](FRHICommandListBase& CmdList)
	{
		FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(CmdList);
		
		if(Fence->SharedEvent)
		{
			Context.SignalSharedEvent(Fence->SharedEvent, 1);
		}
		else
		{
			Context.SignalSyncPoint(SyncPoint);
		}
		
		Fence->NumPendingWriteCommands.Decrement();
	});
}

void FMetalRHICommandContext::RHIWriteGPUFence(FRHIGPUFence* FenceRHI)
{
	checkNoEntry(); // Should never be called
}

FGPUFenceRHIRef FMetalDynamicRHI::RHICreateGPUFence(const FName &Name)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	return new FMetalGPUFence(*Device, Name);
}
