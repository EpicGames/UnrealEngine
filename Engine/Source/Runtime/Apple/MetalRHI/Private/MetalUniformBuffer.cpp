// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalConstantBuffer.cpp: Metal Constant buffer implementation.
=============================================================================*/

#include "MetalUniformBuffer.h"
#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "MetalTempAllocator.h"
#include "MetalDevice.h"
#include "ShaderParameterStruct.h"
#include "RHIUniformBufferDataShared.h"
#include "Misc/ScopeLock.h"

#pragma mark Suballocated Uniform Buffer Implementation

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
inline bool DoesUniformBufferNeedBindlessHandle(FMetalDevice& InDevice, const FRHIUniformBufferLayout* Layout)
{
	if (EnumHasAnyFlags(Layout->Flags, ERHIUniformBufferFlags::BindlessAccessible))
	{
		return IsMetalBindlessEnabled();
	}

	return false;
}
#endif

FMetalSuballocatedUniformBuffer::FMetalSuballocatedUniformBuffer(FMetalDevice& InDevice, const void *Contents, const FRHIUniformBufferLayout* Layout,
																EUniformBufferUsage Usage, EUniformBufferValidation InValidation)
    : FRHIUniformBuffer(Layout)
    , LastFrameUpdated(0)
    , Shadow(FMemory::Malloc(GetSize()))
	, Device(InDevice)
#if METAL_UNIFORM_BUFFER_VALIDATION
    , Validation(InValidation)
#endif // METAL_UNIFORM_BUFFER_VALIDATION
{
	if (Contents)
	{
        UE::RHICore::UpdateUniformBufferConstants(Shadow, Contents, GetLayout());
		CopyResourceTable(Contents, ResourceTable);
		PushToGPUBacking(Shadow);
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (DoesUniformBufferNeedBindlessHandle(InDevice, Layout))
	{
		if (BackingBuffer)
		{
			BindlessHandle = Device.GetBindlessDescriptorManager()->AllocateDescriptor(ERHIDescriptorType::CBV, BackingBuffer.Get(), 0);
		}
		else
		{
			BindlessHandle = Device.GetBindlessDescriptorManager()->AllocateDescriptor(ERHIDescriptorType::CBV);
		}
	}
#endif
}

FMetalSuballocatedUniformBuffer::~FMetalSuballocatedUniformBuffer()
{
	if(BackingBuffer)
	{
		FMetalDynamicRHI::Get().DeferredDelete(BackingBuffer);
		BackingBuffer.Reset();
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BindlessHandle.IsValid())
	{
		FMetalDynamicRHI::Get().DeferredDelete(BindlessHandle);
	}
#endif
	
	FMemory::Free(Shadow);
}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
FRHIDescriptorHandle FMetalSuballocatedUniformBuffer::GetBindlessHandle() const
{
	return BindlessHandle;
}
#endif

void FMetalSuballocatedUniformBuffer::Update(FRHICommandListBase& RHICmdList, const void* Contents)
{
    UE::RHICore::UpdateUniformBufferConstants(Shadow, Contents, GetLayout());
	CopyResourceTable(Contents, ResourceTable);
	PushToGPUBacking(Shadow);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (BackingBuffer && BindlessHandle.IsValid())
	{
		Device.GetBindlessDescriptorManager()->UpdateDescriptor(RHICmdList, BindlessHandle, BackingBuffer.Get(), EMetalDescriptorUpdateType::GPU);
	}
#endif
}

// Acquires a region in the current frame's uniform buffer and
// pushes the data in Contents into that GPU backing store
// The amount of data read from Contents is given by the Layout
void FMetalSuballocatedUniformBuffer::PushToGPUBacking(const void* Contents)
{    
	FMetalTempAllocator* Allocator = Device.GetUniformAllocator();
	
	if(BackingBuffer)
	{
		FMetalDynamicRHI::Get().DeferredDelete(BackingBuffer);
		BackingBuffer.Reset();
	}
	
	FMetalPooledBufferArgs ArgsCPU(&Device, GetSize(), BUF_Static, MTL::StorageModeShared);
	BackingBuffer = Device.CreatePooledBuffer(ArgsCPU);
	
	if(!BackingBuffer)
	{
		UE_LOGF(LogMetal, Fatal, "BackingBuffer returned from MetalTempAllocator::Allocate is nullptr");
	}
	
	uint8* ConstantSpace = reinterpret_cast<uint8*>(BackingBuffer->Contents());
	FMemory::Memcpy(ConstantSpace, Contents, GetSize());
	LastFrameUpdated = Device.GetFrameNumberRHIThread();
}

void FMetalSuballocatedUniformBuffer::CopyResourceTable(const void* Contents, TArray<TRefCountPtr<FRHIResource> >& OutResourceTable) const
{
#if METAL_UNIFORM_BUFFER_VALIDATION
	if (Validation == EUniformBufferValidation::ValidateResources)
	{
		ValidateShaderParameterResourcesRHI(Contents, GetLayout());
	}
#endif // METAL_UNIFORM_BUFFER_VALIDATION

	const FRHIUniformBufferLayout& Layout = GetLayout();
    const uint32 NumResources = Layout.Resources.Num();
    if (NumResources > 0)
    {
		OutResourceTable.Empty(NumResources);
		OutResourceTable.AddZeroed(NumResources);
        
        for (uint32 Index = 0; Index < NumResources; ++Index)
		{
			OutResourceTable[Index] = GetShaderParameterResourceRHI(Contents, Layout.Resources[Index].MemberOffset, Layout.Resources[Index].MemberType);
		}
    }
}
