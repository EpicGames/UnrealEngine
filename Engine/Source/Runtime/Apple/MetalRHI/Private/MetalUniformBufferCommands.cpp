// Copyright Epic Games, Inc. All Rights Reserved.

#include "MetalRHIPrivate.h"
#include "MetalDynamicRHI.h"
#include "MetalShaderTypes.h"
#include "MetalTempAllocator.h"

FUniformBufferRHIRef FMetalDynamicRHI::RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation)
{
	return new FMetalUniformBuffer(*Device, Contents, Layout, Usage, Validation);
}

void FMetalDynamicRHI::RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents)
{
	FMetalUniformBuffer* UniformBuffer = ResourceCast(UniformBufferRHI);

	const void* SrcContents = Contents;

	if (RHICmdList.IsTopOfPipe())
	{
		const FRHIUniformBufferLayout& Layout = UniformBuffer->GetLayout();

		// Copy the contents memory region into the RHICmdList to allow deferred execution on the RHI thread.
		void* DstContents = RHICmdList.Alloc(Layout.ConstantBufferSize, alignof(FRHIResource*));
		FMemory::ParallelMemcpy(DstContents, Contents, Layout.ConstantBufferSize, EMemcpyCachePolicy::StoreUncached);

		SrcContents = DstContents;
	}

	RHICmdList.EnqueueLambda([UniformBuffer, SrcContents](FRHICommandListBase& RHICmdList)
	{
		UniformBuffer->Update(RHICmdList, SrcContents);
	});
	
	RHICmdList.RHIThreadFence(true);
}
