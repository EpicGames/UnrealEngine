// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalVertexDeclaration.cpp: Metal vertex declaration RHI implementation.
=============================================================================*/

#include "MetalVertexDeclaration.h"

#include "MetalRHIPrivate.h"
#include "MetalHashedVertexDescriptor.h"
#include "MetalProfiler.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Vertex Declaration Globals

//------------------------------------------------------------------------------

#pragma mark - Metal Vertex Declaration Support Routines

#if METAL_USE_METAL_SHADER_CONVERTER
static IRFormat TranslateElementTypeToIRType(EVertexElementType Type)
{
    switch (Type)
    {
        case VET_Float1:  		return IRFormatR32Float;
        case VET_Float2:  		return IRFormatR32G32Float;
        case VET_Float3:  		return IRFormatR32G32B32Float;
        case VET_Float4:  		return IRFormatR32G32B32A32Float;
        case VET_PackedNormal:  return IRFormatR8G8B8A8Snorm;
        case VET_UByte4:  		return IRFormatR8G8B8A8Uint;
        case VET_UByte4N:  		return IRFormatR8G8B8A8Unorm;
        case VET_Color:  		return IRFormatB8G8R8A8Unorm;
        case VET_Short2:  		return IRFormatR16G16Sint;
        case VET_Short4:  		return IRFormatR16G16B16A16Sint;
        case VET_Short2N:  		return IRFormatR16G16Snorm;
        case VET_Half2:  		return IRFormatR16G16Float;
        case VET_Half4:  		return IRFormatR16G16B16A16Float;
        case VET_Short4N:  		return IRFormatR16G16B16A16Snorm;
        case VET_UShort2:  		return IRFormatR16G16Uint;
        case VET_UShort4:  		return IRFormatR16G16B16A16Uint;
        case VET_UShort2N:  	return IRFormatR16G16Unorm;
        case VET_UShort4N:  	return IRFormatR16G16B16A16Unorm;
        case VET_URGB10A2N:  	return IRFormatR10G10B10A2Unorm;
        case VET_UInt:  		return IRFormatR32Uint;
        default: METAL_FATAL_ERROR(TEXT("Unknown vertex element type %d!"), (uint32)Type); return IRFormatR32Float;
    };
}
#endif
 
static MTL::VertexFormat TranslateElementTypeToMTLType(EVertexElementType Type)
{
	switch (Type)
	{
		case VET_Float1:		return MTL::VertexFormatFloat;
		case VET_Float2:		return MTL::VertexFormatFloat2;
		case VET_Float3:		return MTL::VertexFormatFloat3;
		case VET_Float4:		return MTL::VertexFormatFloat4;
		case VET_PackedNormal:	return MTL::VertexFormatChar4Normalized;
		case VET_UByte4:		return MTL::VertexFormatUChar4;
		case VET_UByte4N:		return MTL::VertexFormatUChar4Normalized;
		case VET_Color:			return MTL::VertexFormatUChar4Normalized_BGRA;
		case VET_Short2:		return MTL::VertexFormatShort2;
		case VET_Short4:		return MTL::VertexFormatShort4;
		case VET_Short2N:		return MTL::VertexFormatShort2Normalized;
		case VET_Half2:			return MTL::VertexFormatHalf2;
		case VET_Half4:			return MTL::VertexFormatHalf4;
		case VET_Short4N:		return MTL::VertexFormatShort4Normalized;
		case VET_UShort2:		return MTL::VertexFormatUShort2;
		case VET_UShort4:		return MTL::VertexFormatUShort4;
		case VET_UShort2N:		return MTL::VertexFormatUShort2Normalized;
		case VET_UShort4N:		return MTL::VertexFormatUShort4Normalized;
		case VET_URGB10A2N:		return MTL::VertexFormatUInt1010102Normalized;
		case VET_UInt:			return MTL::VertexFormatUInt;
        default:				METAL_FATAL_ERROR(TEXT("Unknown vertex element type %d!"), (uint32)Type); return MTL::VertexFormatFloat;
    };
}

static uint32 TranslateElementTypeToSize(EVertexElementType Type)
{
	switch (Type)
	{
		case VET_Float1:		return 4;
		case VET_Float2:		return 8;
		case VET_Float3:		return 12;
		case VET_Float4:		return 16;
		case VET_PackedNormal:	return 4;
		case VET_UByte4:		return 4;
		case VET_UByte4N:		return 4;
		case VET_Color:			return 4;
		case VET_Short2:		return 4;
		case VET_Short4:		return 8;
		case VET_UShort2:		return 4;
		case VET_UShort4:		return 8;
		case VET_Short2N:		return 4;
		case VET_UShort2N:		return 4;
		case VET_Half2:			return 4;
		case VET_Half4:			return 8;
		case VET_Short4N:		return 8;
		case VET_UShort4N:		return 8;
		case VET_URGB10A2N:		return 4;
		case VET_UInt:			return 4;
		default:				METAL_FATAL_ERROR(TEXT("Unknown vertex element type %d!"), (uint32)Type); return 0;
	};
}


//------------------------------------------------------------------------------

#pragma mark - Metal Vertex Declaration Class Implementation


FMetalVertexDeclaration::FMetalVertexDeclaration(const FVertexDeclarationElementList& InElements)
	: Elements(InElements)
	, BaseHash(0)
{
	GenerateLayout(InElements);
}

FMetalVertexDeclaration::~FMetalVertexDeclaration()
{
	// void
}

bool FMetalVertexDeclaration::GetInitializer(FVertexDeclarationElementList& Init)
{
	Init = Elements;
	return true;
}

void FMetalVertexDeclaration::GenerateLayout(const FVertexDeclarationElementList& InElements)
{
#if METAL_USE_METAL_SHADER_CONVERTER
	typedef TArray<IRInputElementDescriptor1, TFixedAllocator<MaxVertexElementCount> > FMetalVertexElements;
	
	if(IsMetalBindlessEnabled())
	{
		BaseHash = 0;
		uint32 StrideHash = BaseHash;
		
		FMetalVertexElements VertexElements;
		for (uint32 ElementIndex = 0; ElementIndex < InElements.Num(); ElementIndex++)
		{
			const FVertexElement& Element = InElements[ElementIndex];

			IRInputElementDescriptor1 Descriptor = {0};
			Descriptor.inputSlot = Element.StreamIndex;
			Descriptor.alignedByteOffset = Element.Offset;
			Descriptor.format = TranslateElementTypeToIRType(Element.Type);
			Descriptor.semanticIndex = Element.AttributeIndex;
			Descriptor.inputSlotClass = Element.bUseInstanceIndex ? IRInputClassificationPerInstanceData : IRInputClassificationPerVertexData;
			Descriptor.instanceDataStepRate = Element.bUseInstanceIndex ? 1 : 0;
			
			VertexElements.Add(Descriptor);
			BaseHash = FCrc::MemCrc32(&Element.StreamIndex, sizeof(Element.StreamIndex), BaseHash);
			BaseHash = FCrc::MemCrc32(&Element.Offset, sizeof(Element.Offset), BaseHash);
			BaseHash = FCrc::MemCrc32(&Element.Type, sizeof(Element.Type), BaseHash);
			BaseHash = FCrc::MemCrc32(&Element.AttributeIndex, sizeof(Element.AttributeIndex), BaseHash);
		
			uint32 Stride = Element.Stride;
			StrideHash = FCrc::MemCrc32(&Stride, sizeof(Stride), StrideHash);
			checkSlow(InputDescriptorBufferStrides[Element.StreamIndex] != 0);
			InputDescriptorBufferStrides[Element.StreamIndex] = Element.Stride;
		}
			
		// Sort by stream then offset.
		struct FCompareDesc
		{
			FORCEINLINE bool operator()(const IRInputElementDescriptor1& A, const IRInputElementDescriptor1 &B) const
			{
				if (A.inputSlot != B.inputSlot)
				{
					return A.inputSlot < B.inputSlot;
				}
				if (A.alignedByteOffset != B.alignedByteOffset)
				{
					return A.alignedByteOffset < B.alignedByteOffset;
				}
				return A.semanticIndex < B.semanticIndex;
			}
		};
		
		Algo::Sort(VertexElements, FCompareDesc());
		InputDescriptor.version = IRInputLayoutDescriptorVersion_1;
		InputDescriptor.desc_1_0.numElements = VertexElements.Num();
		
		// Assign all the SemanticName after hashing. It's a constant string, always the same, so no need to hash the data.
		for (int32 ElementIndex = 0; ElementIndex < VertexElements.Num(); ElementIndex++)
		{
			InputDescriptor.desc_1_0.inputElementDescs[ElementIndex] = VertexElements[ElementIndex];
			InputDescriptor.desc_1_0.semanticNames[ElementIndex] = "ATTRIBUTE";
		}
		
		Layout = FMetalHashedVertexDescriptor(InputDescriptor, HashCombine(BaseHash, StrideHash));
	}
	else
#endif
	{
		MTLVertexDescriptorPtr NewLayout = NS::RetainPtr(MTL::VertexDescriptor::vertexDescriptor());
		
		MTL::VertexBufferLayoutDescriptorArray* Layouts = NewLayout->layouts();
		MTL::VertexAttributeDescriptorArray* Attributes = NewLayout->attributes();
		
		BaseHash = 0;
		uint32 StrideHash = BaseHash;
		
		TMap<uint32, uint32> BufferStrides;
		for (uint32 ElementIndex = 0; ElementIndex < InElements.Num(); ElementIndex++)
		{
			const FVertexElement& Element = InElements[ElementIndex];
			
			//checkf(Element.Stride == 0 || Element.Offset + TranslateElementTypeToSize(Element.Type) <= Element.Stride,
			//	   TEXT("Stream component is bigger than stride: Offset: %d, Size: %d [Type %d], Stride: %d"), Element.Offset, TranslateElementTypeToSize(Element.Type), (uint32)Element.Type, Element.Stride);
			
			BaseHash = FCrc::MemCrc32(&Element.StreamIndex, sizeof(Element.StreamIndex), BaseHash);
			BaseHash = FCrc::MemCrc32(&Element.Offset, sizeof(Element.Offset), BaseHash);
			BaseHash = FCrc::MemCrc32(&Element.Type, sizeof(Element.Type), BaseHash);
			BaseHash = FCrc::MemCrc32(&Element.AttributeIndex, sizeof(Element.AttributeIndex), BaseHash);
			
			uint32 Stride = Element.Stride;
			StrideHash = FCrc::MemCrc32(&Stride, sizeof(Stride), StrideHash);
			
			// Vertex & Constant buffers are set up in the same space, so add VB's from the top
			uint32 ShaderBufferIndex = UNREAL_TO_METAL_BUFFER_INDEX(Element.StreamIndex);
			
			// track the buffer stride, making sure all elements with the same buffer have the same stride
			uint32* ExistingStride = BufferStrides.Find(ShaderBufferIndex);
			if (ExistingStride == NULL)
			{
				// handle 0 stride buffers
				MTL::VertexStepFunction Function = (Element.Stride == 0 ? MTL::VertexStepFunctionConstant : (Element.bUseInstanceIndex ? MTL::VertexStepFunctionPerInstance : MTL::VertexStepFunctionPerVertex));
				uint32 StepRate = (Element.Stride == 0 ? 0 : 1);
				
				// even with MTLVertexStepFunctionConstant, it needs a non-zero stride (not sure why)
				if (Element.Stride == 0)
				{
					Stride = TranslateElementTypeToSize(Element.Type);
				}
				
				// look for any unset strides coming from the Engine (this can be removed when all are fixed)
				if (Element.Stride == 0xFFFF)
				{
					UE_LOGF(LogMetal, Display, "Setting illegal stride - break here if you want to find out why, but this won't break until we try to render with it");
					Stride = 200;
				}
				
				// set the stride once per buffer
				MTL::VertexBufferLayoutDescriptor* VBLayout = Layouts->object(ShaderBufferIndex);
				VBLayout->setStride(Stride);
				VBLayout->setStepFunction(Function);
				VBLayout->setStepRate(StepRate);
				
				// track this buffer and stride
				BufferStrides.Add(ShaderBufferIndex, Element.Stride);
			}
			else
			{
				// if the strides of elements with same buffer index have different strides, something is VERY wrong
				check(Element.Stride == *ExistingStride);
			}
			
			// set the format for each element
			MTL::VertexAttributeDescriptor* Attrib = Attributes->object(Element.AttributeIndex);
			Attrib->setFormat(TranslateElementTypeToMTLType(Element.Type));
			Attrib->setOffset(Element.Offset);
			Attrib->setBufferIndex(ShaderBufferIndex);
		}
		
		Layout = FMetalHashedVertexDescriptor(NewLayout, HashCombine(BaseHash, StrideHash));
	}
}
