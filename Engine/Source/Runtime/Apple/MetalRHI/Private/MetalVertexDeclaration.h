// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalVertexDeclaration.h: Metal RHI Vertex Declaration.
=============================================================================*/

#pragma once

#include "MetalHashedVertexDescriptor.h"
#include "MetalRHIPrivate.h"
#include "RHI.h"
#include "RHIResources.h"

//------------------------------------------------------------------------------

#pragma mark - Metal Vertex Declaration Class


/**
 * This represents a vertex declaration that hasn't been combined with a
 * specific shader to create a bound shader.
 */
class FMetalVertexDeclaration : public FRHIVertexDeclaration
{
public:
	/** Initialization constructor. */
	FMetalVertexDeclaration(const FVertexDeclarationElementList& InElements);
	~FMetalVertexDeclaration();

#if METAL_USE_METAL_SHADER_CONVERTER
    IRVersionedInputLayoutDescriptor InputDescriptor;
	TStaticArray<uint32, MaxVertexElementCount> InputDescriptorBufferStrides;
#endif
	/** Cached element info array (offset, stream index, etc) */
	FVertexDeclarationElementList Elements;

	/** This is the layout for the vertex elements */
	FMetalHashedVertexDescriptor Layout;

	/** Hash without considering strides which may be overriden */
	uint32 BaseHash;

	virtual bool GetInitializer(FVertexDeclarationElementList& Init) override final;
	virtual uint32 GetPrecachePSOHash() const override final { return BaseHash; }

protected:
	void GenerateLayout(const FVertexDeclarationElementList& Elements);
};
