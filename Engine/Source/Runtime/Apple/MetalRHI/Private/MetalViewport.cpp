// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalViewport.cpp: Metal viewport RHI implementation.
=============================================================================*/

#include "MetalViewport.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "RHIUtilities.h"
#include "MetalDynamicRHI.h"
#include "MetalRHIPrivate.h"
#include "MetalCommandBuffer.h"
#include "MetalProfiler.h"
#include "MetalRHIVisionOSBridge.h"
#include "MetalDevice.h"

#import <QuartzCore/CAMetalLayer.h>

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#include "Mac/CocoaWindow.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacApplication.h"
#else
#include "IOS/IOSSystemIncludes.h"
#include "IOS/IOSApplication.h"
#include "IOS/IOSAppDelegate.h"
#endif
#include "RenderCommandFence.h"
#include "Containers/Set.h"
#include "RenderUtils.h"
#include "Engine/RendererSettings.h"

#include "Input/HittestGrid.h"
#include "Framework/Application/SlateApplication.h"

extern int32 GMetalSupportsIntermediateBackBuffer;
extern float GMetalPresentFramePacing;

#if PLATFORM_IOS
static int32 GEnablePresentPacing = 0;
static FAutoConsoleVariableRef CVarMetalEnablePresentPacing(
	   TEXT("ios.PresentPacing"),
	   GEnablePresentPacing,
	   TEXT(""),
		ECVF_Default);
#endif


int32 GMetalNonBlockingPresent = 0;
static FAutoConsoleVariableRef CVarMetalNonBlockingPresent(
	TEXT("rhi.Metal.NonBlockingPresent"),
	GMetalNonBlockingPresent,
	TEXT("When enabled (> 0) this will force MetalRHI to query if a back-buffer is available to present and if not will skip the frame. Only functions on macOS, it is ignored on iOS/tvOS.\n")
	TEXT("(Off by default (0))"));

#if PLATFORM_MAC

// Quick way to disable availability warnings is to duplicate the definitions into a new type - gotta love ObjC dynamic-dispatch!
@interface FCAMetalLayer : CALayer
@property BOOL displaySyncEnabled;
@property BOOL allowsNextDrawableTimeout;
@end

@implementation FMetalView

@synthesize LayerRange;

- (id)initWithFrame:(NSRect)frameRect LayerRange:(TInterval<int32>)layerRange MetalViewport:(FMetalViewport*)metalViewport
{
	self = [super initWithFrame:frameRect];
	if (self)
	{
		LayerRange = layerRange;
		MetalViewport = metalViewport;
		bSlateWidgetHit = false;
	}
	return self;
}

- (void)setAllowMouseActions: (BOOL)allowMouseActions
{
	bAllowMouseActionsPastLayerRange = allowMouseActions;
}

- (NSView *) hitTest:(NSPoint) point
{
	// we unconditionally allow the bottommost layer to go through:
	// there's no way that a WKWebView/native view could incorrectly
	// preempt its input, and we need to have it go through due to the
	// parent child relationship with the lowest layer being the parent
	// of all the subsequent higher layers/native views
	
	if(LayerRange.Min != std::numeric_limits<int32>::min())
	{
		// want to convert to screen coords as that's what 
		// FMacApplication::ConvertCocoaPositionToSlate bases its calculations on
		NSPoint ConvertedPoint = [[self superview] convertPoint: point toView: nil];
		ConvertedPoint = [[self window] convertPointToScreen: ConvertedPoint];
		
		GameThreadCall(^
	    {
			TSharedPtr<SWindow> Window = MetalViewport->GetBackingSlateWindow();
			
			if(Window.IsValid())
			{
				FHittestGrid& HittestGrid = Window->GetHittestGrid();
				FVector2D SlatePoint = FMacApplication::ConvertCocoaPositionToSlate(ConvertedPoint.x, ConvertedPoint.y);
				
				// we want to test [LayerRange.Min + 1, LayerRange.Max) so as to NOT include the widgets that represent the 
				// native layer (e.g. SAppleWebBrowser)
				TArray<FWidgetAndPointer> BubblePath = HittestGrid.GetBubblePath(SlatePoint, 1.0f, true, INDEX_NONE, 
																				 TInterval<int32>(LayerRange.Min + 1, LayerRange.Max));
				
				bSlateWidgetHit = BubblePath.Num() > 1;
			}
		}, false);
		
		return bSlateWidgetHit ? [super hitTest: point] : nil;
	}
	else
	{
		return [super hitTest: point];
	}
}

- (BOOL)isOpaque
{
	return !bAllowMouseActionsPastLayerRange;
}

- (BOOL)mouseDownCanMoveWindow
{
	return YES;
}

@end

#endif

FCriticalSection FMetalViewport::ViewportsMutex;
TSet<FMetalViewport*> FMetalViewport::Viewports;

FMetalViewport::FMetalViewport(FMetalDevice& InDevice, void* WindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InFormat)
	: Device(InDevice)
	, Mutex{}
	, DisplayID{0}
	, Block{nullptr}
	, FrameAvailable{0}
	, bIsFullScreen{bInIsFullscreen}
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	, SizeX(InSizeX)
	, SizeY(InSizeY)
	, Format(InFormat)
#endif
	, DefaultNativeLayer{std::numeric_limits<int32>::min()}
#if PLATFORM_MAC
	, WindowHandle{(FCocoaWindow*)WindowHandle}
#endif
#if !WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	, Drawable{nullptr}
	, BackBuffer{nullptr, nullptr}
#if PLATFORM_MAC
	, View{nullptr}
#endif
#endif
	, DrawableTextures{}
#if PLATFORM_MAC || PLATFORM_VISIONOS
	, CustomPresent{nullptr}
#endif
{
#if PLATFORM_VISIONOS
	// look to see if we need to hook up to a Swift compositor renderer
	SwiftLayer = [IOSAppDelegate GetDelegate].SwiftLayer;
#endif

#if !WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	// if we are using the multi view slate window rendering path, the equivalent code
	// gets called from FSlateRHIRenderer::CreateViewport() upon viewport
	// creation, except initializing the base draw layer of std::numeric_limits<int32>::min() 
	// instead of a view variable
#if PLATFORM_MAC
	MainThreadCall(^{
		View = CreateNewMetalView(true, NSWindowAbove, nil, TInterval<int32>(std::numeric_limits<int32>::min(), std::numeric_limits<int32>::max()));
	});
#endif
	
	Resize(InSizeX, InSizeY, bInIsFullscreen, InFormat);
#endif

#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	// An orientation change from LandscapeLeft to LandscapeRight won't trigger a SetRes, so we need to react to an orientation change here
	OrientationChangedHandle = FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.AddLambda([this](int32 ScreenOrientation)
	{
		if ([IOSAppDelegate GetMaskFromEDeviceScreenOrientation:((EDeviceScreenOrientation)ScreenOrientation)] != OrientationMask)
		{
			Resize(SizeX, SizeY, bIsFullScreen, Format);
		}
	});
#endif
	
	{
		FScopeLock Lock(&ViewportsMutex);
		Viewports.Add(this);
	}
}

FMetalViewport::~FMetalViewport()
{
	FScopeLock Lock(&ViewportsMutex);
	Viewports.Remove(this);
	
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.Remove(OrientationChangedHandle);
#endif

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	for(TPair<int32, DrawLayer>& DrawLayer : DrawLayers)
	{
		DrawLayer.Value.BackBuffer[0].SafeRelease();	// when the rest of the engine releases it, its framebuffers will be released too (those the engine knows about)
		DrawLayer.Value.BackBuffer[1].SafeRelease();
		check(!IsValidRef(DrawLayer.Value.BackBuffer[0]));
		check(!IsValidRef(DrawLayer.Value.BackBuffer[1]));
	}
#else
	BackBuffer[0].SafeRelease();	// when the rest of the engine releases it, its framebuffers will be released too (those the engine knows about)
	BackBuffer[1].SafeRelease();
	check(!IsValidRef(BackBuffer[0]));
	check(!IsValidRef(BackBuffer[1]));
#endif
}

#if PLATFORM_MAC
FMetalView* FMetalViewport::CreateNewMetalView(bool bIsContentView, NSWindowOrderingMode ViewOrder, NSView* ViewRelativeTo,
											   TInterval<int32> LayerRange)
{	
	const NSRect ContentRect = NSMakeRect(0, 0, WindowHandle.openGLFrame.size.width, WindowHandle.openGLFrame.size.height);
	
	FMetalView* MetalView = [[FMetalView alloc] initWithFrame:ContentRect LayerRange:LayerRange MetalViewport:this];
	[MetalView setAutoresizingMask:NSViewWidthSizable | NSViewHeightSizable];
	[MetalView setWantsLayer:YES];
	[MetalView setAllowMouseActions: bIsContentView];

	CAMetalLayer* Layer = [CAMetalLayer new];

	CGFloat bgColor[] = { 0.0, 0.0, 0.0, 0.0 };
	Layer.edgeAntialiasingMask = 0;
	Layer.masksToBounds = YES;
	Layer.backgroundColor = CGColorCreate(CGColorSpaceCreateDeviceRGB(), bgColor);
	Layer.presentsWithTransaction = NO;
	Layer.anchorPoint = CGPointMake(0.5, 0.5);
	Layer.frame = ContentRect;
	Layer.magnificationFilter = kCAFilterNearest;
	Layer.minificationFilter = kCAFilterNearest;
	Layer.opaque = bIsContentView;

	[Layer setDevice:(__bridge id<MTLDevice>)Device.GetDevice()];
	
	[Layer setFramebufferOnly:NO];
	[Layer removeAllAnimations];
	
	[MetalView setLayer:Layer];
	
	if(bIsContentView)
	{		
		[WindowHandle setContentView: MetalView];
		[[WindowHandle standardWindowButton:NSWindowCloseButton] setAction:@selector(performClose:)];
	}
	else
	{
		// we are adding a new metal view to expose slate drawing layers to a native widget
		// (e.g. ApplePlatformWebBrowser) so that slate drawing layers above the native widget
		// draw over the native widget
		// For now, relativeTo is nil but we will need it to not be nil with multiple native layers
		NSView* ContentView = [WindowHandle contentView];
		
		[ContentView addSubview: MetalView positioned: ViewOrder relativeTo: ViewRelativeTo];
		
		MetalView.frame = ContentView.frame;
	}
	
	return MetalView;
}
#endif

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
void FMetalViewport::DeleteNativeLayer(int32 OldNativeLayer)
{
	UE_LOGF(LogMetal, Display, "Deleting NativeLayer %d from Metal Viewport %p", OldNativeLayer, this);
	
	ENQUEUE_RENDER_COMMAND(FlushPendingRHICommands)(
		[Viewport = this, OldNativeLayer](FRHICommandListImmediate& RHICmdList)
		{
			GRHICommandList.GetImmediateCommandList().SubmitAndBlockUntilGPUIdle();
			dispatch_sync(dispatch_get_main_queue(), ^
		    {
				FScopeLock Lock(&Viewport->Mutex);
				
				[Viewport->DrawLayers[OldNativeLayer].View removeFromSuperview];
				
				Viewport->DrawLayers[OldNativeLayer].BackBuffer[0].SafeRelease();
				Viewport->DrawLayers[OldNativeLayer].BackBuffer[1].SafeRelease();
				
				Viewport->DrawLayers.Remove(OldNativeLayer);
			});
		});
}

void FMetalViewport::CreateNativeLayer(int32 NewNativeLayer, void* NativeViewHandle, const TArray<TInterval<int32>>& LayerRanges)
{
	if(DrawLayers.Contains(NewNativeLayer))
	{
		// the native layer already exists!
		return;
	}
	
	if (WindowHandle)
	{
		float ContentScaleFactor = FSlateApplication::Get().GetApplicationScale() * MacApplication->FindWindowByNSWindow(WindowHandle)->GetDPIScaleFactor();
		SizeX = ContentScaleFactor * WindowHandle.openGLFrame.size.width;
		SizeY = ContentScaleFactor * WindowHandle.openGLFrame.size.height;
	}	
	
	bool bWaitForRenderingThread = true;
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	bWaitForRenderingThread = false;
	
	MainThreadCall(^{
		UE_LOGF(LogMetal, Display, "Creating NativeLayer %d in Metal Viewport %p", NewNativeLayer, this);
		
		TOptional<int32> LayerRightBeforeNewLayer;
		TInterval<int32> LayerRange(NewNativeLayer, std::numeric_limits<int32>::max());
		FScopeLock Lock(&Mutex);
		
		if(NewNativeLayer != std::numeric_limits<int32>::min())
		{
			for(const TPair<int32, DrawLayer>& DrawLayer : DrawLayers)
			{
				// this is reverse sorted, so once we find a DrawLayer lower
				// than the NewNativeLayer, we know that it should be directly
				// above that layer
				if(DrawLayer.Key < NewNativeLayer)
				{
					LayerRightBeforeNewLayer = DrawLayer.Key;
					break;
				}
				LayerRange.Max = DrawLayer.Key;
			}
		}
		
		FMetalView* View;
		
		if(NativeViewHandle != nil)
		{
			// we can't have a view we want to insert relative to if we are the
			// base layer, because the base layer is the only native layer
			// which exists when created.
			check(NewNativeLayer != std::numeric_limits<int32>::min());
			
			NSView* ContentView = [WindowHandle contentView];
			NSView* NativeView = (NSView*)NativeViewHandle;
			
			if(LayerRightBeforeNewLayer.IsSet()) 
			{
				UE_LOGF(LogMetal, Display, "Adding NativeLayer %d right below %d", NewNativeLayer, LayerRightBeforeNewLayer.GetValue());
			}
			
			bool bPlaceBelowNativeView = 
				LayerRightBeforeNewLayer.IsSet() && LayerRightBeforeNewLayer.GetValue() == std::numeric_limits<int32>::min();
			bool bPlaceRelToSpecificView = LayerRightBeforeNewLayer.IsSet() && LayerRightBeforeNewLayer.GetValue() != std::numeric_limits<int32>::min();
			
			[ContentView addSubview: NativeView positioned: bPlaceBelowNativeView ? NSWindowBelow : NSWindowAbove 
						 relativeTo: bPlaceRelToSpecificView ? DrawLayers[LayerRightBeforeNewLayer.GetValue()].View : nil];
			View = CreateNewMetalView(false, NSWindowAbove, NativeView, LayerRange);
		}
		else
		{
			View = CreateNewMetalView(NewNativeLayer == std::numeric_limits<int32>::min(), NSWindowAbove, LayerRightBeforeNewLayer.IsSet() ? 					  DrawLayers[LayerRightBeforeNewLayer.GetValue()].View : nil, LayerRange);
		}
		
		DrawLayers.Add(NewNativeLayer, DrawLayer
	    {
			.Drawable = nullptr,
			.BackBuffer = {nullptr, nullptr},
			.View = View
		});
		
		DrawLayers.KeySort([](int32 A, int32 B)
		{
			return A > B;
		});
		
		for(size_t i = 0; LayerRanges.Num() > i; ++i)
		{
			if(DrawLayers.Contains(LayerRanges[i].Min))
			{
				DrawLayers[LayerRanges[i].Min].View.LayerRange = LayerRanges[i];
			}
		}
	});
#endif
	
	Resize(SizeX, SizeY, bIsFullScreen, Format, bWaitForRenderingThread);
}
#endif

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
void FMetalViewport::SetDefaultNativeLayer(int32 NativeLayer)
{
	DefaultNativeLayer = NativeLayer;
}
#endif

uint32 FMetalViewport::GetViewportIndex(EMetalViewportAccessFlag Accessor) const
{
	switch(Accessor)
	{
		case EMetalViewportAccessRHI:
			check(IsInParallelRenderingThread());
			// Deliberate fall-through
		case EMetalViewportAccessDisplayLink: // Displaylink is not an index, merely an alias that avoids the check...
			return (GRHISupportsRHIThread && IsRunningRHIInSeparateThread()) ? EMetalViewportAccessRHI : EMetalViewportAccessRenderer;
		case EMetalViewportAccessRenderer:
			check(IsInRenderingThread());
			return Accessor;
		case EMetalViewportAccessGame:
			check(IsInGameThread());
			return EMetalViewportAccessRenderer;
		default:
			check(false);
			return EMetalViewportAccessRenderer;
	}
}

#if PLATFORM_MAC
TSharedPtr<SWindow> FMetalViewport::GetBackingSlateWindow()
{
	TSharedPtr<SWindow> BackingSlateWindowStrong = BackingSlateWindow.Pin();
	
	if (!BackingSlateWindowStrong.IsValid() && MacApplication)
	{
		TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow(WindowHandle);
		
		if (Window.IsValid())
		{
			for(TSharedRef<SWindow>& SlateWindow : FSlateApplication::Get().GetInteractiveTopLevelWindows())
			{
				if(Window == SlateWindow->GetNativeWindow())
				{
					BackingSlateWindow = SlateWindow;
					return SlateWindow;
				}
			}
		}
	}
	
	return BackingSlateWindowStrong;
}
#endif

void FMetalViewport::ResizeBackBuffer(int32 DrawLayerKey, uint32 Index, uint32 InSizeX, uint32 InSizeY,
									  EPixelFormat InFormat)
{
	TRefCountPtr<FMetalSurface> NewBackBuffer;

	FString BackBufferName = FString::Printf(TEXT("BackBuffer-%d"), DrawLayerKey);
	
	FRHITextureCreateDesc CreateDesc =
		FRHITextureCreateDesc::Create2D(*BackBufferName, InSizeX, InSizeY, InFormat)
		.SetClearValue(FClearValueBinding::Transparent)
		.SetFlags(ETextureCreateFlags::RenderTargetable);
	
	if (!GMetalSupportsIntermediateBackBuffer)
	{
		CreateDesc.AddFlags(ETextureCreateFlags::Presentable);
	}
	
	CreateDesc.SetInitialState(RHIGetDefaultResourceState(CreateDesc.Flags, false));

	NewBackBuffer = new FMetalSurface(Device, FMetalTextureCreateDesc(Device, CreateDesc));
	NewBackBuffer->Viewport = this;

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	DrawLayers[DrawLayerKey].BackBuffer[Index] = NewBackBuffer;
	DrawLayers[DrawLayerKey].BackBuffer[EMetalViewportAccessRHI] = DrawLayers[DrawLayerKey].BackBuffer[Index];
#else
	BackBuffer[Index] = NewBackBuffer;
	BackBuffer[EMetalViewportAccessRHI] = BackBuffer[Index];
#endif
}

#if PLATFORM_MAC
void ResizeMetalLayer(CAMetalLayer* MetalLayer, MTL::PixelFormat MetalFormat, bool bUseHDR,
					  uint32 InSizeX, uint32 InSizeY)
{	
	MetalLayer.drawableSize = CGSizeMake(InSizeX, InSizeY);
	
	if (MetalFormat != (MTL::PixelFormat)MetalLayer.pixelFormat)
	{
		MetalLayer.pixelFormat = (MTLPixelFormat)MetalFormat;
	}
	
	if (bUseHDR != MetalLayer.wantsExtendedDynamicRangeContent)
	{
		MetalLayer.wantsExtendedDynamicRangeContent = bUseHDR;
	}
}
#endif

void FMetalViewport::Resize(uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen, EPixelFormat InFormat, bool bWaitForRenderingThread)
{
	bIsFullScreen = bInIsFullscreen;
	uint32 Index = GetViewportIndex(EMetalViewportAccessGame);
	
	bool bUseHDR = GRHISupportsHDROutput && InFormat == GRHIHDRDisplayOutputFormat;
	
    MTL::PixelFormat MetalFormat = (MTL::PixelFormat)GPixelFormats[InFormat].PlatformFormat;
	
    ENQUEUE_RENDER_COMMAND(FlushPendingRHICommands)(
        [Viewport = this](FRHICommandListImmediate& RHICmdList)
        {
            GRHICommandList.GetImmediateCommandList().SubmitAndBlockUntilGPUIdle();
        });
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	bool bNeedToReleaseBackBuffer = true;
#else
	bool bNeedToReleaseBackBuffer = IsValidRef(BackBuffer[Index]) && InFormat != BackBuffer[Index]->GetFormat();
#endif
    
	if (bNeedToReleaseBackBuffer)
	{
		// Really need to flush the RHI thread & GPU here...
		AddRef();
		ENQUEUE_RENDER_COMMAND(FlushPendingRHICommands)(
			[Viewport = this](FRHICommandListImmediate& RHICmdList)
			{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
				for(auto& DrawLayer : Viewport->DrawLayers)
				{
					Viewport->ReleaseDrawable(DrawLayer.Key);
				}
#else
				Viewport->ReleaseDrawable(std::numeric_limits<int32>::min());
#endif

				Viewport->Release();			
			});
	}
    
	if(bWaitForRenderingThread)
	{
		// Issue a fence command to the rendering thread and wait for it to complete.
		FRenderCommandFence Fence;
		Fence.BeginFence();
		Fence.Wait();
	}
    
#if PLATFORM_MAC
	float WindowScaleDPIFactor = 1.0f; 
	if (WindowHandle)
	{
		WindowScaleDPIFactor = MacApplication->FindWindowByNSWindow(WindowHandle)->GetDPIScaleFactor();
	}
	
	float ContentScaleFactor = FSlateApplication::Get().GetApplicationScale() * WindowScaleDPIFactor;
	
	SizeX = ContentScaleFactor * InSizeX;
	SizeY = ContentScaleFactor * InSizeY;
	
	MainThreadCall(^
	{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
		for(TPair<int32, DrawLayer>& DrawLayer : DrawLayers)
		{
			ResizeMetalLayer((CAMetalLayer*)DrawLayer.Value.View.layer, MetalFormat, 
							 bUseHDR, InSizeX, InSizeY);
		}
#else
		ResizeMetalLayer((CAMetalLayer*)[View layer], MetalFormat, bUseHDR, InSizeX, InSizeY);
#endif		
	});
#else
	// A note on HDR in iOS
	// Setting the pixel format to one of the Apple XR formats is all you need.
	// iOS expects the app to output in sRGB regadless of the display
	// (even though Apple's HDR displays are P3)
	// and its compositor will do the conversion.
	{
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
		__block UIInterfaceOrientationMask CachedOrientationMask;
#endif

		dispatch_sync(dispatch_get_main_queue(), ^{
			IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
			FIOSView* IOSView = AppDelegate.IOSView;
			
			CAMetalLayer* MetalLayer = (CAMetalLayer*) IOSView.layer;
			
			if (MetalFormat != (MTL::PixelFormat) MetalLayer.pixelFormat)
			{
				MetalLayer.pixelFormat = (MTLPixelFormat) MetalFormat;
			}
			
			[IOSView UpdateRenderWidth:InSizeX andHeight:InSizeY];
			
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
			CachedOrientationMask = [IOSAppDelegate GetMaskFromUIInterfaceOrientation:(FIOSApplication::CachedOrientation)];
#endif
	});

		// Cache new size and orientation
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
		SizeX = InSizeX;
		SizeY = InSizeY;
#if PLATFORM_IOS
		Format = InFormat;
		OrientationMask = CachedOrientationMask;
#if IOS_ROTATION_DEBUG_LOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] FMetalViewport::Resize: OrientationMask %d"), OrientationMask);
#endif
#endif
#endif
	}
#endif

    {
        FScopeLock Lock(&Mutex);

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
		for(TPair<int32, DrawLayer>& DrawLayer : DrawLayers)
		{
			ResizeBackBuffer(DrawLayer.Key, Index, InSizeX, InSizeY, InFormat);
		}
#else
		ResizeBackBuffer(std::numeric_limits<int32>::min(), Index, InSizeX, InSizeY, InFormat);
#endif
	}
}

TRefCountPtr<FMetalSurface> FMetalViewport::GetBackBuffer(EMetalViewportAccessFlag Accessor,
														  TOptional<int32> DrawLayerIndex) const
{
	FScopeLock Lock(&Mutex);
	uint32 Index = GetViewportIndex(Accessor);
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	int32 DrawLayerIndexValue = DrawLayerIndex.IsSet() ? DrawLayerIndex.GetValue() : DefaultNativeLayer;
	
	check(DrawLayers.Contains(DrawLayerIndexValue));
	check(IsValidRef(DrawLayers[DrawLayerIndexValue].BackBuffer[Index]));
	return DrawLayers[DrawLayerIndexValue].BackBuffer[Index];
#else
	check(IsValidRef(BackBuffer[Index]));
	return BackBuffer[Index];
#endif
}

#if PLATFORM_MAC
@protocol CAMetalLayerSPI <NSObject>
- (BOOL)isDrawableAvailable;
@end
#endif

CA::MetalDrawable* FMetalViewport::GetDrawableInternal(EMetalViewportAccessFlag Accessor, TOptional<int32> DrawLayerIndex,
													   bool bLockAlreadyAcquired)
{
	int32 DrawLayerIndexValue = DrawLayerIndex.IsSet() ? DrawLayerIndex.GetValue() : DefaultNativeLayer;
	
	TUniquePtr<FScopeLock> Lock;
	
	if(!bLockAlreadyAcquired)
	{
		Lock = MakeUnique<FScopeLock>(&Mutex);
	}
	
#if PLATFORM_VISIONOS
	// no CAMetalDrawable in Swift mode
	if (SwiftLayer != nullptr)
	{
		return nullptr;
	}
#endif
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	if(!DrawLayers.Contains(DrawLayerIndexValue))
	{
		UE_LOGF(LogMetal, Warning, "Couldn't find DrawLayer with index %d in Metal Viewport %p", DrawLayerIndexValue, this);
		return nullptr;
	}
	
	TRefCountPtr<FMetalSurface> BackBufferPtr = DrawLayers[DrawLayerIndexValue].BackBuffer[GetViewportIndex(Accessor)];
	CA::MetalDrawable* Drawable = DrawLayers[DrawLayerIndexValue].Drawable;
#else
	TRefCountPtr<FMetalSurface> BackBufferPtr = BackBuffer[GetViewportIndex(Accessor)];
#endif
	
	if(BackBufferPtr == nullptr)
	{
		UE_LOGF(LogMetal, Warning, "BackBufferPtr is null with layer %d in Metal Viewport %p", DrawLayerIndexValue, this);
		return nullptr;
	}
	
	
	SCOPE_CYCLE_COUNTER(STAT_MetalMakeDrawableTime);
	if (!Drawable || (Drawable->texture()->width() != BackBufferPtr->GetSizeX() ||
					  Drawable->texture()->height() != BackBufferPtr->GetSizeY()))
	{
		// Drawable changed, release the previously retained object.
		if (Drawable != nullptr)
		{
			Drawable->release();
			Drawable = nullptr;
		}

		MTL_SCOPED_AUTORELEASE_POOL;
		{
			FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUPresent);

#if PLATFORM_MAC
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
			CA::MetalLayer* CurrentLayer = (__bridge CA::MetalLayer*)[DrawLayers[DrawLayerIndexValue].View layer];
#else
			CA::MetalLayer* CurrentLayer = (__bridge CA::MetalLayer*)[View layer];
#endif
			
			if (GMetalNonBlockingPresent == 0 || [((id<CAMetalLayerSPI>)CurrentLayer) isDrawableAvailable])
			{
				Drawable = CurrentLayer ? CurrentLayer->nextDrawable() : nullptr;
			}

#if METAL_DEBUG_OPTIONS
			if (Drawable)
			{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
				TRefCountPtr<FMetalSurface> CurrentBackBuffer = DrawLayers[DrawLayerIndexValue].BackBuffer[GetViewportIndex(Accessor)];
#else
				TRefCountPtr<FMetalSurface> CurrentBackBuffer = BackBuffer[GetViewportIndex(Accessor)];
#endif
				
				CGSize Size = Drawable->layer()->drawableSize();
				if ((Size.width != CurrentBackBuffer->GetSizeX() || Size.height != CurrentBackBuffer->GetSizeY()))
				{
					UE_LOGF(LogMetal, Display, "Viewport Size Mismatch: Drawable W:%f H:%f, Viewport W:%u H:%u", Size.width, Size.height, 
						   CurrentBackBuffer->GetSizeX(), CurrentBackBuffer->GetSizeY());
				}
			}
#endif // METAL_DEBUG_OPTIONS

#else // PLATFORM_MAC
			CGSize Size;
			IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
			do
			{
				Drawable = (__bridge CA::MetalDrawable*)[AppDelegate.IOSView MakeDrawable];
				if (Drawable != nullptr)
				{
					Size.width = Drawable->texture()->width();
					Size.height = Drawable->texture()->height();
				}
				else
				{
					FPlatformProcess::SleepNoStats(0.001f);
				}
			}
			while (Drawable == nullptr || Size.width != BackBuffer[GetViewportIndex(Accessor)]->GetSizeX() || Size.height != BackBuffer[GetViewportIndex(Accessor)]->GetSizeY());

#endif // PLATFORM_MAC
		}

		// Retain the drawable here or it will be released when the
		// autorelease pool goes out of scope.
		if (Drawable != nullptr)
		{
			Drawable->retain();
		}
	}
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	DrawLayers[DrawLayerIndexValue].Drawable = Drawable;
#endif
	
	return Drawable;
}

CA::MetalDrawable* FMetalViewport::GetDrawable(EMetalViewportAccessFlag Accessor,
											   TOptional<int32> DrawLayerIndex)
{
	return GetDrawableInternal(Accessor, DrawLayerIndex, false);
}

MTL::Texture* FMetalViewport::GetDrawableTexture(EMetalViewportAccessFlag Accessor, TOptional<int32> DrawLayerIndex)
{
	FScopeLock Lock(&Mutex);
	int32 DrawLayerIndexValue = DrawLayerIndex.IsSet() ? DrawLayerIndex.GetValue() : DefaultNativeLayer;
	CA::MetalDrawable* CurrentDrawable = GetDrawableInternal(Accessor, DrawLayerIndexValue, true);
	
	if(CurrentDrawable == nullptr)
	{
		UE_LOGF(LogMetal, Warning, "CurrentDrawable is null with layer %d in Metal Viewport %p", DrawLayerIndexValue, this);
		return nullptr;
	}
	
    uint32 Index = GetViewportIndex(Accessor);
    
#if METAL_DEBUG_OPTIONS
    MTL_SCOPED_AUTORELEASE_POOL;

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
    CAMetalLayer* CurrentLayer = (CAMetalLayer*)[DrawLayers[DrawLayerIndexValue].View layer];
	TRefCountPtr<FMetalSurface> BackBufferPtr = DrawLayers[DrawLayerIndexValue].BackBuffer[Index];
#else
#if PLATFORM_MAC
	CAMetalLayer* CurrentLayer = (CAMetalLayer*)[View layer];
#else
    CAMetalLayer* CurrentLayer = (CAMetalLayer*)[[IOSAppDelegate GetDelegate].IOSView layer];
#endif
	TRefCountPtr<FMetalSurface> BackBufferPtr = BackBuffer[Index];
#endif
	
	if(BackBufferPtr == nullptr)
	{
		UE_LOGF(LogMetal, Warning, "BackBufferPtr is null with layer %d in Metal Viewport %p", DrawLayerIndexValue, this);
		return nullptr;
	}
	
    CGSize Size = CurrentLayer.drawableSize;
	CGSize ViewportSize = CGSizeMake(BackBufferPtr->GetSizeX(), BackBufferPtr->GetSizeY());
    if (CurrentDrawable->texture()->width() != ViewportSize.width || 
		CurrentDrawable->texture()->height() != ViewportSize.height)
    {		
        UE_LOGF(LogMetal, Display, "Viewport Size Mismatch: Drawable W:%f H:%f, Texture W:%llu H:%llu, Viewport W:%f H:%f, layer frame W:%f, H:%f", 
		    Size.width, Size.height, 
		    CurrentDrawable->texture()->height(), CurrentDrawable->texture()->height(), 
		    ViewportSize.width, ViewportSize.height, CurrentLayer.frame.size.width, CurrentLayer.frame.size.height);
		
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
		DrawLayers[DrawLayerIndexValue].View.frame.size = ViewportSize;
#endif
    }
#endif
    
	MTL::Texture* ReturnValue = CurrentDrawable->texture();
	if (ReturnValue == nullptr)
	{
		UE_LOGF(LogMetal, Warning, "Drawable has no backing texture in Metal Viewport %p", this);
	}

	DrawableTextures[Index] = ReturnValue;

	return ReturnValue;
}

MTL::Texture* FMetalViewport::GetDrawableTexture(EMetalViewportAccessFlag Accessor, FMetalSurface* SurfaceWithDrawLayer)
{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	for(TPair<int32, DrawLayer> &DrawLayer : DrawLayers)
	{
		for(int i = 0; 2 > i; ++i)
		{
			if(SurfaceWithDrawLayer == DrawLayer.Value.BackBuffer[i])
			{
				return GetDrawableTexture(Accessor, DrawLayer.Key);
			}
		}
	}
	
	return nullptr;
#else
	return GetDrawableTexture(Accessor, TOptional<int32>(std::numeric_limits<int32>::min()));
#endif
}

MTL::Texture* FMetalViewport::GetCurrentTexture(EMetalViewportAccessFlag Accessor)
{
	uint32 Index = GetViewportIndex(Accessor);
	return DrawableTextures[Index];
}

void FMetalViewport::ReleaseDrawable(int32 DrawLayerKey)
{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	TRefCountPtr<FMetalSurface> BackBufferTexture = DrawLayers[DrawLayerKey].BackBuffer[GetViewportIndex(EMetalViewportAccessRHI)];
	CA::MetalDrawable* Drawable = DrawLayers[DrawLayerKey].Drawable;
#else
	TRefCountPtr<FMetalSurface> BackBufferTexture = BackBuffer[GetViewportIndex(EMetalViewportAccessRHI)];
#endif
	
	
	if (Drawable != nullptr)
	{
		Drawable->release();
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
		DrawLayers[DrawLayerKey].Drawable = nullptr;
#else
		Drawable = nullptr;
#endif
	}

	if(!GMetalSupportsIntermediateBackBuffer)
	{
		if (IsValidRef(BackBufferTexture))
		{
			BackBufferTexture->ReleaseDrawableTexture();
		}
	}
}

void FMetalViewport::ReleaseDrawable()
{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	for(TPair<int32, DrawLayer>& DrawLayer : DrawLayers)
	{
		ReleaseDrawable(DrawLayer.Key);
	}
#else
	ReleaseDrawable(std::numeric_limits<int32>::min());
#endif
}

#if PLATFORM_MAC
NSWindow* FMetalViewport::GetWindow() const
{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	return [DrawLayers[std::numeric_limits<int32>::min()].View window];
#else
	return [View window];
#endif
}
#endif

#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
FMetalViewport::CommittedDrawable FMetalViewport::GetCommittedDrawableToPresent(int32 DrawLayer, FMetalCommandBuffer* CommandBuffer,
																bool& bLayersToDraw, FMetalView** TheView)
#else
FMetalViewport::CommittedDrawable FMetalViewport::GetCommittedDrawableToPresent(int32 DrawLayer, FMetalCommandBuffer* CommandBuffer,
																				bool& bLayersToDraw)
#endif
{
	CA::MetalDrawable* LocalDrawable = GetDrawable(EMetalViewportAccessDisplayLink, DrawLayer);
	LocalDrawable->retain();
	MTL::Texture* DrawableTexture = GetDrawableTexture(EMetalViewportAccessDisplayLink, DrawLayer);
	
	if(DrawableTexture)
	{
		check(CommandBuffer);
		
		if (GMetalSupportsIntermediateBackBuffer)
		{
			TRefCountPtr<FMetalSurface> Texture = GetBackBuffer(EMetalViewportAccessRHI, DrawLayer);
			check(IsValidRef(Texture));
			
			MTLTexturePtr Src = Texture->Texture;
			MTLTexturePtr Dst = NS::RetainPtr(DrawableTexture);
			
			NS::UInteger Width = FMath::Min(Src->width(), Dst->width());
			NS::UInteger Height = FMath::Min(Src->height(), Dst->height());
			
			MTLBlitCommandEncoderPtr Encoder = NS::RetainPtr(CommandBuffer->GetMTLCmdBuffer()->blitCommandEncoder());
			check(Encoder);

			Encoder->copyFromTexture(Src.get(), 0, 0, MTL::Origin(0, 0, 0), MTL::Size(Width, Height, 1), Dst.get(), 0, 0, MTL::Origin(0, 0, 0));
			Encoder->endEncoding();
			
			LocalDrawable->release();
			LocalDrawable = nullptr;
		}
		
		if(!bLayersToDraw)
		{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
			check(*TheView == nullptr);
			*TheView = DrawLayers[DrawLayer].View;
#endif
		}
		
		bLayersToDraw = true;
	}
	
	return CommittedDrawable
	{
		.Drawable = LocalDrawable,
		.Texture = DrawableTexture,
		.Key = DrawLayer
	};
}

void FMetalViewport::PresentDrawLayers(FMetalCommandBuffer* CommandBuffer)
{
	FPlatformAtomics::InterlockedDecrement(&FrameAvailable);
	bool bLayersToDraw = false;
	
	TArray<CommittedDrawable> CommittedDrawables;
	
#if PLATFORM_MAC
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	FMetalView* TheView = nullptr;
#else
	FMetalView* TheView = View;
#endif
#endif
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	for(TPair<int32, DrawLayer> &DrawLayer : DrawLayers)
	{
		CommittedDrawables.Add(GetCommittedDrawableToPresent(DrawLayer.Key, CommandBuffer, 
															 bLayersToDraw, &TheView));
	}
#else
	CA::MetalDrawable* LocalDrawable = GetDrawable(EMetalViewportAccessDisplayLink);
	LocalDrawable->retain();
	
	CommittedDrawables.Add(GetCommittedDrawableToPresent(std::numeric_limits<int32>::min(), CommandBuffer, bLayersToDraw));
#endif
	
	if(!bLayersToDraw)
	{
		return;
	}

	dispatch_semaphore_t& FrameSemaphore = Device.GetFrameSemaphore();
	dispatch_retain(FrameSemaphore);
	
#if PLATFORM_MAC
	MTL::HandlerFunction CommandBufferHandler = [TheView, CommittedDrawables, FrameSemaphore](MTL::CommandBuffer* cmd_buf)
#else
	MTL::HandlerFunction CommandBufferHandler = [CommittedDrawables, FrameSemaphore](MTL::CommandBuffer* cmd_buf)
#endif
	{
		dispatch_semaphore_signal(FrameSemaphore);
		dispatch_release(FrameSemaphore);

		for(CommittedDrawable CommittedDrawableInstance : CommittedDrawables)
		{
			if(CommittedDrawableInstance.Texture && CommittedDrawableInstance.Drawable)
			{
				CommittedDrawableInstance.Drawable->release();
			}
		}
#if PLATFORM_MAC
		MainThreadCall(^{
			FCocoaWindow* Window = (FCocoaWindow*)[TheView window];
			[Window startRendering];
		}, false);
#endif
	};
	
#if PLATFORM_MAC		// Mac needs the older way to present otherwise we end up with bad behaviour of the completion handlers that causes GPU timeouts.
	MTL::HandlerFunction ScheduledHandler = [CommittedDrawables](MTL::CommandBuffer*)
	{
		int DrawablesPresented = 0;
		for(CommittedDrawable CommittedDrawableInstance : CommittedDrawables)
		{
			if(CommittedDrawableInstance.Texture && CommittedDrawableInstance.Drawable)
			{
				CommittedDrawableInstance.Drawable->present();
				++DrawablesPresented;
			}
		}
	};
			
	CommandBuffer->GetMTLCmdBuffer()->addCompletedHandler(CommandBufferHandler);
	CommandBuffer->GetMTLCmdBuffer()->addScheduledHandler(ScheduledHandler);

#else // PLATFORM_MAC
	CommandBuffer->GetMTLCmdBuffer()->addCompletedHandler(CommandBufferHandler);

	{
		uint32 FramePace = FPlatformRHIFramePacer::GetFramePace();
		float MinPresentDuration = FramePace ? (1.0f / (float)FramePace) : 0.0f;
		
		// Queue this on the current command buffer to ensure that all work is committed prior to the present, present only knows about dependencies on committed work.
		if (MinPresentDuration && GEnablePresentPacing)
		{
			CommandBuffer->GetMTLCmdBuffer()->presentDrawableAfterMinimumDuration(LocalDrawable, 1.0f/(float)FramePace);
		}
		else
		{
			CommandBuffer->GetMTLCmdBuffer()->presentDrawable(LocalDrawable);
		}
	}
#endif // PLATFORM_MAC
	
	// Wait for the frame semaphore
	dispatch_semaphore_wait(Device.GetFrameSemaphore(), DISPATCH_TIME_FOREVER);
}

void FMetalViewport::Present(FMetalCommandBuffer* CommandBuffer, bool bLockToVsync)
{
	FScopeLock Lock(&Mutex);
	
#if PLATFORM_MAC
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	NSNumber* ScreenId = [DrawLayers[std::numeric_limits<int32>::min()].View.window.screen.deviceDescription objectForKey:@"NSScreenNumber"];
#else
	NSNumber* ScreenId = [View.window.screen.deviceDescription objectForKey:@"NSScreenNumber"];
#endif
	DisplayID = ScreenId.unsignedIntValue;
	{
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
		for(TPair<int32, DrawLayer>& DrawLayer : DrawLayers)
		{
			FCAMetalLayer* CurrentLayer = (FCAMetalLayer*)[DrawLayer.Value.View layer];
			CurrentLayer.displaySyncEnabled = bLockToVsync || (!(IsRunningGame() && bIsFullScreen));
		}
#else
		FCAMetalLayer* CurrentLayer = (FCAMetalLayer*)[View layer];
		CurrentLayer.displaySyncEnabled = bLockToVsync || (!(IsRunningGame() && bIsFullScreen));
#endif
	}
#endif

#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS
	extern bool GIOSDelayRotationUntilPresent;
	if (GIOSDelayRotationUntilPresent)
	{
		// If this frame is at a new orientation, notify to allow rotation to the new orientation
		if (OrientationMask != OldOrientationMask)
		{
			OldOrientationMask = OrientationMask;

			IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
			uint32 ViewportSizeX = SizeX;
			uint32 ViewportSizeY = SizeY;
			UIInterfaceOrientationMask ViewportOrientationMask = OrientationMask;

			// dispatched sync so the main queue can take a snapshot of the view in the old orientation before we complete presenting, to crossfade
			dispatch_sync(dispatch_get_main_queue(), ^{
				[AppDelegate.IOSController notifyPresentAfterRotateOrientationMask:ViewportOrientationMask withSizeX:ViewportSizeX withSizeY:ViewportSizeY];
			});
		}
	}
#endif

	FPlatformAtomics::InterlockedExchange(&FrameAvailable, 1);
	
#if !PLATFORM_MAC
	uint32 FramePace = FPlatformRHIFramePacer::GetFramePace();
	float MinPresentDuration = FramePace ? (1.0f / (float)FramePace) : 0.0f;
#endif
	bool bIsInLiveResize = false;

	if (FrameAvailable > 0)
	{
		PresentDrawLayers(CommandBuffer);
	}
}

#if PLATFORM_VISIONOS
void FMetalViewport::GetDrawableImmersiveTextures(EMetalViewportAccessFlag Accessor, cp_drawable_t SwiftDrawable, MTL::Texture*& OutColorTexture, MTL::Texture*& OutDepthTexture)
{
	check(SwiftDrawable != nullptr);
	
	// get the color texture out and use that with the RHI
	uint32 Index = GetViewportIndex(Accessor);
	uint32 TextureCount = cp_drawable_get_texture_count(SwiftDrawable);
	check(TextureCount = 1);
	OutColorTexture = (__bridge MTL::Texture*)cp_drawable_get_color_texture(SwiftDrawable, 0);
	OutDepthTexture = (__bridge MTL::Texture*)cp_drawable_get_depth_texture(SwiftDrawable, 0);
	DrawableTextures[Index] = OutColorTexture;
}

// This is the present for Immersive visionOS, through the OXRVisionOS plugin.
void FMetalViewport::PresentImmersive(const MetalRHIVisionOS::PresentImmersiveParams& VisionOSParams)
{
	
	// This case means that we are not really submitting a frame to the compositor.
	if (VisionOSParams.PresentChoice == MetalRHIVisionOS::EPresentChoice::DoNotPresent)
	{
		FScopeLock Lock(&Mutex);
		
		dispatch_semaphore_t& FrameSemaphore = Device.GetFrameSemaphore();
		dispatch_semaphore_signal(FrameSemaphore);
		return;
	}
	
	check(SwiftLayer);  // If no SwiftLayer we should not be trying to be immersive.
	check(VisionOSParams.SwiftFrame);

	check(VisionOSParams.RHICommandContext);
	FMetalRHICommandContext& Context = *static_cast<FMetalRHICommandContext*>(VisionOSParams.RHICommandContext);
	
	FScopeLock Lock(&Mutex);
	
	TRefCountPtr<FMetalSurface> MyLastCompleteFrame = GetMetalSurfaceFromRHITexture(VisionOSParams.Texture);
	TRefCountPtr<FMetalSurface> MyLastCompleteDepth = GetMetalSurfaceFromRHITexture(VisionOSParams.Depth);
	{
		if (VisionOSParams.SwiftDrawable)
		{
			MTL::Texture* DrawableTextureParam = nullptr;
			MTL::Texture* DrawableDepthTextureParam = nullptr;
			GetDrawableImmersiveTextures(EMetalViewportAccessDisplayLink, VisionOSParams.SwiftDrawable, DrawableTextureParam, DrawableDepthTextureParam);
			MTLTexturePtr DrawableTexture = NS::RetainPtr(DrawableTextureParam);
			MTLTexturePtr DrawableDepthTexture = NS::RetainPtr(DrawableDepthTextureParam);
			if (DrawableTexture)
			{
				// TODO Currently we are using intermediate back buffer to connect the OXRVisionOS Swapchain to the drawable.
				// I think we could use the drawable directly and avoid this copy.
				check(GMetalSupportsIntermediateBackBuffer);
				if (GMetalSupportsIntermediateBackBuffer)
				{
					{
						TRefCountPtr<FMetalSurface> Texture = MyLastCompleteFrame;
						check(IsValidRef(Texture));
						MTLTexturePtr Src = Texture->Texture;
						MTLTexturePtr& Dst = DrawableTexture;
						
						NSUInteger Width = FMath::Min(Src->width(), Dst->width());
						NSUInteger Height = FMath::Min(Src->height(), Dst->height());
						
						Context.CopyFromTextureToTexture(Src.get(), 0, 0, MTL::Origin(0, 0, 0), MTL::Size(Width, Height, 1), Dst.get(), 0, 0, MTL::Origin(0, 0, 0));
					}
					
					{
						TRefCountPtr<FMetalSurface> Texture = MyLastCompleteDepth;
						check(IsValidRef(Texture));
						MTLTexturePtr Src = Texture->Texture;
						MTLTexturePtr& Dst = DrawableDepthTexture;
						
						NS::UInteger Width = FMath::Min(Src->width(), Dst->width());
						NS::UInteger Height = FMath::Min(Src->height(), Dst->height());
						
						Context.CopyFromTextureToTexture(Src.get(), 0, 0, MTL::Origin(0, 0, 0), MTL::Size(Width, Height, 1), Dst.get(), 0, 0, MTL::Origin(0, 0, 0));
					}
				}
			}
			
			// We need to make sure that any outstanding encoders have been completed before
			// We add our completion handler and encode_present.
			Context.EndCommandBuffer();
			Context.StartCommandBuffer();
			
			// We need to attach the completion handler and the present signal to the final
			// command buffer
			FMetalCommandBuffer* FinalCommandBuffer = Context.GetCurrentCommandBuffer();
			cp_drawable_encode_present(VisionOSParams.SwiftDrawable, (__bridge id<MTLCommandBuffer>)FinalCommandBuffer->GetMTLCmdBuffer());
		}
				
		FMetalCommandBuffer* FinalCommandBuffer = Context.GetCurrentCommandBuffer();;
		
		{
			dispatch_semaphore_t& FrameSemaphore = Device.GetFrameSemaphore();
			dispatch_retain(FrameSemaphore);
			MTL::HandlerFunction CommandBufferHandler = [FrameSemaphore](MTL::CommandBuffer* cmd_buf)
			{
				dispatch_semaphore_signal(FrameSemaphore);
				dispatch_release(FrameSemaphore);
			};
			FinalCommandBuffer->GetMTLCmdBuffer()->addCompletedHandler(CommandBufferHandler);
		}
		
		if (VisionOSParams.SwiftDrawable)
		{
			cp_frame_end_submission(VisionOSParams.SwiftFrame);
		}
		
		// Wait for the frame semaphore
		dispatch_semaphore_wait(Device.GetFrameSemaphore(), DISPATCH_TIME_FOREVER);
	}
}
#endif //PLATFORM_VISIONOS


/*=============================================================================
 *	The following RHI functions must be called from the main thread.
 *=============================================================================*/
FViewportRHIRef FMetalDynamicRHI::RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	check(IsInGameThread());
	return new FMetalViewport(*Device, WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FMetalDynamicRHI::RHIResizeViewport(FRHIViewport* ViewportRHI, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
    MTL_SCOPED_AUTORELEASE_POOL;
	check(IsInGameThread());
	ResourceCast(ViewportRHI)->Resize(SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FMetalDynamicRHI::RHITick( float DeltaTime )
{
	check( IsInGameThread() );
}

/*=============================================================================
 *	Viewport functions.
 *=============================================================================*/

void FMetalRHICommandContext::EndDrawingViewport(FMetalViewport* Viewport, const FRHIPresentArgs& InPresentArgs)
{
    MTL_SCOPED_AUTORELEASE_POOL;

	// enqueue a present if desired
	static bool const bOffscreenOnly = FParse::Param(FCommandLine::Get(), TEXT("MetalOffscreenOnly"));
	if (InPresentArgs.bPresent && !bOffscreenOnly)
	{
		bool bNeedNativePresent = true;
#if PLATFORM_MAC || PLATFORM_VISIONOS
		// Handle custom present
		FRHICustomPresent* const CustomPresent = Viewport->GetCustomPresent();
		if (CustomPresent != nullptr)
		{
			int32 SyncInterval = 0;
			{
				SCOPE_CYCLE_COUNTER(STAT_MetalCustomPresentTime);
                SetCustomPresentViewport(Viewport);
                bNeedNativePresent = CustomPresent->Present(Viewport, *this, SyncInterval);
                SetCustomPresentViewport(nullptr);
			}
			
			if (!CurrentEncoder.GetCommandBuffer())
			{
				StartCommandBuffer();
			}
			FMetalCommandBuffer* CurrentCommandBuffer = CurrentEncoder.GetCommandBuffer();
			check(CurrentCommandBuffer && CurrentCommandBuffer->GetMTLCmdBuffer());
			
			MTL::HandlerFunction Handler = [CustomPresent](MTL::CommandBuffer*) {
				CustomPresent->PostPresent();
			};
			
			CurrentCommandBuffer->GetMTLCmdBuffer()->addScheduledHandler(Handler);
		}
#endif
		
		if (bNeedNativePresent)
		{
			FMetalCommandBuffer* CommandBuffer = CurrentEncoder.GetCommandBuffer();
			if(!CommandBuffer)
			{
				StartCommandBuffer();
				CommandBuffer = CurrentEncoder.GetCommandBuffer();
			}
			Viewport->Present(CommandBuffer, InPresentArgs.bLockToVsync);
		}
	}
	
	Device.EndDrawingViewport(InPresentArgs);
	
	Viewport->ReleaseDrawable();
}

void FMetalDynamicRHI::RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* ViewportRHI, const FRHIPresentArgs& PresentArgs)
{
	//
	// Make sure all prior graphics and async compute work has been submitted. This is necessary because Metal RHI handles the present from inside the below lambda.
	// Since the present is not yet pipelined via payloads etc, we must ensure prior work has been handed to RHISubmitCommandLists before we translate the lambda.
	//
	// In future, Present() itself should be an enqueued command, and platform RHIs should never implicitly send commands to the GPU queues during RHI translation.
	// All GPU work should be submitted via RHISubmitCommandLists.
	//
	// This is also why there is a task dependency between RHI translate and RHI submission tasks in RHICommandList.cpp.
	// See GRHIGlobals.SupportsConcurrentTranslateAndSubmit
	//
	RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

	{
		FRHICommandListScopedPipeline PipelineScope(RHICmdList, ERHIPipeline::Graphics);
	    RHICmdList.EnqueueLambda(TEXT("RHIEndDrawingViewport"), [ViewportRHI, PresentArgs](FRHICommandListImmediate& ExecutingCmdList)
	    {
			// Not all platforms have control on the Present Counter, correlate the information FrameCounter -> PresentCounter before Presenting the frame
			RHIFrameInfoSetPresentCounter(PresentArgs.FrameCounter, GRHIPresentCounter);

			FMetalRHICommandContext& Context = FMetalRHICommandContext::Get(ExecutingCmdList);
			Context.EndDrawingViewport(ResourceCast(ViewportRHI), PresentArgs);
	    });
	}
}

FTextureRHIRef FMetalDynamicRHI::RHIGetViewportBackBuffer(FRHIViewport* ViewportRHI)
{
    MTL_SCOPED_AUTORELEASE_POOL;
    
	FMetalViewport* Viewport = ResourceCast(ViewportRHI);
	return FTextureRHIRef(Viewport->GetBackBuffer(EMetalViewportAccessRenderer, TOptional<int32>()).GetReference());
}

FTextureRHIRef FMetalDynamicRHI::RHIGetViewportBackBufferAtLayerIndex(FRHIViewport* ViewportRHI, uint32 DrawLayerIndex)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	FMetalViewport* Viewport = ResourceCast(ViewportRHI);
	return FTextureRHIRef(Viewport->GetBackBuffer(EMetalViewportAccessRenderer, DrawLayerIndex).GetReference());
}
