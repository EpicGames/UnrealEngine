// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderDebugCache.cpp: Metal RHI Shader Debug Cache.
=============================================================================*/

#include "MetalShaderDebugCache.h"
#include "CoreMinimal.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

#include "MetalShaderDebugZipFile.h"

#if !UE_BUILD_SHIPPING

FMetalShaderDebugZipFile* FMetalShaderDebugCache::GetDebugFile(FString Path)
{
	FScopeLock Lock(&Mutex);
	FMetalShaderDebugZipFile* Ref = DebugFiles.FindRef(Path);
	if (!Ref)
	{
		Ref = new FMetalShaderDebugZipFile(Path);
		DebugFiles.Add(Path, Ref);
	}
	return Ref;
}

NS::String* FMetalShaderDebugCache::GetShaderCode(uint32 ShaderSrcLen, uint32 ShaderSrcCRC)
{
    NS::String* Code = nullptr;
	FScopeLock Lock(&Mutex);
	for (auto const& Ref : DebugFiles)
	{
		Code = Ref.Value->GetShaderCode(ShaderSrcLen, ShaderSrcCRC);
		if (Code)
		{
			break;
		}
	}
    
    return Code;
}

#endif // !UE_BUILD_SHIPPING
