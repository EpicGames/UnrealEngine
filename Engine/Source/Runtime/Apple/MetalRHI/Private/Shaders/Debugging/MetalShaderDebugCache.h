// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderDebugCache.h: Metal RHI Shader Debug Cache.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"

#if !UE_BUILD_SHIPPING

#include "MetalRHIPrivate.h"

struct FMetalShaderDebugCache
{
	static FMetalShaderDebugCache& Get()
	{
		static FMetalShaderDebugCache sSelf;
		return sSelf;
	}
	
	class FMetalShaderDebugZipFile* GetDebugFile(FString Path);
	NS::String* GetShaderCode(uint32 ShaderSrcLen, uint32 ShaderSrcCRC);
	
	FCriticalSection Mutex;
	TMap<FString, class FMetalShaderDebugZipFile*> DebugFiles;
};

#endif // !UE_BUILD_SHIPPING
