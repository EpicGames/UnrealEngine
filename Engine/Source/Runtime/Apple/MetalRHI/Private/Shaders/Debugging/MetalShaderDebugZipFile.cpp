// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderDebugZipFile.cpp: Metal shader RHI implementation.
=============================================================================*/


#include "MetalShaderDebugZipFile.h"
#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

#if !UE_BUILD_SHIPPING

FMetalShaderDebugZipFile::FMetalShaderDebugZipFile(FString LibPath)
	: File(nullptr)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	File = PlatformFile.OpenRead(*LibPath);
	if (File)
	{
		int64 SeekEndOffset = -1;

		// Write normal end of central directory record
		const static uint8 EndRecord[] =
		{
			0x50, 0x4b, 0x05, 0x06, 0xff, 0xff, 0xff, 0xff,
			0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff,
			0xff, 0xff, 0xff, 0xff, 0x00, 0x00
		};

		SeekEndOffset += sizeof(EndRecord);
		bool bOK = File->SeekFromEnd(-SeekEndOffset);
		if (bOK)
		{
			TArray<uint8> Data;
			Data.AddZeroed(sizeof(EndRecord));
			bOK = File->Read(Data.GetData(), sizeof(EndRecord));
			if (bOK)
			{
				bOK = (FMemory::Memcmp(Data.GetData(), EndRecord, sizeof(EndRecord)) == 0);
			}
		}

		// Write ZIP64 end of central directory locator
		const static uint8 Locator[] =
		{
			0x50, 0x4b, 0x06, 0x07, 0x00, 0x00, 0x00, 0x00,
		};
		uint64 DirEndOffset = 0;
		if (bOK)
		{
			SeekEndOffset += sizeof(Locator) + sizeof(uint64) + sizeof(uint32);
			bOK = File->SeekFromEnd(-SeekEndOffset);
			if (bOK)
			{
				TArray<uint8> Data;
				Data.AddZeroed(sizeof(Locator));
				bOK = File->Read(Data.GetData(), sizeof(Locator));
				if (bOK)
				{
					bOK = (FMemory::Memcmp(Data.GetData(), Locator, sizeof(Locator)) == 0);
				}
				if (bOK)
				{
					bOK = File->Read((uint8*)&DirEndOffset, sizeof(uint64));
				}
			}
		}

		// Write ZIP64 end of central directory record
		const static uint8 Record[] =
		{
			0x50, 0x4b, 0x06, 0x06, 0x2c, 0x00, 0x00, 0x00,
			0x00, 0x00, 0x00, 0x00, 0x2d, 0x00, 0x2d, 0x00,
			0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
		};
		struct FMetalZipRecordData
		{
			uint64 FilesNum;
			uint64 FilesNum2;
			uint64 DirectorySizeInBytes;
			uint64 DirStartOffset;
		} RecordData;
		if (bOK)
		{
			SeekEndOffset += sizeof(Record) + (sizeof(uint64) * 4);
			bOK = File->SeekFromEnd(-SeekEndOffset);
			if (bOK)
			{
				TArray<uint8> Data;
				Data.AddZeroed(sizeof(Record));
				bOK = File->Read(Data.GetData(), sizeof(Record));
				if (bOK)
				{
					bOK = (FMemory::Memcmp(Data.GetData(), Record, sizeof(Record)) == 0);
				}
				if (bOK)
				{
					bOK = File->Read((uint8*)&RecordData, sizeof(RecordData));
				}
			}
		}

		if (bOK)
		{
			bOK = File->Seek(RecordData.DirStartOffset);
			if (bOK)
			{
				const static uint8 Footer[] =
				{
					0x50, 0x4b, 0x01, 0x02, 0x3f, 0x00, 0x2d, 0x00,
					0x00, 0x00, 0x00, 0x00
				};
				const static uint8 Fields[] =
				{
					0x20, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
					0x20, 0x00, 0x00, 0x00, 0xff, 0xff, 0xff, 0xff
				};

				TArray<uint8> FooterData;
				FooterData.AddZeroed(sizeof(Footer));
				TArray<uint8> FieldsData;
				FieldsData.AddZeroed(sizeof(Fields));

				struct FMetalZipFileHeader {
					uint32 Time;
					uint32 CRC;
					uint64 SizeMarker;
					uint16 FilenameLen;
				} __attribute__((packed)) Header;

				struct FMetalZipFileTrailer {
					uint16 Flags;
					uint16 Attribs;
					uint64 CompressedLen;
					uint64 UncompressedLen;
					uint64 Offset;
					uint32 DiskNum;
				} __attribute__((packed)) Trailer;

				static const uint8 FileHeader[] =
				{
					0x50, 0x4b, 0x03, 0x04, 0x2d, 0x00, 0x00, 0x00,
					0x00, 0x00
				};

				uint32 FileHeaderFixedSize = sizeof(FileHeader) + sizeof(FMetalZipFileHeader) + sizeof(uint16) + sizeof(FMetalZipFileTrailer);

				FString Filename;

				while (bOK && Files.Num() < RecordData.FilesNum && File->Tell() < RecordData.DirStartOffset + RecordData.DirectorySizeInBytes)
				{
					bOK = File->Read(FooterData.GetData(), sizeof(Footer));
					if (bOK)
					{
						bOK = (FMemory::Memcmp(FooterData.GetData(), Footer, sizeof(Footer)) == 0);
					}

					if (bOK)
					{
						bOK = File->Read((uint8*)&Header, sizeof(Header));
						if (bOK)
						{
							bOK = (Header.SizeMarker == (uint64)0xffffffffffffffff);
						}
					}

					if (bOK)
					{
						bOK = File->Read(FieldsData.GetData(), sizeof(Fields));
						if (bOK)
						{
							bOK = (FMemory::Memcmp(FieldsData.GetData(), Fields, sizeof(Fields)) == 0);
						}
					}

					if (bOK)
					{
						TArray<uint8> FilenameData;
						FilenameData.AddZeroed(Header.FilenameLen+1);
						bOK = File->Read(FilenameData.GetData(), Header.FilenameLen);
						if (bOK)
						{
							Filename = UTF8_TO_TCHAR((char const*)FilenameData.GetData());
						}
					}

					if (bOK)
					{
						bOK = File->Read((uint8*)&Trailer, sizeof(Trailer));
						if (bOK)
						{
							bOK = (Trailer.Flags == (uint16)0x01 && Trailer.Attribs == (uint16)0x1c && Trailer.DiskNum == 0);
						}
					}

					if (bOK)
					{
						FFileEntry NewEntry(Filename, Header.CRC, Trailer.UncompressedLen, Trailer.Offset + FileHeaderFixedSize + Header.FilenameLen, Header.Time);
						Files.Add(NewEntry);
					}
				}
			}
		}
	}
}

FMetalShaderDebugZipFile::~FMetalShaderDebugZipFile()
{
	if (File)
	{
		delete File;
	}
}

NS::String* FMetalShaderDebugZipFile::GetShaderCode(uint32 ShaderSrcLen, uint32 ShaderSrcCRC)
{
    NS::String* OutString = nullptr;
	FScopeLock Lock(&Mutex);
	FString Name = FString::Printf(TEXT("%u_%u.metal"), ShaderSrcLen, ShaderSrcCRC);
	for (auto const& Entry : Files)
	{
		if (FPaths::GetCleanFilename(Entry.Filename) == Name)
		{
			if (File->Seek(Entry.Offset))
			{
				TArray<uint8> Data;
				Data.AddZeroed(Entry.Length+1);
				if (File->Read(Data.GetData(), Entry.Length))
				{
                    OutString = NS::String::string((char const*)Data.GetData(), NS::UTF8StringEncoding);
				}
			}

			break;
		}
	}
    
    return OutString;
}

#endif // !UE_BUILD_SHIPPING
