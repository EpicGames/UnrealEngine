// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderDebugZipFile.h: Metal RHI Shader Debug Zip File.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"

#if !UE_BUILD_SHIPPING

#include "MetalRHIPrivate.h"
#include "Containers/UnrealString.h"

class FMetalShaderDebugZipFile
{
	struct FFileEntry
	{
		FString Filename;
		uint32  Crc32;
		uint64  Length;
		uint64  Offset;
		uint32  Time;

		FFileEntry(const FString& InFilename, uint32 InCrc32, uint64 InLength, uint64 InOffset, uint32 InTime)
			: Filename(InFilename)
			, Crc32(InCrc32)
			, Length(InLength)
			, Offset(InOffset)
			, Time(InTime)
		{
			// VOID
		}
	};

public:
	FMetalShaderDebugZipFile(FString LibPath);
	~FMetalShaderDebugZipFile();

    NS::String* GetShaderCode(uint32 ShaderSrcLen, uint32 ShaderSrcCRC);

private:
	FCriticalSection Mutex;
	IFileHandle* File;
	TArray<FFileEntry> Files;
};

#endif // !UE_BUILD_SHIPPING
