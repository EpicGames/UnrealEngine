// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCompiledShaderCache.cpp: Metal RHI Compiled Shader Cache.
=============================================================================*/

#include "MetalCompiledShaderCache.h"
#include "MetalCompiledShaderKey.h"
#include "MetalCompiledShaderCache.h"

FMetalCompiledShaderCache& GetMetalCompiledShaderCache()
{
	static FMetalCompiledShaderCache CompiledShaderCache;
	return CompiledShaderCache;
}
