// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCompiledShaderCache.h: Metal RHI Compiled Shader Cache.
=============================================================================*/

#pragma once

#include "MetalCompiledShaderKey.h"
#include "Containers/StripedMap.h"
#include "MetalRHIPrivate.h"

struct FCachedCompiledShader
{
	MTLFunctionPtr Function;
	MTLLibraryPtr Library;
};

struct FMetalCompiledShaderCache
{
public:
	FCachedCompiledShader Find(FMetalCompiledShaderKey const& Key)
	{
		return Cache.FindRef(Key);
	}

	void Add(FMetalCompiledShaderKey Key, MTLLibraryPtr Lib, MTLFunctionPtr Function)
	{
		Cache.FindOrProduce(Key, [&]() { return FCachedCompiledShader{ Function, Lib }; });
	}

	void Remove(FMetalCompiledShaderKey const& Key)
	{
		Cache.Remove(Key);
	}

private:
	TStripedMap<32, FMetalCompiledShaderKey, FCachedCompiledShader> Cache;
};

extern FMetalCompiledShaderCache& GetMetalCompiledShaderCache();
