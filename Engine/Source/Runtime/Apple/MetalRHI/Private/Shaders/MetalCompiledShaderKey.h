// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCompiledShaderKey.h: Metal RHI Compiled Shader Key.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "Templates/TypeHash.h"

struct FMetalCompiledShaderKey
{
	FMetalCompiledShaderKey(uint32 CodeSize, uint32 CodeCRC, bool bIsExtendedBytecode = false, uint64 InLibrarySeed = 0)
	{
		// Prevent multiple FRHIShaderLibrary from sharing MTLLibrary & MTLFunction by adding the LibrarySeed inside the hash
		Hash = HashCombine(HashCombine(HashCombine(GetTypeHash(CodeSize), GetTypeHash(CodeCRC)), bIsExtendedBytecode), GetTypeHash(InLibrarySeed));
	}

	friend bool operator ==(const FMetalCompiledShaderKey& A, const FMetalCompiledShaderKey& B)
	{
		return A.Hash == B.Hash;
	}

	friend uint32 GetTypeHash(const FMetalCompiledShaderKey &Key)
	{
		return Key.Hash;
	}

	uint32 Hash;
};
