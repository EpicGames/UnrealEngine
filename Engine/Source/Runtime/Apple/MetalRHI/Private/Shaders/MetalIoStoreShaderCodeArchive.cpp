// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shaders/MetalIoStoreShaderCodeArchive.h"
#include "MetalRHIPrivate.h"
#include "MetalShaderTypes.h"
#if METAL_RHI_RAYTRACING
#include "Shaders/Types/MetalRayShader.h"
#endif

FMetalIoStoreShaderCodeArchive::FMetalIoStoreShaderCodeArchive(EShaderPlatform InPlatform, const FString& InLibraryName, FIoDispatcher& InIoDispatcher)
	: FIoStoreShaderCodeArchive(InPlatform, InLibraryName, InIoDispatcher)
{
}

void FMetalIoStoreShaderCodeArchive::ReleaseMetalObjectsForShader(FRHIShader* Shader)
{
	switch (Shader->GetFrequency())
	{
	case SF_Vertex:        static_cast<FMetalVertexShader*>(Shader)->ReleaseMetalObjects(); break;
	case SF_Pixel:         static_cast<FMetalPixelShader*>(Shader)->ReleaseMetalObjects(); break;
	case SF_Compute:       static_cast<FMetalComputeShader*>(Shader)->ReleaseMetalObjects(); break;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	case SF_Geometry:      static_cast<FMetalGeometryShader*>(Shader)->ReleaseMetalObjects(); break;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:          static_cast<FMetalMeshShader*>(Shader)->ReleaseMetalObjects(); break;
	case SF_Amplification: static_cast<FMetalAmplificationShader*>(Shader)->ReleaseMetalObjects(); break;
#endif
#if METAL_RHI_RAYTRACING
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:   static_cast<FMetalRayShader*>(Shader)->ReleaseMetalObjects(); break;
#endif
	default: break;
	}
}

TRefCountPtr<FRHIShader> FMetalIoStoreShaderCodeArchive::CreateShader(int32 Index, bool bRequired)
{
	TRefCountPtr<FRHIShader> Shader = FIoStoreShaderCodeArchive::CreateShader(Index, bRequired);
	if (Shader)
	{
		CreatedShaders.Enqueue(Shader);
	}
	return Shader;
}

void FMetalIoStoreShaderCodeArchive::WaitForPendingDispatchDataDestructors()
{
	if (PendingDispatchDataDestructorCount.load(std::memory_order_acquire) > 0)
	{
		constexpr double TimeoutSeconds = 5.0;
		const double StartTime = FPlatformTime::Seconds();
		const UE::FMonotonicTimePoint Deadline = UE::FMonotonicTimePoint::Now() + UE::FMonotonicTimeSpan::FromSeconds(TimeoutSeconds);

		while (PendingDispatchDataDestructorCount.load(std::memory_order_acquire) > 0)
		{
			UE::FEventCountToken Token = DispatchDataDestructorEvent.PrepareWait();
			if (PendingDispatchDataDestructorCount.load(std::memory_order_acquire) > 0)
			{
				const bool bNotified = DispatchDataDestructorEvent.WaitUntil(Token, Deadline);
				if (!bNotified)
				{
					UE_LOGF(LogMetal, Warning, "Shader library '%ls' teardown: timed out after %.2f ms waiting for %d Metal shader buffer(s) still held by GCD to be released. Note: Xcode GPU Frame Capture & Metal Validation may hold an additional references to the MTLLibrary, preventing it from being released.",
						*GetName(), (FPlatformTime::Seconds() - StartTime) * 1000.0, PendingDispatchDataDestructorCount.load());
					break;
				}
			}
		}

		UE_LOGF(LogMetal, Display, "Shader library '%ls' teardown: waited %.2f ms for Metal shader buffers held by GCD to be released.",
			*GetName(), (FPlatformTime::Seconds() - StartTime) * 1000.0);
	}
}

void FMetalIoStoreShaderCodeArchive::OnShaderGroupDataOwnerCreated()
{
	PendingDispatchDataDestructorCount.fetch_add(1, std::memory_order_acq_rel);
}

void FMetalIoStoreShaderCodeArchive::OnShaderGroupDataOwnerReleased()
{
	PendingDispatchDataDestructorCount.fetch_sub(1, std::memory_order_acq_rel);
	DispatchDataDestructorEvent.Notify();
}

void FMetalIoStoreShaderCodeArchive::ReleaseMetalObjectsAndWait()
{
	{
		TRefCountPtr<FRHIShader> Shader;
		while (CreatedShaders.Dequeue(Shader))
		{
			ReleaseMetalObjectsForShader(Shader.GetReference());
		}
	}
	WaitForPendingDispatchDataDestructors();
}

void FMetalIoStoreShaderCodeArchive::OnCloseShaderCode()
{
	FIoStoreShaderCodeArchive::OnCloseShaderCode();
	ReleaseMetalObjectsAndWait();
}

void FMetalIoStoreShaderCodeArchive::Teardown()
{
	ReleaseMetalObjectsAndWait();
	FIoStoreShaderCodeArchive::Teardown();
}
