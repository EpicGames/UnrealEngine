// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderCodeArchive.h"
#include "Async/EventCount.h"
#include <atomic>

/**
 * Metal-specific extension of FIoStoreShaderCodeArchive.
 * Tracks created shaders and releases Metal objects (MTLFunction/MTLLibrary) on Teardown,
 * independently of when FRHIShader objects are destroyed.
 */
class FMetalIoStoreShaderCodeArchive : public FIoStoreShaderCodeArchive
{
public:
	FMetalIoStoreShaderCodeArchive(EShaderPlatform InPlatform, const FString& InLibraryName, FIoDispatcher& InIoDispatcher);

	virtual void Teardown() override;
	virtual void OnCloseShaderCode() override;
	virtual TRefCountPtr<FRHIShader> CreateShader(int32 Index, bool bRequired = true) override;

protected:
	virtual void OnShaderGroupDataOwnerCreated() override;
	virtual void OnShaderGroupDataOwnerReleased() override;

private:
	/** Helper function to call ReleaseMetalObjects for all shader types. */
	static void ReleaseMetalObjectsForShader(FRHIShader* Shader);

	/** Releases all Metal objects and waits for pending dispatch_data_t destructors. */
	void ReleaseMetalObjectsAndWait();

	/** Waits for all outstanding dispatch_data_t destructors for this archive to complete. */
	void WaitForPendingDispatchDataDestructors();

	/** Counts dispatch_data_t objects with a custom destructor that are still alive for this archive. */
	std::atomic<int32> PendingDispatchDataDestructorCount{0};

	/** Notified each time a dispatch_data_t destructor completes for this archive. */
	UE::FEventCount DispatchDataDestructorEvent;

	/** All shaders created from this archive. Lock-free MPSC queue since CreateShader can be called from multiple threads. */
	TQueue<TRefCountPtr<FRHIShader>, EQueueMode::Mpsc> CreatedShaders;
};
