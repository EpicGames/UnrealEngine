// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderLibrary.cpp: Metal RHI Shader Library Class Implementation.
=============================================================================*/

#include "MetalShaderLibrary.h"
#include "MetalRHIPrivate.h"
#if !UE_BUILD_SHIPPING
#include "Debugging/MetalShaderDebugCache.h"
#include "Debugging/MetalShaderDebugZipFile.h"
#endif // !UE_BUILD_SHIPPING
#include "MetalShaderTypes.h"

#include "Async/Fundamental/Oversubscription.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

static bool GPersistentMappingMetallib = false;
static FAutoConsoleVariableRef CVarPersistentMappingMetallib(
	TEXT("r.Metal.PersistentMappingMetallib"),
	GPersistentMappingMetallib,
	TEXT("Makes the metallib file mapping persistent."),
	ECVF_Default
);

static void ReleaseLibraryMemory(FMetalShaderLibrary::FShaderLibDataOwner* LibraryMemOwner)
{
	if (!GPersistentMappingMetallib)
	{
		delete LibraryMemOwner;
	}
}

FMetalShaderLibrary::FLazyMetalLib::FLazyMetalLib(FString FilePath, TUniquePtr<FShaderLibDataOwner> Data)
	: Data(MoveTemp(Data))
	, FilePath(MoveTemp(FilePath))
{
}

FMetalShaderLibrary::FLazyMetalLib::~FLazyMetalLib()
{
	UE::TUniqueLock Lock(LibraryMutex);
	MTL::Library* Ptr = Library.load(std::memory_order_relaxed);
	if (Ptr)
	{
		Ptr->release();
	}
}

MTLLibraryPtr FMetalShaderLibrary::FLazyMetalLib::EnsureLoaded(FMetalDevice& MetalDevice)
{
	MTL::Library* Ptr = Library.load(std::memory_order_acquire);
	if (UNLIKELY(Ptr == nullptr))
	{
		// Metal library creation can take significant amount of time, blocking
		// a worker thread, so either we block under the lock or block waiting
		// for a lock. Hence we increment the oversubscription.
		LowLevelTasks::FOversubscriptionScope _;

		if (!LibraryMutex.TryLock())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMetalShaderLibrary_LockContention);
			LibraryMutex.Lock();
		}
		ON_SCOPE_EXIT 
		{
			LibraryMutex.Unlock();
		};

		Ptr = Library.load(std::memory_order_relaxed);
		if (UNLIKELY(Ptr == nullptr))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMetalShaderLibrary_Creation);

			LLM_SCOPE(ELLMTag::Shaders);
			LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
			LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
			UE_TRACE_METADATA_CLEAR_SCOPE();

			void const* RawData = nullptr;
			int64 RawSize = 0;

			FShaderLibDataOwner* MyData = Data.Release();
			if (MyData->MappedRegion.IsValid())
			{
				UE_LOGF(LogMetal, Display, "mmapping %ls, %lld bytes", *FilePath, MyData->MappedCacheFile->GetFileSize());
				RawData = MyData->MappedRegion->GetMappedPtr();
				RawSize = MyData->MappedRegion->GetMappedSize();
			}
			else
			{
				TArray<uint8>& FileData = MyData->Mem;
				if (FFileHelper::LoadFileToArray(FileData, *FilePath))
				{
					UE_LOGF(LogMetal, Display, "emulating mmapping %ls, %d bytes!", *FilePath, FileData.Num());
					RawData = FileData.GetData();
					RawSize = FileData.Num();
				}
			}
			
			check(RawData);
			check(RawSize);
			dispatch_data_t data = dispatch_data_create(RawData, RawSize, nil, ^ {
				ReleaseLibraryMemory(MyData);
			});

			NS::Error* Error;
			Ptr = MetalDevice.GetDevice()->newLibrary(data, &Error);
			dispatch_release(data);

			if (!Ptr)
			{
				UE_LOGF(LogMetal, Error, "Metal library creationg error: %ls", *NSStringToFString(Error->description()));
				UE_LOGF(LogMetal, Fatal, "Failed to create library: %ls", *FilePath);
			}

			Library.store(Ptr, std::memory_order_release);
		}
	}
	check(Ptr);
	return NS::RetainPtr(Ptr);
}

//------------------------------------------------------------------------------

#pragma mark - Metal Shader Library Class Support Routines


template<typename ShaderType>
static TRefCountPtr<FRHIShader> CreateMetalShader(FMetalDevice& Device, TConstArrayView<uint8> InView,  MTLLibraryPtr InLibrary)
{
	ShaderType* Shader = new ShaderType(Device, FRHICreateShaderDesc(InView), InLibrary);
	if (!Shader->GetFunction())
	{
		delete Shader;
		Shader = nullptr;
	}

	return TRefCountPtr<FRHIShader>(Shader);
}


//------------------------------------------------------------------------------

#pragma mark - Metal Shader Library Class Public Static Members


FCriticalSection FMetalShaderLibrary::LoadedShaderLibraryMutex;
TMap<FString, FRHIShaderLibrary*> FMetalShaderLibrary::LoadedShaderLibraryMap;


//------------------------------------------------------------------------------

#pragma mark - Metal Shader Library Class


FMetalShaderLibrary::FMetalShaderLibrary(FMetalDevice& MetalDevice,
										 EShaderPlatform Platform,
										 FString const& Name,
										 const FString& InShaderLibraryFilename,
										 const FMetalShaderLibraryHeader& InHeader,
										 FSerializedShaderArchive&& InSerializedShaders,
										 FShaderCodeArrayType&& InShaderCode,
										 TArray<TUniquePtr<FLazyMetalLib>>&& InLazyLibraries)
	: FRHIShaderLibrary(Platform, Name)
	, Device(MetalDevice)
	, ShaderLibraryFilename(InShaderLibraryFilename)
	, Header(InHeader)
	, SerializedShaders(MoveTemp(InSerializedShaders))
	, ShaderCode(MoveTemp(InShaderCode))
	, LazyLibraries(MoveTemp(InLazyLibraries))
{
#if !UE_BUILD_SHIPPING
	DebugFile = nullptr;

	FName PlatformName = LegacyShaderPlatformToShaderFormat(Platform);
	FString LibName = FString::Printf(TEXT("%s_%s"), *Name, *PlatformName.GetPlainNameString());
	LibName.ToLowerInline();
	FString Path = FPaths::ProjectContentDir() / LibName + TEXT(".zip");

	if (IFileManager::Get().FileExists(*Path))
	{
		DebugFile = FMetalShaderDebugCache::Get().GetDebugFile(Path);
	}
#endif // !UE_BUILD_SHIPPING
}

FMetalShaderLibrary::~FMetalShaderLibrary()
{
	FScopeLock Lock(&LoadedShaderLibraryMutex);
	LoadedShaderLibraryMap.Remove(ShaderLibraryFilename);
}

bool FMetalShaderLibrary::IsNativeLibrary() const
{
	return true;
}

int32 FMetalShaderLibrary::GetNumShaders() const
{
	return SerializedShaders.GetShaderEntries().Num();
}

int32 FMetalShaderLibrary::GetNumShaderMaps() const
{
	return SerializedShaders.GetShaderMapEntries().Num();
}

uint32 FMetalShaderLibrary::GetSizeBytes() const
{
#if USE_MMAPPED_SHADERARCHIVE
	return SerializedShaders.GetAllocatedSize() + ShaderCode.Num() * ShaderCode.GetTypeSize();
#else
	return SerializedShaders.GetAllocatedSize() + ShaderCode.GetAllocatedSize();
#endif
}

int32 FMetalShaderLibrary::GetNumShadersForShaderMap(int32 ShaderMapIndex) const
{
	return SerializedShaders.GetShaderMapEntries()[ShaderMapIndex].NumShaders;
}

int32 FMetalShaderLibrary::GetShaderIndex(int32 ShaderMapIndex, int32 i) const
{
	const FShaderMapEntry& ShaderMapEntry = SerializedShaders.GetShaderMapEntries()[ShaderMapIndex];
	return SerializedShaders.GetShaderIndices()[ShaderMapEntry.ShaderIndicesOffset + i];
}

int32 FMetalShaderLibrary::FindShaderMapIndex(const FShaderHash& Hash)
{
	return SerializedShaders.FindShaderMap(Hash);
}

int32 FMetalShaderLibrary::FindShaderIndex(const FShaderHash& Hash)
{
	return SerializedShaders.FindShader(Hash);
}

TRefCountPtr<FRHIShader> FMetalShaderLibrary::CreateShader(int32 Index, bool bRequired)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMetalShaderLibrary::CreateShader);

	const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[Index];

	// We don't handle compressed shaders here, since typically these are just tiny headers.
	check(!ShaderEntry.IsCompressed());

	const TArrayView<const uint8> Code = MakeArrayView(ShaderCode.GetData() + ShaderEntry.Offset, ShaderEntry.Size);
	const int32 LibraryIndex = ShaderEntry.GetLibraryIndex() / Header.NumShadersPerLibrary;

	MTLLibraryPtr Library = LazyLibraries[LibraryIndex]->EnsureLoaded(Device);
	check(Library.get() != nullptr);
	
	TRefCountPtr<FRHIShader> Shader;
	switch (ShaderEntry.GetFrequency())
	{
		case SF_Vertex:
			Shader = CreateMetalShader<FMetalVertexShader>(Device, Code, Library);
			break;

		case SF_Pixel:
			Shader = CreateMetalShader<FMetalPixelShader>(Device, Code, Library);
			break;
 
 		case SF_Geometry:
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
            Shader = CreateMetalShader<FMetalGeometryShader>(Device, Code, Library);
#else
            checkf(false, TEXT("Geometry shaders not supported"));
#endif
            break;


        case SF_Mesh:
#if PLATFORM_SUPPORTS_MESH_SHADERS
            Shader = CreateMetalShader<FMetalMeshShader>(Device, Code, Library);
#else
			checkf(false, TEXT("Mesh shaders not supported"));
#endif
            break;

        case SF_Amplification:
#if PLATFORM_SUPPORTS_MESH_SHADERS
            Shader = CreateMetalShader<FMetalAmplificationShader>(Device, Code, Library);
#else
			checkf(false, TEXT("Amplification shaders not supported"));
#endif
            break;

		case SF_Compute:
			Shader = CreateMetalShader<FMetalComputeShader>(Device, Code, Library);
			break;

#if METAL_RHI_RAYTRACING
		case SF_RayGen:
		case SF_RayMiss:
		case SF_RayHitGroup:
		case SF_RayCallable:
		{
			FMetalRayShader* RayShader = new FMetalRayShader(Device, FRHICreateShaderDesc(Code), (EShaderFrequency)ShaderEntry.GetFrequency(), Library);
			Shader = TRefCountPtr<FRHIShader>(RayShader);
			break;
		}
#endif
			
		default:
			Shader = nullptr;
			checkNoEntry();
			break;
	}

	if (Shader)
	{
		Shader->SetHash(SerializedShaders.GetShaderHashes()[Index]);
	}

	return Shader;
}
