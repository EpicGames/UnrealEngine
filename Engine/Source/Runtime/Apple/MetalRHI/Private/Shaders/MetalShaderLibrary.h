// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderLibrary.h: Metal RHI Shader Library Class.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "MetalShaderResources.h"
#include "Misc/CoreDelegates.h"
#include "RHIShaderLibrary.h"
#include "ShaderCodeArchive.h"
#include "Async/MappedFileHandle.h"

#include <atomic>

class FMetalDevice;

class FMetalShaderLibrary final : public FRHIShaderLibrary
{
public:

	struct FShaderLibDataOwner
	{
		TArray<uint8> Mem;
		TUniquePtr<IMappedFileHandle> MappedCacheFile;
		TUniquePtr<IMappedFileRegion> MappedRegion;
	};

	class FLazyMetalLib
	{
	public:
		FLazyMetalLib(FString FilePath, TUniquePtr<FShaderLibDataOwner> Data);
		~FLazyMetalLib();
		
		FLazyMetalLib(FLazyMetalLib&& Other) = delete;
		FLazyMetalLib(FLazyMetalLib const&) = delete;
		FLazyMetalLib& operator=(FLazyMetalLib const&) = delete;
		
		MTLLibraryPtr EnsureLoaded(FMetalDevice& MetalDevice);

	private:
		TUniquePtr<FShaderLibDataOwner> Data;
		FString FilePath;
		UE::FMutex LibraryMutex;
		std::atomic<MTL::Library*> Library = nullptr;
	};
	
#if !USE_MMAPPED_SHADERARCHIVE
    using FShaderCodeArrayType = TArray<uint8>;
#else 
	using FShaderCodeArrayType = TArrayView<const uint8>;
private:
	TUniquePtr<FShaderLibDataOwner> MemOwner;
public:
    FMetalShaderLibrary(FMetalDevice& Device,
		EShaderPlatform Platform,
        FString const& Name,
        const FString& InShaderLibraryFilename,
        const FMetalShaderLibraryHeader& InHeader,
        FSerializedShaderArchive&& InSerializedShaders,
        FShaderCodeArrayType&& InShaderCode,
		TArray<TUniquePtr<FLazyMetalLib>>&& InLazyLibraries,
        TUniquePtr<FShaderLibDataOwner>&& InMemOwner)
        :
        FMetalShaderLibrary(Device, Platform, Name, InShaderLibraryFilename, InHeader, MoveTemp(InSerializedShaders), MoveTemp(InShaderCode), MoveTemp(InLazyLibraries))
        {
			MemOwner = MoveTemp(InMemOwner);
        }
#endif
  
	static FCriticalSection LoadedShaderLibraryMutex;
	static TMap<FString, FRHIShaderLibrary*> LoadedShaderLibraryMap;

	FMetalShaderLibrary(FMetalDevice& Device,
						EShaderPlatform Platform,
						FString const& Name,
						const FString& InShaderLibraryFilename,
						const FMetalShaderLibraryHeader& InHeader,
						FSerializedShaderArchive&& InSerializedShaders,
						FShaderCodeArrayType&& InShaderCode,
						TArray<TUniquePtr<FLazyMetalLib>>&& InLazyLibraries);

	virtual ~FMetalShaderLibrary();

	virtual bool IsNativeLibrary() const override final;

	virtual int32 GetNumShaders() const override;
	virtual int32 GetNumShaderMaps() const override;
	virtual uint32 GetSizeBytes() const override;
	virtual int32 GetNumShadersForShaderMap(int32 ShaderMapIndex) const override;
	virtual int32 GetShaderIndex(int32 ShaderMapIndex, int32 i) const override;

	virtual int32 FindShaderMapIndex(const FShaderHash& Hash) override;
	virtual int32 FindShaderIndex(const FShaderHash& Hash) override;
	virtual FShaderHash GetShaderHash(int32 ShaderMapIndex, int32 ShaderIndex) override
	{ 
		return SerializedShaders.GetShaderHashes()[GetShaderIndex(ShaderMapIndex, ShaderIndex)];
	};

	virtual FShaderHash GetShaderMapHash(int32 ShaderMapIndex) const override
	{
		return SerializedShaders.GetShaderMapHashes()[ShaderMapIndex];
	};

	virtual bool PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) override { return false; }
	virtual bool PreloadShaderMap(int32 ShaderMapIndex, FGraphEventArray& OutCompletionEvents) override { return false; }

	virtual TRefCountPtr<FRHIShader> CreateShader(int32 Index, bool bRequired = true) override;

private:
	FMetalDevice& Device;
	FString ShaderLibraryFilename;
	FMetalShaderLibraryHeader Header;
	FSerializedShaderArchive SerializedShaders;
	FShaderCodeArrayType ShaderCode;
	TArray<TUniquePtr<FLazyMetalLib>> LazyLibraries;
#if !UE_BUILD_SHIPPING
	class FMetalShaderDebugZipFile* DebugFile;
#endif
};
