// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderParameterCache.cpp: Metal RHI Shader Parameter Cache Class.
=============================================================================*/


#include "MetalShaderParameterCache.h"
#include "MetalCommandEncoder.h"
#include "MetalResources.h"
#include "MetalRHIPrivate.h"
#include "MetalStateCache.h"


//------------------------------------------------------------------------------

#pragma mark - Metal Shader Parameter Cache Class


FMetalShaderParameterCache::FMetalShaderParameterCache()
{
	for (int32 ArrayIndex = 0; ArrayIndex < CrossCompiler::PACKED_TYPEINDEX_MAX; ++ArrayIndex)
	{
		PackedGlobalUniforms[ArrayIndex] = nullptr;
		PackedGlobalUniformsSizes[ArrayIndex] = 0;
		PackedGlobalUniformDirty[ArrayIndex].LowVector = 0;
		PackedGlobalUniformDirty[ArrayIndex].HighVector = 0;
	}
}

void FMetalShaderParameterCache::ResizeGlobalUniforms(uint32 TypeIndex, uint32 UniformArraySize)
{
	if (!PackedGlobalUniforms[TypeIndex])
	{
        PackedGlobalUniforms[TypeIndex] = new FMetalBufferData;
        PackedGlobalUniforms[TypeIndex]->InitWithSize(UniformArraySize);
	}
	else
	{
        PackedGlobalUniforms[TypeIndex]->Data = (uint8*)FMemory::Realloc(PackedGlobalUniforms[TypeIndex]->Data, UniformArraySize);
        PackedGlobalUniforms[TypeIndex]->Len = UniformArraySize;
	}
	PackedGlobalUniformsSizes[TypeIndex] = UniformArraySize;
	PackedGlobalUniformDirty[TypeIndex].LowVector = 0;
	PackedGlobalUniformDirty[TypeIndex].HighVector = 0;
}

FMetalShaderParameterCache::~FMetalShaderParameterCache()
{
	for (int32 ArrayIndex = 0; ArrayIndex < CrossCompiler::PACKED_TYPEINDEX_MAX; ++ArrayIndex)
	{
        if(PackedGlobalUniforms[ArrayIndex])
        {
            delete PackedGlobalUniforms[ArrayIndex];
        }
	}
}

void FMetalShaderParameterCache::Reset()
{
	for (int32 ArrayIndex = 0; ArrayIndex < CrossCompiler::PACKED_TYPEINDEX_MAX; ++ArrayIndex)
	{
		PackedGlobalUniformDirty[ArrayIndex].LowVector = 0;
		PackedGlobalUniformDirty[ArrayIndex].HighVector = 0;
	}
}

void FMetalShaderParameterCache::MarkAllDirty()
{
	for (int32 ArrayIndex = 0; ArrayIndex < CrossCompiler::PACKED_TYPEINDEX_MAX; ++ArrayIndex)
	{
		PackedGlobalUniformDirty[ArrayIndex].LowVector = 0;
		PackedGlobalUniformDirty[ArrayIndex].HighVector = PackedGlobalUniformsSizes[ArrayIndex] / SizeOfFloat;
	}
}

void FMetalShaderParameterCache::Set(uint32 BufferIndexName, uint32 ByteOffset, uint32 NumBytes, const void* NewValues)
{
	if (NumBytes)
	{
		uint32 BufferIndex = CrossCompiler::PackedTypeNameToTypeIndex(BufferIndexName);
		check(BufferIndex < CrossCompiler::PACKED_TYPEINDEX_MAX);
		check(PackedGlobalUniforms[BufferIndex]);
		check(ByteOffset + NumBytes <= PackedGlobalUniformsSizes[BufferIndex]);
		PackedGlobalUniformDirty[BufferIndex].LowVector = FMath::Min(PackedGlobalUniformDirty[BufferIndex].LowVector, ByteOffset / SizeOfFloat);
		PackedGlobalUniformDirty[BufferIndex].HighVector = FMath::Max(PackedGlobalUniformDirty[BufferIndex].HighVector, (ByteOffset + NumBytes + SizeOfFloat - 1) / SizeOfFloat);
        FMemory::Memcpy(PackedGlobalUniforms[BufferIndex]->Data + ByteOffset, NewValues, NumBytes);
	}
}

void FMetalShaderParameterCache::CommitPackedGlobals(FMetalStateCache* Cache, FMetalCommandEncoder* Encoder, uint32 Frequency, const FMetalShaderBindings& Bindings)
{
	// copy the current uniform buffer into the ring buffer to submit
	for (int32 Index = 0; Index < Bindings.PackedGlobalArrays.Num(); ++Index)
	{
		int32 UniformBufferIndex = Bindings.PackedGlobalArrays[Index].TypeIndex;

		// is there any data that needs to be copied?
		if (PackedGlobalUniformDirty[Index].HighVector > 0)
		{
			uint32 TotalSize = Bindings.PackedGlobalArrays[Index].Size;
			uint32 SizeToUpload = PackedGlobalUniformDirty[Index].HighVector * SizeOfFloat;

			//@todo-rco: Temp workaround
			SizeToUpload = TotalSize;

			//@todo-rco: Temp workaround
			uint32 Size = FMath::Min(TotalSize, SizeToUpload);
#if METAL_USE_METAL_SHADER_CONVERTER
			if(IsMetalBindlessEnabled())
			{
				check(PackedGlobalUniforms[Index]);
				uint8 const* Bytes = PackedGlobalUniforms[Index]->Data;
				
				FMetalBufferPtr Buffer;
				PackedGlobalUniforms[Index]->Len = Size;
				Cache->IRBindPackedUniforms((EMetalShaderStages)Frequency, UniformBufferIndex, Bytes, TotalSize, Buffer);
			}
			else
#endif
			{
				if (Size > MetalBufferPageSize)
				{
					uint8 const* Bytes = PackedGlobalUniforms[Index]->Data;
					FMetalBufferPtr Buffer = Encoder->GetRingBuffer().NewBuffer(Size, 0);
					FMemory::Memcpy((uint8*)Buffer->Contents(), Bytes, Size);
					Cache->SetShaderBuffer((EMetalShaderStages)Frequency, Buffer, nullptr, 0, Size, UniformBufferIndex, MTL::ResourceUsageRead);
				}
				else
				{
					PackedGlobalUniforms[Index]->Len = Size;
					Cache->SetShaderBuffer((EMetalShaderStages)Frequency, nullptr, nullptr, 0, 0, UniformBufferIndex, MTL::ResourceUsage(0));
					Cache->SetShaderBuffer((EMetalShaderStages)Frequency, nullptr, PackedGlobalUniforms[Index], 0, Size, UniformBufferIndex, MTL::ResourceUsageRead);
				}
			}
			// mark as clean
			PackedGlobalUniformDirty[Index].HighVector = 0;
		}
	}
}
