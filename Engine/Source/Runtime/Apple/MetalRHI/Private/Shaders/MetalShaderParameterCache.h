// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderParameterCache.h: Metal RHI Shader Parameter Cache Class.
=============================================================================*/

#pragma once

#include "CrossCompilerCommon.h"
#include "MetalRHIPrivate.h"

class FMetalBufferData;
struct FMetalShaderBindings;

class FMetalShaderParameterCache
{
public:
	/** Constructor. */
	FMetalShaderParameterCache();

	/** Destructor. */
	~FMetalShaderParameterCache();

	inline void PrepareGlobalUniforms(uint32 TypeIndex, uint32 UniformArraySize)
	{
		if (PackedGlobalUniformsSizes[TypeIndex] < UniformArraySize)
		{
			ResizeGlobalUniforms(TypeIndex, UniformArraySize);
		}
	}

	/**
	 * Invalidates all existing data.
	 */
	void Reset();

	/**
	 * Marks all uniform arrays as dirty.
	 */
	void MarkAllDirty();

	/**
	 * Sets values directly into the packed uniform array
	 */
	void Set(uint32 BufferIndex, uint32 ByteOffset, uint32 NumBytes, const void* NewValues);

	/**
	 * Commit shader parameters to the currently bound program.
	 */
	void CommitPackedGlobals(class FMetalStateCache* Cache, class FMetalCommandEncoder* Encoder, uint32 Frequency, const FMetalShaderBindings& Bindings);

private:
	static constexpr uint32 SizeOfFloat = sizeof(float);

	/** CPU memory block for storing uniform values. */
	FMetalBufferData* PackedGlobalUniforms[CrossCompiler::PACKED_TYPEINDEX_MAX];

	struct FRange
	{
		uint32 LowVector;
		uint32 HighVector;
	};

	/** Dirty ranges for each uniform array. */
	FRange PackedGlobalUniformDirty[CrossCompiler::PACKED_TYPEINDEX_MAX];

	uint32 PackedGlobalUniformsSizes[CrossCompiler::PACKED_TYPEINDEX_MAX];

	void ResizeGlobalUniforms(uint32 TypeIndex, uint32 UniformArraySize);
};
