// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalAmplificationShader.cpp: Metal RHI Amplification Shader Class Implementation.
=============================================================================*/


#include "MetalAmplificationShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Amplification Shader Class

#if PLATFORM_SUPPORTS_MESH_SHADERS
FMetalAmplificationShader::FMetalAmplificationShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc) : TMetalBaseShader<FRHIAmplificationShader, SF_Amplification>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, MTLLibraryPtr());
}

FMetalAmplificationShader::FMetalAmplificationShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary) : TMetalBaseShader<FRHIAmplificationShader, SF_Amplification>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, InLibrary);
}

MTLFunctionPtr FMetalAmplificationShader::GetFunction()
{
	return GetCompiledFunction();
}
#endif
