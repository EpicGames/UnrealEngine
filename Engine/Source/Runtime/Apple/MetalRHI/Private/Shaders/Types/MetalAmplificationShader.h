// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalAmplificationShader.h: Metal RHI Amplification Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Shaders/Types/Templates/MetalBaseShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Amplification Shader Class


#if PLATFORM_SUPPORTS_MESH_SHADERS
class FMetalAmplificationShader : public TMetalBaseShader<FRHIAmplificationShader, SF_Amplification>
{
public:
    FMetalAmplificationShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc);
    FMetalAmplificationShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary);

    MTLFunctionPtr GetFunction();
};
#endif
