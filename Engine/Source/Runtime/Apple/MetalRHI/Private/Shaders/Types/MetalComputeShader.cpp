// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalComputeShader.cpp: Metal RHI Compute Shader Class Implementation.
=============================================================================*/

#include "MetalComputeShader.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "MetalCommandBuffer.h"
#include "MetalDevice.h"
#include "MetalProfiler.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Compute Shader Class

extern int32 GMetalShaderValidationType;
extern FString GMetalShaderValidationShaderName;

FMetalComputeShader::FMetalComputeShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary)
	: TMetalBaseShader<FRHIComputeShader, SF_Compute>(Device)
	, NumThreadsX(0)
	, NumThreadsY(0)
	, NumThreadsZ(0)
{
	Pipeline = nullptr;
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, InLibrary);

#if METAL_RHI_RAYTRACING
	RayTracingBindings = Header.RayTracing;
#endif // METAL_RHI_RAYTRACING

	NumThreadsX = FMath::Max((int32)Header.NumThreadsX, 1);
	NumThreadsY = FMath::Max((int32)Header.NumThreadsY, 1);
	NumThreadsZ = FMath::Max((int32)Header.NumThreadsZ, 1);
}

FMetalComputeShader::~FMetalComputeShader()
{
    if(Pipeline)
    {
        Pipeline = nullptr;
    }
}

void FMetalComputeShader::ReleaseMetalObjects()
{
	Pipeline.Reset();
	TMetalBaseShader::ReleaseMetalObjects();
}

FMetalShaderPipelinePtr FMetalComputeShader::GetPipeline()
{
	FScopeLock Lock(&PipelineCS);
	
	if (!Pipeline)
	{
        MTL_SCOPED_AUTORELEASE_POOL;
        
        MTLFunctionPtr Func = GetCompiledFunction();
		check(Func);

		NS::Error* Error;
		MTL::ComputePipelineDescriptor* Descriptor = MTL::ComputePipelineDescriptor::alloc()->init();
        check(Descriptor);
		
#if !UE_BUILD_SHIPPING
		if(Device.IsShaderValidationEnabled())
		{
			bool bEnableValidation = false;
			
			// Handle Validation
			switch((EMetalShaderValidationType)GMetalShaderValidationType)
			{
				case EMetalShaderValidationType::All:
				case EMetalShaderValidationType::ComputeOnly:
					bEnableValidation = true;
					break;
				case EMetalShaderValidationType::ShaderName:
				{
					if(NSStringToFString(Func->name()).Contains(GMetalShaderValidationShaderName)) 
					{
						bEnableValidation = true;
					}
				}
				case EMetalShaderValidationType::RenderPipelineOnly:
				default:
					break;
			}
			
			if(bEnableValidation)
			{
				Descriptor->setShaderValidation(MTL::ShaderValidationEnabled);
			}
		}
#endif 
		
		Descriptor->setLabel(Func->name());
		Descriptor->setComputeFunction(Func.get());
        
		Descriptor->setMaxTotalThreadsPerThreadgroup(NumThreadsX*NumThreadsY*NumThreadsZ);

		MTL::PipelineBufferDescriptorArray* PipelineBuffers = Descriptor->buffers();

		uint32 ImmutableBuffers = Bindings.ConstantBuffers | Bindings.ArgumentBuffers;
		while(ImmutableBuffers)
		{
			uint32 Index = __builtin_ctz(ImmutableBuffers);
			ImmutableBuffers &= ~(1 << Index);

			if (Index < ML_MaxBuffers)
			{
				MTL::PipelineBufferDescriptor* PipelineBuffer = PipelineBuffers->object(Index);
				PipelineBuffer->setMutability(MTL::MutabilityImmutable);
			}
		}
		if (SideTableBinding > 0)
		{
			MTL::PipelineBufferDescriptor* PipelineBuffer = PipelineBuffers->object(SideTableBinding);
			PipelineBuffer->setMutability(MTL::MutabilityImmutable);
		}

        MTLComputePipelineStatePtr Kernel;
		MTL::ComputePipelineReflection* Reflection = nullptr;

#if METAL_DEBUG_OPTIONS
		if (Device.GetRuntimeDebuggingLevel() >= EMetalDebugLevelFastValidation)
		{
			NS::Error* ComputeError = nullptr;
            MTL::ComputePipelineReflection* ComputeReflection = nullptr;
            
			NS::UInteger ComputeOption = MTL::PipelineOptionArgumentInfo | MTL::PipelineOptionBufferTypeInfo;
			Kernel = NS::TransferPtr(Device.GetDevice()->newComputePipelineState(Descriptor, MTL::PipelineOption(ComputeOption), &ComputeReflection, &ComputeError));
			Error = ComputeError;
			Reflection = ComputeReflection;
		}
		else
#endif // METAL_DEBUG_OPTIONS
		{
			NS::Error* ComputeError;
			Kernel = NS::TransferPtr(Device.GetDevice()->newComputePipelineState(Descriptor, MTL::PipelineOption(0), nullptr, &ComputeError));
			Error = ComputeError;
		}

		if (Kernel.get() == nullptr)
		{
			UE_LOGF(LogRHI, Error, "*********** Error\n%ls", *NSStringToFString(GetSourceCode()));
			UE_LOGF(LogRHI, Fatal, "Failed to create compute kernel: %ls", *NSStringToFString(Error->description()));
		}

		Pipeline = FMetalShaderPipelinePtr(new FMetalShaderPipeline(Device));
		Pipeline->ComputePipelineState = Kernel;
		check(Pipeline->ComputePipelineState);
#if METAL_DEBUG_OPTIONS
        if(Reflection)
        {
            Pipeline->ComputePipelineReflection = NS::RetainPtr(Reflection);
        }
        
		Pipeline->ComputeSource = GetSourceCode();
        Pipeline->ComputeSource->retain();
        
		if (Reflection)
		{
			Pipeline->ComputeDesc = NS::RetainPtr(Descriptor);
		}
#endif // METAL_DEBUG_OPTIONS
        Descriptor->release();
        
		METAL_DEBUG_OPTION(FMemory::Memzero(Pipeline->ResourceMask, sizeof(Pipeline->ResourceMask)));
	}
	check(Pipeline);
	check(Pipeline->ComputePipelineState);

	return FMetalShaderPipelinePtr(Pipeline);
}

MTLFunctionPtr FMetalComputeShader::GetFunction()
{
	return GetCompiledFunction();
}

