// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalComputeShader.h: Metal RHI Compute Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Shaders/Types/Templates/MetalBaseShader.h"


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Compute Shader Class

class FMetalDevice;
class FMetalComputeShader : public TMetalBaseShader<FRHIComputeShader, SF_Compute>
{
public:
	FMetalComputeShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary);
	virtual ~FMetalComputeShader();

	// Release the Metal Pipeline.
	virtual void ReleaseMetalObjects() override;

	FMetalShaderPipelinePtr GetPipeline();
    MTLFunctionPtr GetFunction();

	// thread group counts
	int32 NumThreadsX;
	int32 NumThreadsY;
	int32 NumThreadsZ;

#if METAL_RHI_RAYTRACING
	/** Meta-data for function table binding indexes (UINT32_MAX if unavailable). */
	FMetalRayTracingHeader RayTracingBindings;
#endif // METAL_RHI_RAYTRACING

private:
	// the state object for a compute shader
	FMetalShaderPipelinePtr Pipeline;
	
	FCriticalSection PipelineCS;
};
