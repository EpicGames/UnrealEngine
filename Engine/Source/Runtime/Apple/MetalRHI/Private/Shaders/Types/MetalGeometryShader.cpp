// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalGeometryShader.cpp: Metal RHI Geometry Shader Class Implementation.
=============================================================================*/

#include "MetalGeometryShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Geometry Shader Class

#if METAL_USE_METAL_SHADER_CONVERTER
FMetalGeometryShader::FMetalGeometryShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc) :
	TMetalBaseShader<FRHIGeometryShader, SF_Geometry>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, MTLLibraryPtr());
}

FMetalGeometryShader::FMetalGeometryShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary) :
	TMetalBaseShader<FRHIGeometryShader, SF_Geometry>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, InLibrary);
}

MTLFunctionPtr FMetalGeometryShader::GetFunction()
{
	return GetCompiledFunction();
}
#endif
