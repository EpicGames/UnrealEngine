// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalGeometryShader.h: Metal RHI Geometry Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Shaders/Types/Templates/MetalBaseShader.h"


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Geometry Shader Class


class FMetalGeometryShader : public TMetalBaseShader<FRHIGeometryShader, SF_Geometry>
{
#if METAL_USE_METAL_SHADER_CONVERTER
public:
    FMetalGeometryShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc);
    FMetalGeometryShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary);

    MTLFunctionPtr GetFunction();
#endif
};
