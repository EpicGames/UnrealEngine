// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalMeshShader.cpp: Metal RHI Mesh Shader Class Implementation.
=============================================================================*/

#include "MetalMeshShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Mesh Shader Class

#if PLATFORM_SUPPORTS_MESH_SHADERS
FMetalMeshShader::FMetalMeshShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc)
	: TMetalBaseShader<FRHIMeshShader, SF_Mesh>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, MTLLibraryPtr());
}

FMetalMeshShader::FMetalMeshShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary)
	: TMetalBaseShader<FRHIMeshShader, SF_Mesh>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, InLibrary);
}

MTLFunctionPtr FMetalMeshShader::GetFunction()
{
	return GetCompiledFunction();
}
#endif
