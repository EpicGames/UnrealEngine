// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalMeshShader.h: Metal RHI Mesh Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Shaders/Types/Templates/MetalBaseShader.h"


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Mesh Shader Class


#if PLATFORM_SUPPORTS_MESH_SHADERS
class FMetalMeshShader : public TMetalBaseShader<FRHIMeshShader, SF_Mesh>
{
public:
    FMetalMeshShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc);
    FMetalMeshShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary);

    MTLFunctionPtr GetFunction();
};
#endif
