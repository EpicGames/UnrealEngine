// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalPixelShader.cpp: Metal RHI Pixel Shader Class Implementation.
=============================================================================*/

#include "MetalPixelShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Pixel Shader Class


FMetalPixelShader::FMetalPixelShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc)
	: TMetalBaseShader<FRHIPixelShader, SF_Pixel>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, MTLLibraryPtr());
}

FMetalPixelShader::FMetalPixelShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary)
	: TMetalBaseShader<FRHIPixelShader, SF_Pixel>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, InLibrary);
}

MTLFunctionPtr FMetalPixelShader::GetFunction()
{
	return GetCompiledFunction();
}
