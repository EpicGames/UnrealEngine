// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalPixelShader.h: Metal RHI Pixel Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Shaders/Types/Templates/MetalBaseShader.h"


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Pixel Shader Class


class FMetalPixelShader : public TMetalBaseShader<FRHIPixelShader, SF_Pixel>
{
public:
	FMetalPixelShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc);
	FMetalPixelShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary);

	MTLFunctionPtr GetFunction();
};
