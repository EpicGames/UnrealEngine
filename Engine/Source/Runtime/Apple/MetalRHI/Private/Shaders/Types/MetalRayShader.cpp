// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRayShader.cpp: Metal RHI Ray Shader Class Implementation.
=============================================================================*/

#include "MetalRayShader.h"
#include "Templates/MetalBaseShader.h"
#include "MetalShaderResources.h"

#if METAL_RHI_RAYTRACING

FMetalRayShader::FMetalRayShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency InFrequency, MTLLibraryPtr InLibrary)
	: FRHIRayTracingShader(InFrequency)
	, Frequency(InFrequency)
{
	check(IsRayTracingShaderFrequency(InFrequency));
	FMetalCodeHeader Header;
	FShaderResourceTable SRT;
	
	FShaderCodeReader ShaderCodeReader(CreateShaderDesc.Code);
	
	uint8 OfflineCompiledFlag = 0;
	IRShaderStage DefaultBytecodeType, ExtendedBytecodeType;
	{ 
		FMemoryReaderView Ar(CreateShaderDesc.Code, true);
		Ar.SetLimitSize(ShaderCodeReader.GetActualShaderCodeSize());
		Ar << OfflineCompiledFlag;
		check(OfflineCompiledFlag);
		
		Header.Serialize(Ar, SRT);

		check(Header.Frequency == InFrequency);

		DefaultBytecodeType = (IRShaderStage)Header.RayTracing.DefaultBytecodeStage;
		ExtendedBytecodeType = (IRShaderStage)Header.RayTracing.ExtendedBytecodeStage;
		check(DefaultBytecodeType || ExtendedBytecodeType);

		EntryPoint = Header.RayTracing.EntryPoint;
		AnyHitEntryPoint = Header.RayTracing.AnyHitEntryPoint;
		IntersectionEntryPoint = Header.RayTracing.IntersectionEntryPoint;

		UE::RHICore::InitStaticUniformBufferSlots(this);

		GlobalRootParams = Header.RayTracing.GlobalRootParams;
		LocalRootParams = Header.RayTracing.LocalRootParams;
	}

	constexpr bool bCompileRayTracingShadersAsBinary = false;

	switch (InFrequency)
	{
		case SF_RayGen:
		{
			check(DefaultBytecodeType == IRShaderStageRayGeneration);
			// This shouldn't need anything special?
			RayGen = new FMetalRayGenShader(Device, CreateShaderDesc, InLibrary, DefaultBytecodeType, EntryPoint, bCompileRayTracingShadersAsBinary);
			break;
		}
		case SF_RayMiss:
		{
			check(DefaultBytecodeType == IRShaderStageMiss);
			// This also doesn't need anything special
			Miss = new FMetalRayMissShader(Device, CreateShaderDesc, InLibrary, DefaultBytecodeType, EntryPoint, bCompileRayTracingShadersAsBinary);
			break;
		}
		case SF_RayCallable:
		{
			check(DefaultBytecodeType == IRShaderStageCallable);
			// This also doesn't need anything special
			Callable = new FMetalRayCallableShader(Device, CreateShaderDesc, InLibrary, DefaultBytecodeType, EntryPoint, bCompileRayTracingShadersAsBinary);
			break;
		}
		case SF_RayHitGroup:
		{
			switch (DefaultBytecodeType)
			{
				case IRShaderStageClosestHit:
				{
					ClosestHit = new FMetalHitGroupShader(Device, CreateShaderDesc, InLibrary, DefaultBytecodeType, EntryPoint, bCompileRayTracingShadersAsBinary, false);
					break;
				}
				case IRShaderStageAnyHit:
				{
					AnyHitAndIntersection = new FMetalHitGroupShader(Device, CreateShaderDesc, InLibrary, DefaultBytecodeType, AnyHitEntryPoint, bCompileRayTracingShadersAsBinary, false);
					break;
				}
				case IRShaderStageIntersection:
				{
					AnyHitAndIntersection = new FMetalHitGroupShader(Device, CreateShaderDesc, InLibrary, DefaultBytecodeType, IntersectionEntryPoint, bCompileRayTracingShadersAsBinary, false);
					break;
				}
				default:
					checkf(false, TEXT("Invalid default bytecode type %u"), (uint32)DefaultBytecodeType);
					break;
			}

			switch (ExtendedBytecodeType)
			{
				case IRShaderStageAnyHit:
				{
					check(!AnyHitAndIntersection);
					AnyHitAndIntersection = new FMetalHitGroupShader(Device, CreateShaderDesc, InLibrary, ExtendedBytecodeType, AnyHitEntryPoint, bCompileRayTracingShadersAsBinary, true);
					break;
				}
				case IRShaderStageIntersection:
				{
					check(!AnyHitAndIntersection);
					AnyHitAndIntersection = new FMetalHitGroupShader(Device, CreateShaderDesc, InLibrary, ExtendedBytecodeType, IntersectionEntryPoint, bCompileRayTracingShadersAsBinary, true);
					break;
				}
				case IRShaderStageInvalid:
				{
					break;
				}
				default:
					checkf(false, TEXT("Invalid extended bytecode type %u"), (uint32)ExtendedBytecodeType);
					break;
			}

			break;
		}
		default:
		{
			checkf(false, TEXT("Bad raytracing shader frequency %u"), InFrequency);
			break;
		}
	}

	RayTracingPayloadType = Header.RayTracing.PayloadType;
	RayTracingPayloadSize = Header.RayTracing.PayloadSize;

	checkf(RayTracingPayloadType != 0, TEXT("Ray Tracing Shader must not have an empty payload type!"));
	checkf(RayTracingPayloadType != 0xFAFAFAFA, TEXT("Ray Tracing Shader did not set payload type"));
	checkf(	(FMath::CountBits(RayTracingPayloadType) == 1 && (InFrequency == SF_RayHitGroup || InFrequency == SF_RayMiss || InFrequency == SF_RayCallable)) ||
			(FMath::CountBits(RayTracingPayloadType) >= 1 && (InFrequency == SF_RayGen)),
			TEXT("Ray Tracing Shader has %d bits set, which is not the expected count for shader frequency %d"), FMath::CountBits(RayTracingPayloadType), int(InFrequency)
	);

	
	GlobalRootSignatureStartIdx = FRHIShaderBindingLayout::MaxUniformBufferEntries;
	
	GlobalRootSignature = CreateRootSignature(GlobalRootParams, false);
	GlobalRootSignatureSize = CalculateRootSignatureSize(GlobalRootSignature);
	LocalRootSignature = CreateRootSignature(LocalRootParams, true);
	LocalBindingDataSize = LocalRootSignatureSize = CalculateRootSignatureSize(LocalRootSignature);
}

FMetalRayShader::~FMetalRayShader()
{
	if (GlobalRootSignature)
	{
		IRRootSignatureDestroy(GlobalRootSignature);
	}
	if (LocalRootSignature)
	{
		IRRootSignatureDestroy(LocalRootSignature);
	}
}

void FMetalRayShader::ReleaseMetalObjects()
{	
	if (RayGen)       { RayGen->ReleaseMetalObjects(); }
	if (Miss)         { Miss->ReleaseMetalObjects(); }
	if (Callable)     { Callable->ReleaseMetalObjects(); }
	if (ClosestHit)   { ClosestHit->ReleaseMetalObjects(); }
	if (AnyHitAndIntersection) { AnyHitAndIntersection->ReleaseMetalObjects(); }
}

IRRootSignature *FMetalRayShader::CreateRootSignature(const TArray<IRRootParameter1> &InRootParams, bool bIsLocal)
{
	TArray<IRRootParameter1> RootParams;
	
	if(!bIsLocal)
	{
		// Adding static shader bindings
		for(uint32_t RegisterIdx = 0; RegisterIdx < FRHIShaderBindingLayout::MaxUniformBufferEntries; RegisterIdx++)
		{
			IRRootParameter1 RootParam;
			RootParam.ParameterType = IRRootParameterTypeCBV;
			RootParam.ShaderVisibility = IRShaderVisibility::IRShaderVisibilityAll;
			RootParam.Descriptor.ShaderRegister = RegisterIdx;
			RootParam.Descriptor.RegisterSpace = UE_HLSL_SPACE_STATIC_SHADER_BINDINGS;
			RootParam.Descriptor.Flags = IRRootDescriptorFlagDataStaticWhileSetAtExecute;
			
			RootParams.Add(RootParam);
		}
	}
	RootParams.Append(InRootParams);
	
	IRVersionedRootSignatureDescriptor RootSignatureDescriptor;
	// Create the global root signature for air generation.
	RootSignatureDescriptor.version = IRRootSignatureVersion_1_1;
	RootSignatureDescriptor.desc_1_1.Flags = bIsLocal ? IRRootSignatureFlagLocalRootSignature : IRRootSignatureFlagNone;
	RootSignatureDescriptor.desc_1_1.pStaticSamplers = StaticSamplerDescs;
	RootSignatureDescriptor.desc_1_1.NumStaticSamplers = UE_ARRAY_COUNT(StaticSamplerDescs);
	// Knowingly discarding const here but doesn't matter
	RootSignatureDescriptor.desc_1_1.pParameters = (IRRootParameter1 *)(RootParams.GetData());
	RootSignatureDescriptor.desc_1_1.NumParameters = RootParams.Num();

	IRError* RootSignatureCreationError = nullptr;
	IRRootSignature* RootSignature = IRRootSignatureCreateFromDescriptor(&RootSignatureDescriptor, &RootSignatureCreationError);
	check(RootSignature && !RootSignatureCreationError);
	return RootSignature;
}

uint32 FMetalRayShader::CalculateRootSignatureSize(IRRootSignature* RootSignature)
{
	size_t ResourceCount = IRRootSignatureGetResourceCount(RootSignature);
	if (ResourceCount == 0)
	{
		return 0;
	}

	IRResourceLocation* ResourceLocations = new IRResourceLocation[ResourceCount];
	memset(ResourceLocations, 0xFF, ResourceCount * sizeof(IRResourceLocation));
	IRRootSignatureGetResourceLocations(RootSignature, ResourceLocations);

	uint32 RootSignatureSize = 0;
	for (uint32 i = 0; i < ResourceCount; ++i)
	{
		RootSignatureSize += ResourceLocations[i].sizeBytes;
	}

	delete[] ResourceLocations;
	return RootSignatureSize;
}

#endif
