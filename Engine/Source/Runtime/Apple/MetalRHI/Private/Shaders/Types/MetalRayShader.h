// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRayShader.h: Metal RHI Ray Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"

#if METAL_RHI_RAYTRACING

#include "Shaders/Types/Templates/MetalBaseShader.h"
#include "RHIResources.h"
#include "ShaderCore.h"

template<typename BaseResourceType, int32 ShaderType>
class FMetalBaseRayShader : public TMetalBaseShader<BaseResourceType, ShaderType>
{
public:
	FMetalBaseRayShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary, IRShaderStage InShaderStage, FString& InEntryPoint, bool bInCompileAsBinary, bool bUseExtendedCodeSection = false)
				: TMetalBaseShader<BaseResourceType, ShaderType>(MetalDevice),
				bCompileAsBinary(bInCompileAsBinary),
				ShaderStage(InShaderStage),
				EntryPoint(InEntryPoint)
	{
		FMetalCodeHeader Header;
		TMetalBaseShader<BaseResourceType, ShaderType>::Init(CreateShaderDesc, Header, InLibrary, bCompileAsBinary, bUseExtendedCodeSection);
	}
	
	IRShaderStage GetShaderStage()
	{
		return ShaderStage;
	}
	
	MTLFunctionPtr GetFunction()
	{  
		return TMetalBaseShader<BaseResourceType, ShaderType>::GetCompiledFunction(false, -1, bCompileAsBinary);
	}
	
	virtual FString GetFunctionName() override
	{
		return EntryPoint;
	}
	
	bool bCompileAsBinary = false;
	IRShaderStage ShaderStage;
	FString EntryPoint;
};

class FMetalRayShader : public FRHIRayTracingShader
{
public:
	UE_NONCOPYABLE(FMetalRayShader)

	FMetalRayShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, EShaderFrequency InFrequency, MTLLibraryPtr InLibrary = nullptr);
	virtual ~FMetalRayShader();

	// Releases MTLFunction and MTLLibrary references on all sub-shaders.
	void ReleaseMetalObjects();

	inline IRRootSignature *GetGlobalRootSignature() const { return GlobalRootSignature; }
	inline IRRootSignature *GetLocalRootSignature() const { return LocalRootSignature; }

	const inline TArray<IRRootParameter1>& GetGlobalRootParams() const { return GlobalRootParams; }
	const inline TArray<IRRootParameter1>& GetLocalRootParams() const { return LocalRootParams; }

	inline uint32 GetGlobalRootSignatureSize() const { return GlobalRootSignatureSize; }
	inline uint32 GetLocalRootSignatureSize() const { return LocalRootSignatureSize; }

	// These are the original names when the shaders were compiled, not necessarily the final name generated by IRConverter
	FString EntryPoint; // Primary entry point for all ray tracing shaders. Assumed to be closest hit shader for SF_RayHitGroup.
	FString AnyHitEntryPoint; // Optional any-hit shader entry point for SF_RayHitGroup.
	FString IntersectionEntryPoint; // Optional intersection shader entry point for SF_RayHitGroup.

	/* Uniform buffer static slots */
	TArray<FUniformBufferStaticSlot> StaticSlots;

	static constexpr bool bBinaryCompile = true;
	
	inline MTLFunctionPtr GetRayGenFunction() { return RayGen->GetFunction(); }
	inline MTLFunctionPtr GetMissFunction() { return Miss->GetFunction(); }
	inline MTLFunctionPtr GetCallableFunction() { return Callable->GetFunction(); }
	inline MTLFunctionPtr GetClosestHitFunction() { return ClosestHit->GetFunction(); }
	inline MTLFunctionPtr GetAnyHitAndIntersectionFunction() { return AnyHitAndIntersection->GetFunction(); }

	inline MTLFunctionPtr GetEntryPointFunction()
	{
		switch (Frequency)
		{
			case SF_RayGen: return GetRayGenFunction();
			case SF_RayMiss: return GetMissFunction();
			case SF_RayCallable: return GetCallableFunction();
			case SF_RayHitGroup: return GetClosestHitFunction();
		}
		return nullptr;
	}

	inline EShaderFrequency GetShaderFrequency() const { return Frequency; }

	inline bool HasVisibleFunctionEntryPoint() const { return RayGen || Miss || Callable || ClosestHit; }
	inline bool HasIntersectionFunctionEntryPoint() const { return AnyHitAndIntersection; }

	inline uint32 GetGlobalRootSignatureStartIdx() const { return GlobalRootSignatureStartIdx; }
	
private:
	static IRRootSignature *CreateRootSignature(const TArray<IRRootParameter1> &RootParams, bool bIsLocal);
	static uint32 CalculateRootSignatureSize(IRRootSignature* RootSignature);

	IRRootSignature *GlobalRootSignature = nullptr;
	IRRootSignature *LocalRootSignature = nullptr;

	uint32 GlobalRootSignatureSize = 0, LocalRootSignatureSize = 0;

	TArray<IRRootParameter1> GlobalRootParams;
	TArray<IRRootParameter1> LocalRootParams;

	uint32 GlobalRootSignatureStartIdx = 0;
	
	EShaderFrequency Frequency;

	using FMetalRayGenShader = FMetalBaseRayShader<FRHIRayGenShader, SF_RayGen>;
	using FMetalRayMissShader = FMetalBaseRayShader<FRHIRayMissShader, SF_RayMiss>;
	using FMetalRayCallableShader = FMetalBaseRayShader<FRHIRayCallableShader, SF_RayCallable>;
	using FMetalHitGroupShader = FMetalBaseRayShader<FRHIRayHitGroupShader, SF_RayHitGroup>;

	TRefCountPtr<FMetalRayGenShader> RayGen = nullptr;
	TRefCountPtr<FMetalRayMissShader> Miss = nullptr;
	TRefCountPtr<FMetalRayCallableShader> Callable = nullptr;
	TRefCountPtr<FMetalHitGroupShader> ClosestHit = nullptr;
	TRefCountPtr<FMetalHitGroupShader> AnyHitAndIntersection = nullptr;
};

#endif
