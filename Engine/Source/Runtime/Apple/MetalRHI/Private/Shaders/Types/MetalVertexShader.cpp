// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalVertexShader.cpp: Metal RHI Vertex Shader Class Implementation.
=============================================================================*/

#include "MetalVertexShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Vertex Shader Class


FMetalVertexShader::FMetalVertexShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc)
	: TMetalBaseShader<FRHIVertexShader, SF_Vertex>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, MTLLibraryPtr());
}

FMetalVertexShader::FMetalVertexShader(FMetalDevice& MetalDevice, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary)
	: TMetalBaseShader<FRHIVertexShader, SF_Vertex>(MetalDevice)
{
	FMetalCodeHeader Header;
	Init(CreateShaderDesc, Header, InLibrary);
}

MTLFunctionPtr FMetalVertexShader::GetFunction()
{
	return GetCompiledFunction();
}

#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
MTLFunctionPtr FMetalVertexShader::GetObjectFunctionForGeometryEmulation()
{
    return GetCompiledFunction(false, 0);
}
#endif
