// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalVertexShader.h: Metal RHI Vertex Shader Class Definition.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Shaders/Types/Templates/MetalBaseShader.h"


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Vertex Shader Class


class FMetalVertexShader : public TMetalBaseShader<FRHIVertexShader, SF_Vertex>
{
public:
	FMetalVertexShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc);
	FMetalVertexShader(FMetalDevice& Device, const FRHICreateShaderDesc& CreateShaderDesc, MTLLibraryPtr InLibrary);

	MTLFunctionPtr GetFunction();
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
    MTLFunctionPtr GetObjectFunctionForGeometryEmulation();
#endif
};
