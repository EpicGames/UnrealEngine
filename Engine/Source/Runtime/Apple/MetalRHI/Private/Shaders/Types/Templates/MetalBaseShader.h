// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalBaseShader.h: Metal RHI Base Shader Class Template.
=============================================================================*/

#pragma once

#include "Apple/ScopeAutoreleasePool.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "MetalCommandQueue.h"
#include "MetalDevice.h"
#include "MetalProfiler.h"
#include "MetalShaderResources.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeExit.h"
#include "Serialization/MemoryReader.h"
#include "Shaders/Debugging/MetalShaderDebugCache.h"
#include "Shaders/MetalCompiledShaderKey.h"
#include "Shaders/MetalCompiledShaderCache.h"
#include "RHICoreShader.h"

//------------------------------------------------------------------------------

#pragma mark - Metal RHI Base Shader Class Support Routines


extern MTL::LanguageVersion ValidateVersion(uint32 Version);


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Base Shader Class Template Defines


/** Set to 1 to enable shader debugging (makes the driver save the shader source) */
#define DEBUG_METAL_SHADERS (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Base Shader Class Template

struct FMetalShaderData
{
    /** External bindings for this shader. */
    FMetalShaderBindings Bindings;
    
    /* Argument encoders for shader IABs */
    TMap<uint32, MTL::ArgumentEncoder*> ArgumentEncoders;

    /* Tier1 Argument buffer bitmasks */
    TMap<uint32, TBitArray<>> ArgumentBitmasks;

    /* Uniform buffer static slots */
    TArray<FUniformBufferStaticSlot> StaticSlots;

    /** The binding for the buffer side-table if present */
    int32 SideTableBinding = -1;

    /** CRC & Len for name disambiguation */
    uint32 SourceLen = 0;
    uint32 SourceCRC = 0;

    /** Unique identifier for the owning FRHIShaderLibrary, used to scope the compiled shader cache per library. 0 if no library. */
    uint64 LibrarySeed = 0;
    
    // this is the compiler shader
    MTLFunctionPtr Function;
    // This is the MTLLibrary for the shader so we can dynamically refine the MTLFunction
    MTLLibraryPtr Library;

    /** The debuggable text source */
    NS::String* MetalCodeNSString = nullptr;

    /** Index of the function (in the library) pointing to the function requested by the user (when GetCompiledFunction() is called with an explicit index). */
    uint32 LibraryFunctionIndex = -1;
};

template<typename BaseResourceType, int32 ShaderType>
class TMetalBaseShader : public BaseResourceType, public FMetalShaderData
{
public:
	enum
	{
		StaticFrequency = ShaderType
	};

	TMetalBaseShader(FMetalDevice& MetalDevice) : Device(MetalDevice)
	{
		// void
	}

	virtual ~TMetalBaseShader()
	{
		this->Destroy();
	}

	void Init(const FRHICreateShaderDesc& CreateShaderDesc, FMetalCodeHeader& Header, MTLLibraryPtr InLibrary, bool bBinaryCompiled = false, bool bUseExtendedCodeSection = false);
	void Destroy();

	/** Releases MTLFunction and MTLLibrary references. Called during shader library teardown to free Metal objects
	 *  while the FRHIShader may still be alive (held by materials, PSO cache, etc.). */
	virtual void ReleaseMetalObjects()
	{
		FMetalCompiledShaderKey Key(SourceLen, SourceCRC, bIsExtendedBytecode, LibrarySeed);
		GetMetalCompiledShaderCache().Remove(Key);
		Library.reset();
		Function.reset();
	}

	virtual FString GetFunctionName()
	{
		return FString::Printf(TEXT("Main_%0.8x_%0.8x"), SourceLen, SourceCRC);
	}
	
	/**
	 * Gets the Metal source code as an NSString if available or build a string with the hash & function name inside instead.
	 */
	NS::String* GetSourceCode();

protected:
	bool bIsExtendedBytecode = 0;
	
	FMetalDevice& Device;
	MTLFunctionPtr GetCompiledFunction(bool const bAsync = false, const int32 FunctionIndex = -1, bool bCompileToBinary = false);
};


//------------------------------------------------------------------------------

#pragma mark - Metal RHI Base Shader Class Template Member Functions


template<typename BaseResourceType, int32 ShaderType>
void TMetalBaseShader<BaseResourceType, ShaderType>::Init(const FRHICreateShaderDesc& CreateShaderDesc, FMetalCodeHeader& Header, MTLLibraryPtr InLibrary, bool bBinaryCompiled, bool bUseExtendedCodeSection)
{
	FShaderCodeReader ShaderCode(CreateShaderDesc.Code);

	FMemoryReaderView Ar(CreateShaderDesc.Code, true);

	Ar.SetLimitSize(ShaderCode.GetActualShaderCodeSize());

	// was the shader already compiled offline?
	uint8 OfflineCompiledFlag;
	Ar << OfflineCompiledFlag;
	check(OfflineCompiledFlag == 0 || OfflineCompiledFlag == 1);

	// get the header
	Header.Serialize(Ar, BaseResourceType::ShaderResourceTable);

	ValidateVersion(Header.Version);

	SourceLen = Header.SourceLen;
	SourceCRC = Header.SourceCRC;

	// If this triggers than a level above us has failed to provide valid shader data and the cook is probably bogus
	UE_CLOGF(Header.SourceLen == 0 || Header.SourceCRC == 0, LogMetal, Fatal, "Invalid Shader Bytecode provided.");

	// remember where the header ended and code (precompiled or source) begins
	int32 CodeOffset = Ar.Tell();
	uint32 BufferSize = ShaderCode.GetActualShaderCodeSize() - CodeOffset;
	const ANSICHAR* SourceCode = (ANSICHAR*)CreateShaderDesc.Code.GetData() + CodeOffset;

	if (bUseExtendedCodeSection)
	{
		int32 ExtendedCodeDataOffset = Header.RayTracing.ExtendedCodeDataOffset;
		int32 ExtendedCodeDataSize = Header.RayTracing.ExtendedCodeDataSize;

		ANSICHAR const* SourceCodeExtended = nullptr;
		if (ExtendedCodeDataOffset != -1 && ExtendedCodeDataSize != -1)
		{
			int32 ExtendedCodeStartOffset = CodeOffset + ExtendedCodeDataOffset;
			SourceCodeExtended = (ANSICHAR*)CreateShaderDesc.Code.GetData() + ExtendedCodeStartOffset;	
		}
		else
		{
			SourceCodeExtended = (ANSICHAR*)ShaderCode.FindOptionalDataAndSize(EShaderOptionalDataKey::VendorExtension, ExtendedCodeDataSize);
		}
		
		check(SourceCodeExtended);
		check(ExtendedCodeDataSize != 0);
		
		BufferSize = (uint32)ExtendedCodeDataSize;
		SourceCode = SourceCodeExtended;
		
		bIsExtendedBytecode = true;
	}

	// Only archived shaders should be in here.
	UE_CLOG(InLibrary && !(Header.CompileFlags & (1 << CFLAG_Archive)), LogMetal, Warning, TEXT("Shader being loaded wasn't marked for archiving but a MTLLibrary was provided - this is unsupported."));

	if (!OfflineCompiledFlag)
	{
		UE_LOGF(LogMetal, Display, "Loaded a text shader (will be slower to load)");
	}

	const bool bOfflineCompile = (OfflineCompiledFlag > 0);
	const bool bUseMetalShaderConverter = EnumHasAnyFlags(Header.Bindings.Flags, EMetalBindingsFlags::UseMetalShaderConverter); 
	
	const ANSICHAR* ShaderSource = ShaderCode.FindOptionalData(EShaderOptionalDataKey::SourceCode);
	bool bHasShaderSource = false;
	static bool bForceTextShaders = false;
	
	if(!bUseMetalShaderConverter)
	{
		bForceTextShaders = FParse::Param(FCommandLine::Get(),TEXT("metalshaderdebug"));
		bHasShaderSource = (ShaderSource && FCStringAnsi::Strlen(ShaderSource) > 0);
	}
	
	if (!bHasShaderSource)
	{
#if !UE_BUILD_SHIPPING
		if(bForceTextShaders)
		{
			MetalCodeNSString = FMetalShaderDebugCache::Get().GetShaderCode(SourceLen, SourceCRC);
			if(MetalCodeNSString)
			{
				MetalCodeNSString->retain();
			}
			else
			{
				UE_LOGF(LogMetal, Warning, "-metalshaderdebug was used but shader source are unavailable, please use r.Shaders.ExtraData=1 or enable AllowsRuntimeShaderCompiling for the platform and add CFLAG_Debug to this shader before cooking.");
			}
		}
#endif
	}
	else if (bOfflineCompile && bHasShaderSource)
	{
		MetalCodeNSString = NS::String::string(ShaderSource, NS::UTF8StringEncoding);
		check(MetalCodeNSString);
		MetalCodeNSString->retain();
	}

	Library = InLibrary;

	LibrarySeed = reinterpret_cast<uint64>(CreateShaderDesc.Library);

	bool bNeedsCompiling = false;

	FMetalCompiledShaderKey Key(Header.SourceLen, Header.SourceCRC, bIsExtendedBytecode, LibrarySeed);

	FCachedCompiledShader CachedShader = GetMetalCompiledShaderCache().Find(Key);
	Function = CachedShader.Function;
	if (!Library && Function)
	{
		Library = CachedShader.Library;
	}
	else
	{
		bNeedsCompiling = true;
	}

	Bindings = Header.Bindings;
	if (bNeedsCompiling || !Library)
	{
		if (bOfflineCompile METAL_DEBUG_OPTION(&& !(bHasShaderSource && bForceTextShaders)))
		{
			if (InLibrary)
			{
				Library = InLibrary;
			}
			else
			{
				// Archived shaders should never get in here.
				check(!(Header.CompileFlags & (1 << CFLAG_Archive)) || BufferSize > 0);
				dispatch_data_t GCDBuffer;
				if (CreateShaderDesc.CodeOwner.IsValid())
				{
					// Hold a reference until the MTLLibrary is done using it
					__block auto OwnerCopy = CreateShaderDesc.CodeOwner;
					GCDBuffer = dispatch_data_create(SourceCode, BufferSize, nil, ^{
						OwnerCopy.Reset();
					});

					// Append seed data after the metallib content to prevent Metal's
					// newLibrary cache from returning a cached library. Metal's cache key appears
					// to be based on the full dispatch_data bytes, so appending unique data
					// ensures each library is treated as distinct.
					if (LibrarySeed != 0)
					{
						uint64* SeedData = new uint64;
						*SeedData = LibrarySeed;
						dispatch_data_t SeedBuffer = dispatch_data_create(SeedData, sizeof(uint64), nil, ^{ delete SeedData; });
						dispatch_data_t CombinedBuffer = dispatch_data_create_concat(GCDBuffer, SeedBuffer);
						dispatch_release(GCDBuffer);
						dispatch_release(SeedBuffer);
						GCDBuffer = CombinedBuffer;
					}
				}
				else
				{
					// Create a copy of the memory
					GCDBuffer = dispatch_data_create(SourceCode, BufferSize, nil, DISPATCH_DATA_DESTRUCTOR_DEFAULT);
				}

				// load up the already compiled shader
				NS::Error* AError;
				Library = NS::TransferPtr(Device.GetDevice()->newLibrary(GCDBuffer, &AError));
				dispatch_release(GCDBuffer);

				if (!Library)
				{
                    UE_LOGF(LogMetal, Display, "Failed to create library: %ls", *NSStringToFString(AError->description()));
				}
			}
		}
		else
		{
			MTL_SCOPED_AUTORELEASE_POOL;

			NS::String* ShaderString = ((OfflineCompiledFlag == 0) ? NS::String::string(SourceCode, NS::UTF8StringEncoding) : MetalCodeNSString);

            FString FinalShaderString;
			const FString ShaderName = ShaderCode.FindOptionalData(FShaderCodeName::Key);
			if(ShaderName.Len())
			{
                FinalShaderString = FString::Printf(TEXT("// %s\n"), *ShaderName);
			}

            FinalShaderString += NSStringToFString(ShaderString);
            FinalShaderString.Replace(TEXT("#pragma once"), TEXT(""));
            NS::String* NewShaderString = FStringToNSString(FinalShaderString);

            MTL::CompileOptions* CompileOptions = MTL::CompileOptions::alloc()->init();
            check(CompileOptions);

#if DEBUG_METAL_SHADERS
			static bool bForceFastMath = FParse::Param(FCommandLine::Get(), TEXT("metalfastmath"));
			static bool bForceNoFastMath = FParse::Param(FCommandLine::Get(), TEXT("metalnofastmath"));
			if (bForceNoFastMath)
			{
				CompileOptions->setFastMathEnabled(NO);
			}
			else if (bForceFastMath)
			{
				CompileOptions->setFastMathEnabled(YES);
			}
			else
#endif
			{
				CompileOptions->setFastMathEnabled((BOOL)(!(Header.CompileFlags & (1 << CFLAG_NoFastMath))));
			}

#if !PLATFORM_MAC || DEBUG_METAL_SHADERS
            NS::Dictionary* PreprocessorMacros = nullptr;;
#if !PLATFORM_MAC // Pretty sure that as_type-casts work on macOS, but they don't for half2<->uint on older versions of the iOS runtime compiler.
			PreprocessorMacros = NS::Dictionary::dictionary(NS::String::string("1", NS::UTF8StringEncoding),
                                                            NS::String::string("METAL_RUNTIME_COMPILER", NS::UTF8StringEncoding));
#endif
#if DEBUG_METAL_SHADERS
            PreprocessorMacros = NS::Dictionary::dictionary(NS::String::string("1", NS::UTF8StringEncoding),
                                                            NS::String::string("MTLSL_ENABLE_DEBUG_INFO", NS::UTF8StringEncoding));
#endif
            if(PreprocessorMacros)
            {
                CompileOptions->setPreprocessorMacros(PreprocessorMacros);
                PreprocessorMacros->release();
            }
#endif

			MTL::LanguageVersion MetalVersion;
			switch (Header.Version)
			{
                case 8:
                    MetalVersion = MTL::LanguageVersion3_0;
                    break;
				case 7:
					MetalVersion = MTL::LanguageVersion2_4;
					break;
#if PLATFORM_MAC
				case 6:
					MetalVersion = MTL::LanguageVersion2_3;
					break;
				case 5:
					// Fall through
				case 0:
					// Fall through
				default:
					MetalVersion = MTL::LanguageVersion2_2;
					break;
#else
				case 0:
                    MetalVersion = MTL::LanguageVersion2_4;
                    break;
				case 9:
					MetalVersion = MTL::LanguageVersion3_1;
					break;
				default:
					UE_LOGF(LogRHI, Fatal, "Failed to create shader with unknown version %d: %ls", Header.Version, *NSStringToFString(NewShaderString));
					MetalVersion = MTL::LanguageVersion2_4;
					break;
#endif
			}
			CompileOptions->setLanguageVersion(MetalVersion);

			if(ShaderType == SF_Vertex && MetalVersion > MTL::LanguageVersion2_2)
			{
				CompileOptions->setPreserveInvariance(YES);
			}

			NS::Error* Error = nullptr;
			Library = NS::TransferPtr(Device.GetDevice()->newLibrary(NewShaderString, CompileOptions, &Error));
			if (Library.get() == nullptr)
			{
				UE_LOGF(LogRHI, Error, "*********** Error\n%ls", *NSStringToFString(NewShaderString));
				UE_LOGF(LogRHI, Fatal, "Failed to create shader: %ls", *NSStringToFString(Error->description()));
			}
			else if (Error != nullptr)
			{
				// Warning...
				UE_LOGF(LogRHI, Warning, "*********** Warning\n%ls", *NSStringToFString(NewShaderString));
				UE_LOGF(LogRHI, Warning, "Created shader with warnings: %ls", *NSStringToFString(Error->description()));
			}

			MetalCodeNSString = NewShaderString;
			MetalCodeNSString->retain();
		}

		GetCompiledFunction(true, -1, bBinaryCompiled);
	}
	SideTableBinding = Header.SideTable;

	UE::RHICore::InitStaticUniformBufferSlots(this);

#if RHI_INCLUDE_SHADER_DEBUG_DATA
    this->Debug.ShaderName = GetFunctionName();
#endif
}

template<typename BaseResourceType, int32 ShaderType>
void TMetalBaseShader<BaseResourceType, ShaderType>::Destroy()
{
    if(MetalCodeNSString)
    {
        MetalCodeNSString->release();
        MetalCodeNSString = nullptr;
    }
}

template<typename BaseResourceType, int32 ShaderType>
inline NS::String* TMetalBaseShader<BaseResourceType, ShaderType>::GetSourceCode()
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	if (!MetalCodeNSString)
	{
        FString ShaderString = FString::Printf(TEXT("Hash: %s, Name: %s"), *BaseResourceType::GetHash().ToString(), *GetFunctionName());
        MetalCodeNSString = FStringToNSString(ShaderString);
        MetalCodeNSString->retain();
	}
	return MetalCodeNSString;
}

template<typename BaseResourceType, int32 ShaderType>
MTLFunctionPtr TMetalBaseShader<BaseResourceType, ShaderType>::GetCompiledFunction(bool const bAsync, const int32 FunctionIndex, bool bCompileToBinary)
{
	MTL_SCOPED_AUTORELEASE_POOL;
	
	MTLFunctionPtr Func = Function;

	bool bNeedToRecreateFunction = (LibraryFunctionIndex != FunctionIndex);
    if (!Func || bNeedToRecreateFunction)
	{
		// Find the existing compiled shader in the cache.
		FMetalCompiledShaderKey Key(SourceLen, SourceCRC, bIsExtendedBytecode, LibrarySeed);
		Func = Function = GetMetalCompiledShaderCache().Find(Key).Function;

        if (bNeedToRecreateFunction)
        {
            Function = MTLFunctionPtr();
            Func = MTLFunctionPtr();
            LibraryFunctionIndex = FunctionIndex;
        }

		if (!Func)
		{
            NS::String* Name = (LibraryFunctionIndex != -1) ? (NS::String*)Library->functionNames()->object(LibraryFunctionIndex) : FStringToNSString(GetFunctionName());

			MTL::FunctionDescriptor* Descriptor = MTL::FunctionDescriptor::alloc()->init();
			Descriptor->setName(Name);
			if(bCompileToBinary)
			{
				Descriptor->setOptions(MTLFunctionOptionCompileToBinary);
			}
			
			if (!bAsync)
			{
				NS::Error* Error;
				Function = NS::TransferPtr(Library->newFunction(Descriptor, &Error));
				UE_CLOGF(Function.get() == nullptr, LogMetal, Error, "Failed to create function: %ls", *NSStringToFString(Error->description()));
				UE_CLOGF(Function.get() == nullptr, LogMetal, Fatal, "*********** Error\n%ls", *NSStringToFString(GetSourceCode()));

				check(Function);
				GetMetalCompiledShaderCache().Add(Key, Library, Function);

				Func = Function;
				
				Descriptor->release();
			}
			else
			{
				const std::function<void(MTL::Function* pFunction, NS::Error* pError)> Handler = [Key, this](MTL::Function* NewFunction, NS::Error* Error)
				{
					UE_CLOGF(NewFunction == nullptr, LogMetal, Error, "Failed to create function: %ls", *NSStringToFString(Error->description()));
					UE_CLOGF(NewFunction == nullptr, LogMetal, Fatal, "*********** Error\n%ls", *NSStringToFString(GetSourceCode()));

					GetMetalCompiledShaderCache().Add(Key, Library, NS::RetainPtr(NewFunction));
				};
                    
				NS::Array* FunctionNames = Library->functionNames();
				check(FunctionNames->count());
					
                Library->newFunction(Descriptor, Handler);

				Descriptor->release();
				
				return MTLFunctionPtr();
			}
		}
	}

	check(Func);
	return Func;
}
