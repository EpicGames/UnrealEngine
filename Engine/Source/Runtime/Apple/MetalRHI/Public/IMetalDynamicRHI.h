// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "RHI.h"
#include "MetalThirdParty.h"

struct IMetalDynamicRHI : public FDynamicRHI
{
	virtual ERHIInterfaceType GetInterfaceType() const override final
	{
		return ERHIInterfaceType::Metal;
	}
	
	virtual MTL::Device*      RHIGetDevice() = 0;

	virtual void 			  RHIRunOnQueue(TFunction<void(MTL::CommandQueue*)>&& CodeToRun, bool bWaitForSubmission) = 0;
	virtual FTextureRHIRef	  RHICreateTexture2DFromCVMetalTexture(EPixelFormat Format, ETextureCreateFlags TexCreateFlags, const FClearValueBinding& ClearValueBinding, CVMetalTextureRef Resource) = 0;

#if PLATFORM_IOS
	/** Used for getting the last back buffer content after a crash. */
	virtual bool RHIReadViewportDataDirect(uint32& OutWidth, uint32& OutHeight, TArray<FColor>& OutData) = 0;
#endif // PLATFORM_IOS
};

inline bool IsRHIMetal()
{
	return GDynamicRHI != nullptr && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::Metal;
}

inline IMetalDynamicRHI* GetIMetalDynamicRHI()
{
	check(GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::Metal);
	return GetDynamicRHI<IMetalDynamicRHI>();
}

