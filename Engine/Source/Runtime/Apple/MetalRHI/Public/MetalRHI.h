// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalRHI.h: Public Metal RHI definitions.
=============================================================================*/

#pragma once 

