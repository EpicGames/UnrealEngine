// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Metal RHI public headers.
#include "MetalRHIPrivate.h"
#include "MetalThirdParty.h"
#include "MetalState.h"
#include "MetalResources.h"
#include "MetalViewport.h"
#include "MetalDevice.h"
#include "MetalCommandEncoder.h"
#include "MetalRHIRenderQuery.h"
#include "MetalBindlessDescriptors.h"
#include "RHICore.h"

#if PLATFORM_VISIONOS
namespace MetalRHIVisionOS
{
    struct BeginRenderingImmersiveParams;
    struct PresentImmersiveParams;
}
#endif

struct FMetalParallelRenderPassInfo
{
	MTLParallelRenderCommandEncoderPtr ParallelEncoder;
	MTL::RenderPassDescriptor* RenderPassDesc = nullptr;
};

enum class EMetalFlushFlags
{
	None = 0,

	// Block the calling thread until the submission thread has dispatched all work.
	WaitForSubmission = 1,

	// Both the calling thread until the GPU has signaled completion of all dispatched work.
	WaitForCompletion = 2
};
ENUM_CLASS_FLAGS(EMetalFlushFlags)

/** The interface RHI command context. */
class FMetalRHICommandContext : public IRHICommandContext
{
public:
	FMetalRHICommandContext(FMetalDevice& Device);
	virtual ~FMetalRHICommandContext();
	
	static inline FMetalRHICommandContext& Get(FRHICommandListBase& CmdList)
	{
		check(CmdList.IsBottomOfPipe());
		return static_cast<FMetalRHICommandContext&>(CmdList.GetContext().GetLowestLevelContext());
	}
	
	void ResetContext();
	void BeginComputeEncoder();
	void EndComputeEncoder();
	void BeginBlitEncoder();
	void EndBlitEncoder();

	virtual void RHISetComputePipelineState(FRHIComputePipelineState* ComputePipelineState) override;
	
	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	
	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	
	// Useful when used with geometry shader (emit polygons to different viewports), otherwise SetViewPort() is simpler
	// @param Count >0
	// @param Data must not be 0
	virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override;
	
	/** Clears a UAV to the multi-component value provided. */
	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override;
	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override;
	
	virtual void RHICopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo) final override;
	virtual void RHICopyBufferRegion(FRHIBuffer* DstBufferRHI, uint64 DstOffset, FRHIBuffer* SrcBufferRHI, uint64 SrcOffset, uint64 NumBytes) final override;
    
	virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	
	void EndDrawingViewport(FMetalViewport* Viewport, const FRHIPresentArgs& InPresentArgs);

	virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override;

	// @param MinX including like Win32 RECT
	// @param MinY including like Win32 RECT
	// @param MaxX excluding like Win32 RECT
	// @param MaxY excluding like Win32 RECT
	virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override;

	virtual void RHISetStereoViewport(float LeftMinX, float RightMinX, float LeftMinY, float RightMinY, float MinZ, float LeftMaxX, float RightMaxX, float LeftMaxY, float RightMaxY, float MaxZ) final override;
	
	// @param MinX including like Win32 RECT
	// @param MinY including like Win32 RECT
	// @param MaxX excluding like Win32 RECT
	// @param MaxY excluding like Win32 RECT
	virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override;
	
	virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) final override;
	
	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override;
	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer) final override;

	virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;

	virtual void RHISetStencilRef(uint32 StencilRef) final override;

	virtual void RHISetBlendFactor(const FLinearColor& BlendFactor) final override;

	void SetRenderTargets(uint32 NumSimultaneousRenderTargets, const FRHIRenderTargetView* NewRenderTargets, const FRHIDepthRenderTargetView* NewDepthStencilTarget);
	
	void SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo);
	
	virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	
	virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	
	// @param NumPrimitives need to be >0
	virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	
	virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;

#if PLATFORM_SUPPORTS_MESH_SHADERS
    virtual void RHIDispatchMeshShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
    virtual void RHIDispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
#endif

	/**
	* Sets Depth Bounds Testing with the given min/max depth.
	* @param MinDepth	The minimum depth for depth bounds test
	* @param MaxDepth	The maximum depth for depth bounds test.
	*					The valid values for fMinDepth and fMaxDepth are such that 0 <= fMinDepth <= fMaxDepth <= 1
	*/
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override;

#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override;
	virtual void RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override;
#endif

	virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 Offset, uint32 NumBytes) final override;
	virtual void RHIWriteGPUFence(FRHIGPUFence* FenceRHI) final override;

	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions);
	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions);

	virtual void RHIBeginParallelRenderPass(TSharedPtr<FRHIParallelRenderPassInfo> InInfo, const TCHAR* InName) final override;
	virtual void RHIEndParallelRenderPass() final override;
	
	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) final override;
	virtual void RHIEndRenderPass() final override;
	
	virtual void RHINextSubpass() final override;

#if METAL_RHI_RAYTRACING
	virtual void RHIBindAccelerationStructureMemory(FRHIRayTracingScene* Scene, FRHIBuffer* Buffer, uint32 BufferOffset) final override;
	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingGeometryBuildParams> Params, const FRHIBufferRange& ScratchBufferRange) final override;
	virtual void RHIBuildAccelerationStructures(TConstArrayView<FRayTracingSceneBuildParams> Params) final override;
	virtual void RHIRayTraceDispatch(FRHIRayTracingPipelineState* RayTracingPipelineState, FRHIRayTracingShader* RayGenShader,
									 FRHIShaderBindingTable* SBT, const FRayTracingShaderBindings& GlobalResourceBindings,
									 uint32 Width, uint32 Height) final override;
	
	virtual void RHIRayTraceDispatchIndirect(FRHIRayTracingPipelineState* InRayTracingPipelineState, FRHIRayTracingShader* InRayGenShader,
											 FRHIShaderBindingTable* InSBT, const FRayTracingShaderBindings& GlobalResourceBindings,
											 FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	virtual void RHISetBindingsOnShaderBindingTable(
		FRHIShaderBindingTable* SBT, FRHIRayTracingPipelineState* Pipeline,
		uint32 NumBindings, const FRayTracingLocalShaderBindings* Bindings,
		ERayTracingBindingType BindingType) final override;
	
	void RHICommitShaderBindingTable(FRHIShaderBindingTable* InSBT, FRHIBuffer* InlineBindingDataBuffer);
	
	void WriteCompactedAccelerationStructureSize(MTLAccelerationStructurePtr AccelerationStructure, FMetalBufferPtr CompactedStructureSizeBuffer, uint32_t Offset);
	void CopyAndCompactAccelerationStructure(MTLAccelerationStructurePtr AccelerationStructureSrc, MTLAccelerationStructurePtr AccelerationStructureDest);
	
	void BuildAccelerationStructure(FMetalBufferPtr CurInstanceBuffer, uint32_t InstanceBufferOffset, FMetalBufferPtr ScratchBuffer, 
									uint32_t ScratchBufferOffset, FMetalBufferPtr HitGroupContributionsBuffer, uint32_t HitGroupContributionsBufferOffset, uint32_t MaxNumInstances, FMetalAccelerationStructure* AS);
	void BuildAccelerationStructure(MTLAccelerationStructurePtr AS, MTL::AccelerationStructureDescriptor* Descriptor,
									FMetalBufferPtr ScratchBuffer, uint32_t ScratchBufferOffset);
	
	void RefitAccelerationStructure(MTLAccelerationStructurePtr SrcBLAS, MTLAccelerationStructurePtr DstBLAS, MTL::PrimitiveAccelerationStructureDescriptor* Descriptor, MTL::Buffer* ScratchBufferRes, uint32_t ScratchOffset);
#endif // METAL_RHI_RAYTRACING

	void FillBuffer(MTL::Buffer* Buffer, NS::Range Range, uint8 Value);
	void CopyFromTextureToBuffer(MTL::Texture* Texture, uint32 sourceSlice, uint32 sourceLevel, MTL::Origin sourceOrigin, MTL::Size sourceSize, FMetalBufferPtr toBuffer, uint32 destinationOffset, uint32 destinationBytesPerRow, uint32 destinationBytesPerImage, MTL::BlitOption options);
	void CopyFromBufferToTexture(FMetalBufferPtr Buffer, uint32 sourceOffset, uint32 sourceBytesPerRow, uint32 sourceBytesPerImage, MTL::Size sourceSize, MTL::Texture* toTexture, uint32 destinationSlice, uint32 destinationLevel, MTL::Origin destinationOrigin, MTL::BlitOption options);
	void CopyFromTextureToTexture(MTL::Texture* Texture, uint32 sourceSlice, uint32 sourceLevel, MTL::Origin sourceOrigin, MTL::Size sourceSize, MTL::Texture* toTexture, uint32 destinationSlice, uint32 destinationLevel, MTL::Origin destinationOrigin);
	void CopyFromBufferToBuffer(FMetalBufferPtr SourceBuffer, NS::UInteger SourceOffset, FMetalBufferPtr DestinationBuffer, NS::UInteger DestinationOffset, NS::UInteger Size);
	
	void CommitRenderResourceTables(void);
	void PrepareToRender(uint32 PrimitiveType);
	bool PrepareToDraw(uint32 PrimitiveType);
	void PrepareToDispatch();
	void SetupParallelContext(const FRHIParallelRenderPassInfo* ParallelRenderPassInfo);
	
	void Finalize(TArray<FMetalPayload*>& OutPayloads);
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void EnqueueDescriptorUpdate(FRHIDescriptorHandle InHandle, const IRDescriptorTableEntry& InDescriptor)
	{
		PendingDescriptorUpdates.Add(InHandle, InDescriptor);
	}
#endif
	
	void PushDescriptorUpdates();
	void InsertComputeMemoryBarrier();
	
	enum class EPhase
	{
		Wait,
		Execute,
		Signal
	} CurrentPhase = EPhase::Wait;

	FMetalPayload* GetPayload(EPhase Phase)
	{
		if (Payloads.Num() == 0 || Phase < CurrentPhase)
		{
			NewPayload();
		}

		CurrentPhase = Phase;
		return Payloads.Last();
	}

	void NewPayload()
	{
		Payloads.Add(new FMetalPayload(Device.GetCommandQueue(EMetalQueueType::Direct)));
	}
	
	FMetalSyncPoint* GetContextSyncPoint()
	{
		if (!ContextSyncPoint)
		{
			ContextSyncPoint = FMetalSyncPoint::Create(EMetalSyncPointType::GPUAndCPU);
			BatchedSyncPoints.ToSignal.Add(ContextSyncPoint);
		}

		return ContextSyncPoint;
	}
	
	// Sync points which are waited at the start / signaled at the end 
	// of the whole batch of command lists this context recorded.
	struct
	{
		TArray<FMetalSyncPointRef> ToWait;
		TArray<FMetalSyncPointRef> ToSignal;
	} BatchedSyncPoints;
	
	// Inserts a command to signal the specified sync point
	void SignalSyncPoint(FMetalSyncPoint* SyncPoint);

	// Inserts a command that blocks the GPU queue until the specified sync point is signaled.
	void WaitSyncPoint(FMetalSyncPoint* SyncPoint);
	
	void SynchronizeResource(MTL::Resource* Resource);
	void SynchronizeTexture(MTL::Texture* Texture, uint32 Slice, uint32 Level);
	
	/** Update the event to capture all GPU work so far enqueued by this encoder. */
	void SignalEvent(MTLEventPtr Event, uint32_t SignalCount);
	
	/** Prevent further GPU work until the event is reached. */
	void WaitForEvent(MTLEventPtr Event, uint32_t SignalCount);
	
	/** Update the event to capture all GPU work so far enqueued by this encoder. */
	void SignalSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount);
	
	/** Prevent further GPU work until the event is reached. */
	void WaitForSharedEvent(MTLSharedEventPtr Event, uint32_t SignalCount);
	
#if PLATFORM_VISIONOS
    void BeginRenderingImmersive(const MetalRHIVisionOS::BeginRenderingImmersiveParams& Params);
    cp_frame_t SwiftFrame = nullptr;
#endif // PLATFORM_VISIONOS
    void SetCustomPresentViewport(FRHIViewport* Viewport) { CustomPresentViewport = Viewport; }
    FRHIViewport* CustomPresentViewport = nullptr;

	FMetalCommandBuffer* GetCurrentCommandBuffer();
	
	void BeginRecursiveCommand()
	{
		// Nothing to do
	}
    
    inline const TArray<FRHIUniformBuffer*>& GetStaticUniformBuffers() const
    {
        return StaticUniformBuffers;
    }
	
	inline TSharedRef<FMetalQueryBufferPool, ESPMode::ThreadSafe> GetQueryBufferPool()
	{
		return QueryBuffer.ToSharedRef();
	}
	
	inline FMetalStateCache& GetStateCache()
	{
		return StateCache;
	}
	
	inline FMetalCommandQueue& GetCommandQueue()
	{
		return CommandQueue;
	}
	
	inline FMetalDevice& GetDevice()
	{
		return Device;
	}
	
	inline bool IsInsideRenderPass() const
	{
		return bWithinRenderPass;
	}
	
	const FRHIShaderBindingLayout& GetShaderBindingLayout() const
	{
		static const FRHIShaderBindingLayout Default;
		return ShaderBindingLayout ? *ShaderBindingLayout : Default;
	}
	
	void StartCommandBuffer();
	void EndCommandBuffer();
	
	void FlushProfilerStats();
	
	void FlushCommands(EMetalFlushFlags Flags);
	
protected:
	FMetalDevice& Device;
	
	/** The wrapper around the device command-queue for creating & committing command buffers to */
	FMetalCommandQueue& CommandQueue;

	FMetalCommandEncoder CurrentEncoder;
	
	/** The cache of all tracked & accessible state. */
	FMetalStateCache StateCache;
	
	/** A pool of buffers for writing visibility query results. */
	TSharedPtr<FMetalQueryBufferPool, ESPMode::ThreadSafe> QueryBuffer;
	
	MTL::RenderPassDescriptor* RenderPassDesc = nullptr;

	TRefCountPtr<FMetalFence> CurrentEncoderFence;
	uint64_t UploadSyncCounter = 0;
	
	bool bWithinRenderPass = false;
	bool bIsParallelContext = false;
	void ResolveTexture(UE::RHICore::FResolveTextureInfo Info);

	TArray<FRHIUniformBuffer*> StaticUniformBuffers;
	
	// The array of recorded payloads the submission thread will process.
	// These are returned when the context is finalized.
	TArray<FMetalPayload*> Payloads;

	// A sync point signaled when all payloads in this context have completed.
	FMetalSyncPointRef ContextSyncPoint;
	
	FMetalParallelRenderPassInfo* ParallelRenderPassInfo;
	
	const FRHIShaderBindingLayout* ShaderBindingLayout = nullptr;
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FMetalPendingDescriptorUpdates PendingDescriptorUpdates;
#endif
	
private:
	void RHIClearMRT(bool bClearColor, int32 NumClearColors, const FLinearColor* ColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);
};

class FMetalRHIUploadContext : public IRHIUploadContext
{
public:
	FMetalRHIUploadContext(FMetalDevice& Device);
	~FMetalRHIUploadContext();
	
	typedef TFunction<void(FMetalRHICommandContext*)> UploadContextFunction;
	
	void Finalize(TArray<FMetalPayload*>& OutPayloads);
	
	void EnqueueFunction(UploadContextFunction Function)
	{
		UploadFunctions.Add(Function);
	}
	
private:
	FMetalRHICommandContext* UploadContext;
	TArray<UploadContextFunction> UploadFunctions;
};

struct FMetalContextArray : public TRHIPipelineArray<FMetalRHICommandContext*>
{
	FMetalContextArray(FRHIContextArray const& Contexts);
};
