// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_VISIONOS
#include "RHIFwd.h"
#import <CompositorServices/CompositorServices.h>

DECLARE_LOG_CATEGORY_EXTERN(LogMetalVisionOS, Warning, All);

namespace MetalRHIVisionOS
{
	enum class EPresentChoice : uint8
	{
		Present,
		DoNotPresent
	};

    METALRHI_API struct PresentImmersiveParams
    {
		EPresentChoice PresentChoice = EPresentChoice::DoNotPresent;
        const FTextureRHIRef Texture;
		const FTextureRHIRef Depth;
		cp_frame_t SwiftFrame;
        cp_drawable_t SwiftDrawable;
		int FrameCounter = -1;
		class IRHICommandContext* RHICommandContext = nullptr;
    };
    METALRHI_API void PresentImmersive(const PresentImmersiveParams& Params);
}
#endif // PLATFORM_VISIONOS
