// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalResources.h: Metal resource RHI definitions..
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "BoundShaderStateCache.h"
#include "MetalShaderResources.h"
#include "MetalSubmission.h"
#include "ShaderCodeArchive.h"
#include "Templates/TypeHash.h"

#define UE_METAL_RHI_SUPPORT_CLEAR_UAV_WITH_BLIT_ENCODER 1

class FMetalRHICommandContext;

class FMetalContext;
class FMetalShaderPipeline;
class FMetalCommandBuffer;

struct FMetalRenderPipelineHash
{
	friend uint32 GetTypeHash(FMetalRenderPipelineHash const& Hash)
	{
		return HashCombine(GetTypeHash(Hash.RasterBits), GetTypeHash(Hash.TargetBits));
	}
	
	friend bool operator==(FMetalRenderPipelineHash const& Left, FMetalRenderPipelineHash const& Right)
	{
		return Left.RasterBits == Right.RasterBits && Left.TargetBits == Right.TargetBits;
	}
	
	uint64 RasterBits;
	uint64 TargetBits;
};

class FMetalSubBufferHeap;
class FMetalSubBufferLinear;
class FMetalSubBufferMagazine;
class FMetalDevice;

inline uint32 GetTypeHash(const MTL::Buffer* BufferPtr)
{
    return GetTypeHash((void*)BufferPtr);
}

class IMetalBufferAllocator;
class FMetalBuffer
{
public:
	enum class FreePolicy
	{
		Owner, // FMetalBuffer owns releasing memory
		BufferAllocator, // Owned by allocator
		Temporary, // Temporary buffer that does not need a release
	};	
	
	FMetalBuffer(MTL::Buffer* Handle, FreePolicy Allocation);
	FMetalBuffer(MTL::Buffer* Handle, NS::Range Range, IMetalBufferAllocator* InAllocator);
	
	virtual ~FMetalBuffer();
    
    uint32 GetOffset()
    {
        return SubRange.location;
    }
    
    uint32 GetLength()
    {
        return SubRange.length;
    }
	
    const NS::Range& GetRange()
    {
        return SubRange;
    }
    
	friend uint32 GetTypeHash(FMetalBuffer const& Hash)
	{
		return HashCombine(GetTypeHash(Hash.Buffer), GetTypeHash((uint64)Hash.SubRange.location));
	}
    
    void* Contents()
    {
        check(Buffer->length() >= GetOffset() + GetLength());
        return ((uint8_t*)Buffer->contents()) + GetOffset();
    }
	
	uint64_t GetGPUAddress()
	{
		return Buffer->gpuAddress() + GetOffset();
	}
	
	MTL::Buffer* GetMTLBuffer() {return Buffer;};
    
    void MarkDeleted()
    {
        bMarkedDeleted = true;
    }
    
private:
	void Release();
	
	MTL::Buffer* Buffer;
	IMetalBufferAllocator* Allocator;
    
    NS::Range SubRange;
	FreePolicy OnFreePolicy;
    bool bMarkedDeleted = false;
};

typedef TSharedPtr<FMetalBuffer, ESPMode::ThreadSafe> FMetalBufferPtr;

#if METAL_RHI_RAYTRACING
class FMetalAccelerationStructure
{
public:
	FMetalAccelerationStructure(MTLAccelerationStructurePtr InAccelerationStructure, uint32_t InSize) :
		AccelerationStructure(InAccelerationStructure),
		Size(InSize)
	{
	}
	
	~FMetalAccelerationStructure();
	
	uint32_t GetSize() const
	{
		return Size;
	}
	
	void SetLabel(const FString& Label)
	{
		IndirectArgumentBuffer->GetMTLBuffer()->setLabel(FStringToNSString(Label));
		AccelerationStructure->setLabel(FStringToNSString(Label));
	}
	
	void SetIndirectArgumentBuffer(FMetalBufferPtr IndirectArgs);
	void SetHitContributionsBuffer(FMetalBufferPtr InHitContributionsBuffer)
	{
		HitContributionsBuffer = InHitContributionsBuffer;
	}
	
	FMetalBufferPtr GetIndirectArgumentBuffer()
	{
		return IndirectArgumentBuffer;
	}
	
	MTLAccelerationStructurePtr GetAccelerationStructure()
	{
		return AccelerationStructure;
	}
	
	FMetalBufferPtr GetHitContributionsBuffer()
	{
		return HitContributionsBuffer;
	}
	
private:
	MTLAccelerationStructurePtr AccelerationStructure; 
	uint32_t Size;
	
	FMetalBufferPtr IndirectArgumentBuffer;
	FMetalBufferPtr HitContributionsBuffer;
};
#endif

struct FMetalTextureCreateDesc : public FRHITextureCreateDesc
{
	FMetalTextureCreateDesc(FMetalDevice& Device, FRHITextureCreateDesc const& CreateDesc);
    FMetalTextureCreateDesc(FMetalTextureCreateDesc const& Other);
    FMetalTextureCreateDesc& operator=(const FMetalTextureCreateDesc& Other);
    
	MTLTextureDescriptorPtr Desc;
    MTL::PixelFormat MTLFormat;
	bool bIsRenderTarget = false;
	uint8 FormatKey = 0;
};

class FMetalResourceViewBase;
class FMetalShaderResourceView;
class FMetalUnorderedAccessView;

class FMetalViewableResource
{
public:
	~FMetalViewableResource()
	{
		checkf(!HasLinkedViews(), TEXT("All linked views must have been removed before the underlying resource can be deleted."));
	}

	bool HasLinkedViews() const
	{
		return LinkedViews != nullptr;
	}

	void UpdateLinkedViews(FMetalRHICommandContext* Context);

private:
	friend FMetalShaderResourceView;
	friend FMetalUnorderedAccessView;
	FMetalResourceViewBase* LinkedViews = nullptr;
};

inline uint32 GetTypeHash(const MTLTexturePtr& TexturePtr)
{
    return GetTypeHash(TexturePtr.get());
}

// Metal RHI texture resource
class METALRHI_API FMetalSurface : public FRHITexture, public FMetalViewableResource
{
public:
	/** 
	 * Constructor that will create Texture and Color/DepthBuffers as needed
	 */
	FMetalSurface(FMetalDevice& Device, FMetalTextureCreateDesc const& CreateDesc);
	
	/**
	 * Destructor
	 */
	virtual ~FMetalSurface();
	
protected:
	/** 
	 * Constructor for derived classes
	 */
	FMetalSurface(FMetalDevice& MetalDevice, MTLTexturePtr InTexture, FRHITextureCreateDesc const& CreateDesc)
		: FRHITexture		(CreateDesc)
		, Device			(MetalDevice)
		, FormatKey			(0)
		, Texture			(InTexture)
		, Viewport          (nullptr)
		, TotalTextureSize	(0)
	{}
	
public:

	void Initialize(FRHICommandListBase& RHICmdList);

	/** @returns A newly allocated buffer object large enough for the surface within the texture specified. */
	MTL::Buffer* AllocSurface(const FRHILockTextureArgs& Arguments, uint32 MipBytes, uint32 DestStride);

	/** Apply the data in Buffer to the surface specified.
	 * Will also handle destroying SourceBuffer appropriately.
	 */
	void UpdateSurfaceAndDestroySourceBuffer(FMetalRHICommandContext* Context, MTL::Buffer* SourceBuffer, uint32 MipIndex, uint32 ArrayIndex);
	
	/**
	 * Locks one of the texture's mip-maps.
	 * @param ArrayIndex Index of the texture array/face in the form Index*6+Face
	 * @return A pointer to the specified texture data.
	 */
	FRHILockTextureResult Lock(const FRHILockTextureArgs& Arguments, bool bSingleLayer);
	
	/** Unlocks a previously locked mip-map.
	 * @param ArrayIndex Index of the texture array/face in the form Index*6+Face
	 */
	void Unlock(const FRHILockTextureArgs& Arguments);
	
	/**
	 * Locks one of the texture's mip-maps.
	 * @param ArrayIndex Index of the texture array/face in the form Index*6+Face
	 * @return A pointer to the specified texture data.
	 */
	FRHILockTextureResult AsyncLock(FRHICommandListBase& RHICmdList, const FRHILockTextureArgs& Arguments);
	
	/**
	 * Returns how much memory a single mip uses, and optionally returns the stride
	 */
	uint32 GetMipSize(uint32 MipIndex, uint32* Stride, bool bSingleLayer);

	/**
	 * Returns how much memory is used by the surface
	 */
	uint32 GetMemorySize();

	/** Returns the number of faces for the texture */
	uint32 GetNumFaces();
	
	/** Gets the drawable texture if this is a back-buffer surface. */
	MTLTexturePtr GetDrawableTexture();
	void 		  ReleaseDrawableTexture();
	
    MTLTexturePtr GetCurrentTexture();

    MTLTexturePtr Reallocate(MTLTexturePtr Texture, MTL::TextureUsage UsageModifier);
	void MakeAliasable(void);

	virtual void* GetTextureBaseRHI() override final
	{
		return this;
	}

	virtual void* GetNativeResource() const override final
	{
		return Texture.get();
	}
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
    virtual FRHIDescriptorHandle GetDefaultBindlessHandle() const override final
    {
        return BindlessHandle;
    }
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

	FMetalDevice& Device;
	uint8 FormatKey;

	//texture used for store actions and binding to shader params
	MTLTexturePtr Texture;
	//if surface is MSAA, texture used to bind for RT
	MTLTexturePtr MSAATexture;

	//texture used for a resolve target.  Same as texture on iOS.  
	//Dummy target on Mac where RHISupportsSeparateMSAAAndResolveTextures is true.	In this case we don't always want a resolve texture but we
	//have to have one until renderpasses are implemented at a high level.
	// Mac / RHISupportsSeparateMSAAAndResolveTextures == true
	// iOS A9+ where depth resolve is available
	// iOS < A9 where depth resolve is unavailable.
	MTLTexturePtr MSAAResolveTexture;
	
	// Used for atomics
	FMetalBufferPtr BackingBuffer;
	
	// For back-buffers, the owning viewport.
	class FMetalViewport* Viewport;
	
private:
	/** Safely releases the texture when not in use
	 * @param MTLTexturePtr Texture from surface to be released
	 */
	void SafeRelease(MTLTexturePtr Texture);
	
	FCriticalSection DrawableMutex;
	
protected:
	
	// how much memory is allocated for this texture
	uint64 TotalTextureSize;
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FRHIDescriptorHandle 	BindlessHandle;
#endif
};

/** 
 * Derived class that can be used for creating surfaces with external textures 
 */
class METALRHI_API FMetalExternalSurface : public FMetalSurface
{
public:
	FMetalExternalSurface(FMetalDevice& MetalDevice, MTLTexturePtr InTexture, FRHITextureCreateDesc const& CreateDesc, TUniqueFunction<void()>&& InOnDeleteFunction);
	~FMetalExternalSurface();

private:	
	TUniqueFunction<void()> OnDeleteFunction;
};

class FMetalBufferData
{
public:
    ~FMetalBufferData();
    void InitWithSize(uint32 Size);
    
	uint8* Data = nullptr;
	uint32 Len = 0;
};

class FMetalRHIBuffer final : public FRHIBuffer, public FMetalViewableResource
{
public:
	FMetalRHIBuffer(FRHICommandListBase& RHICmdList, FMetalDevice& MetalDevice, const FRHIBufferCreateDesc& CreateDesc, FResourceArrayUploadInterface* InResourceArray);
	virtual ~FMetalRHIBuffer();
	
	bool RequiresTransferBuffer();
	
	void AllocateBuffer();
	void ReleaseBuffer();
	void SwitchBuffer(FRHICommandListBase& RHICmdList);
	
	/**
	 * Prepare a CPU accessible buffer for uploading to GPU memory
	 */
	void* Lock(FRHICommandListBase& RHICmdList, EResourceLockMode LockMode, uint32 Offset, uint32 Size);
	
	/**
	 * Prepare a CPU accessible buffer for uploading to GPU memory
	 */
	void Unlock(FRHICommandListBase& RHICmdList);
	
	void UploadTransferBuffer(FRHICommandListBase& RHICmdList, FMetalBufferPtr&& InTransferBuffer, uint32 UploadSize);

	// We need to allocate here because buffer backed textures can be created without an Allocated buffer
	const FMetalBufferPtr& GetCurrentBuffer()
	{
		if(!CurrentBuffer)
		{
			AllocateBuffer();
		}
		return CurrentBuffer;
	}
	
	const FMetalBufferPtr& GetCurrentBufferOrNull() const
	{
		return CurrentBuffer;
	}
	
	// 16- or 32-bit; used for index buffers only.
	MTL::IndexType GetIndexType() const
	{
		return GetStride() == 2 ? MTL::IndexTypeUInt16 : MTL::IndexTypeUInt32;
	}
	
#if METAL_RHI_RAYTRACING
	bool IsAccelerationStructure() const
	{
		return AccelerationStructure != nullptr;
	}
    FMetalAccelerationStructure* AccelerationStructure = nullptr;
#endif // METAL_RHI_RAYTRACING

	/**
	 * Whether to allocate the resource from private memory.
	 */
	bool UsePrivateMemory() const;
	
    void TakeOwnership(FMetalRHIBuffer& Other);
    void ReleaseOwnership();

	FMetalDevice& Device;

	// A temporary shared/CPU accessible buffer for upload/download
	FMetalBufferPtr TransferBuffer;
	FMetalBufferPtr CurrentBuffer;

	// Initial buffer size.
	uint32 Size;

	// offset into the buffer (for lock usage)
	uint32 LockOffset = 0;

	// Sizeof outstanding lock.
	uint32 LockSize = 0;

	// Storage mode
	MTL::StorageMode Mode;

	// Current lock mode. RLM_Num indicates this buffer is not locked.
	EResourceLockMode CurrentLockMode = RLM_Num;
	
	bool bIsFirstLock = true;

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	void UpdateAllocationTags(); 
#endif

private:
	// Allocate the CPU accessible buffer for data transfer.
	void AllocTransferBuffer(bool bOnRHIThread, uint32 InSize, EResourceLockMode LockMode);
};

class FMetalResourceViewBase : public TIntrusiveLinkedList<FMetalResourceViewBase>
{
public:
	struct FBufferView
	{
		FMetalBufferPtr Buffer;
		uint32 Offset;
		uint32 Size;
		
		FBufferView(FMetalBufferPtr Buffer, uint32 Offset, uint32 Size)
		: Buffer(Buffer)
		, Offset(Offset)
		, Size(Size)
		{}
	};

	struct FTextureBufferBacked
	{
		MTLTexturePtr Texture;
		FMetalBufferPtr Buffer;
		uint32 Offset;
		uint32 Size;
		EPixelFormat Format;
		bool bIsBuffer;
		
		FTextureBufferBacked(MTLTexturePtr Texture, FMetalBufferPtr Buffer, uint32 Offset, uint32 Size, EPixelFormat Format, bool bIsBuffer)
		: Texture(Texture)
		, Buffer(Buffer)
		, Offset(Offset)
		, Size(Size)
		, Format(Format)
		, bIsBuffer(bIsBuffer)
		{}
	};

	typedef TVariant<FEmptyVariantState
	, MTLTexturePtr
	, FBufferView
	, FTextureBufferBacked
#if METAL_RHI_RAYTRACING
	, FMetalAccelerationStructure*
#endif
	> TStorage;

	enum class EMetalType
	{
		Null                  = TStorage::IndexOfType<FEmptyVariantState>(),
		TextureView           = TStorage::IndexOfType<MTLTexturePtr>(),
		BufferView            = TStorage::IndexOfType<FBufferView>(),
		TextureBufferBacked   = TStorage::IndexOfType<FTextureBufferBacked>(),
#if METAL_RHI_RAYTRACING
		AccelerationStructure  = TStorage::IndexOfType<FMetalAccelerationStructure*>()
#endif
	};

protected:
	FMetalResourceViewBase(FMetalDevice& InDevice) : Device(InDevice)
	{}

public:
	virtual ~FMetalResourceViewBase();

	EMetalType GetMetalType() const
	{
		return static_cast<EMetalType>(Storage.GetIndex());
	}

	const MTLTexturePtr& GetTextureView() const
	{
		check(GetMetalType() == EMetalType::TextureView);
		return Storage.Get<MTLTexturePtr>();
	}

	const FBufferView& GetBufferView() const
	{
		check(GetMetalType() == EMetalType::BufferView);
		return Storage.Get<FBufferView>();
	}

	const FTextureBufferBacked& GetTextureBufferBacked() const
	{
		check(GetMetalType() == EMetalType::TextureBufferBacked);
		return Storage.Get<FTextureBufferBacked>();
	}

#if METAL_RHI_RAYTRACING
	FMetalAccelerationStructure* const& GetAccelerationStructure() const
	{
		check(GetMetalType() == EMetalType::AccelerationStructure);
		return Storage.Get<FMetalAccelerationStructure*>();
	}
#endif

	// TODO: This is kinda awkward; should probably be refactored at some point.
	TArray<TTuple<MTL::Resource*, MTL::ResourceUsage>> ReferencedResources;

	virtual void UpdateView(FMetalRHICommandContext* Context, const bool bConstructing) = 0;

protected:
	void InitAsTextureView(MTLTexturePtr);
	void InitAsBufferView(FMetalBufferPtr Buffer, uint32 Offset, uint32 Size);
	void InitAsTextureBufferBacked(MTLTexturePtr Texture, FMetalBufferPtr Buffer, uint32 Offset, uint32 Size, EPixelFormat Format, bool bIsBuffer);

#if METAL_RHI_RAYTRACING
	void InitAsAccelerationStructure(FMetalAccelerationStructure* AccelerationStructure);
#endif
	
	void Invalidate();
	
	FMetalDevice& Device;
	bool bOwnsResource = true;
	
private:
	TStorage Storage;
};

class FMetalShaderResourceView final : public FRHIShaderResourceView, public FMetalResourceViewBase
{
public:
	FMetalShaderResourceView(FMetalDevice& Device, FRHICommandListBase& RHICmdList,
							FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc);
	~FMetalShaderResourceView();
	FMetalViewableResource* GetBaseResource() const;

	virtual void UpdateView(FMetalRHICommandContext* Context, const bool bConstructing) override;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
public:
	FRHIDescriptorHandle BindlessHandle;
	
    virtual FRHIDescriptorHandle GetBindlessHandle() const override
    {
        return BindlessHandle;
    }
	
	FMetalSurface* SurfaceOverride;
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
};

class FMetalUnorderedAccessView final : public FRHIUnorderedAccessView, public FMetalResourceViewBase
{
public:
	FMetalUnorderedAccessView(FMetalDevice& Device, FRHICommandListBase& RHICmdList,
							  FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc);
	~FMetalUnorderedAccessView();
	FMetalViewableResource* GetBaseResource() const;

	virtual void UpdateView(FMetalRHICommandContext* Context, const bool bConstructing) override;

	void ClearUAV(TRHICommandList_RecursiveHazardous<FMetalRHICommandContext>& RHICmdList, const void* ClearValue, bool bFloat);
#if UE_METAL_RHI_SUPPORT_CLEAR_UAV_WITH_BLIT_ENCODER
	void ClearUAVWithBlitEncoder(TRHICommandList_RecursiveHazardous<FMetalRHICommandContext>& RHICmdList, uint32 Pattern);
#endif
	
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
private:
    FRHIDescriptorHandle BindlessHandle;

public:
    virtual FRHIDescriptorHandle GetBindlessHandle() const override
    {
        return BindlessHandle;
    }
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING
};

class FMetalGPUFence final : public FRHIGPUFence
{
public:
	FMetalGPUFence(FMetalDevice& Device, FName InName);
	~FMetalGPUFence();
	
	virtual void Clear() override;
	virtual bool Poll() const override;
	virtual void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const override;

private:
	FMetalSyncPointRef SyncPoint;
	MTLSharedEventPtr SharedEvent;
	FMetalDevice* Device = nullptr;
	
	friend class FMetalDynamicRHI;
};

class FMetalShaderLibrary;
class FMetalGraphicsPipelineState;
class FMetalVertexDeclaration;
class FMetalVertexShader;
class FMetalGeometryShader;
class FMetalPixelShader;
class FMetalComputeShader;
class FMetalRHIStagingBuffer;
class FMetalRHIRenderQuery;
class FMetalSuballocatedUniformBuffer;
#if METAL_RHI_RAYTRACING
class FMetalRayTracingScene;
class FMetalRayTracingGeometry;
class FMetalRayTracingShaderBindingTable;
class FMetalRayTracingPipelineState;
class FMetalRayShader;
#endif // METAL_RHI_RAYTRACING
#if PLATFORM_SUPPORTS_MESH_SHADERS
class FMetalMeshShader;
class FMetalAmplificationShader;
#endif


template<>
struct TMetalResourceTraits<FRHIShaderLibrary>
{
	typedef FMetalShaderLibrary TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIVertexDeclaration>
{
	typedef FMetalVertexDeclaration TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIVertexShader>
{
	typedef FMetalVertexShader TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIGeometryShader>
{
	typedef FMetalGeometryShader TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIPixelShader>
{
	typedef FMetalPixelShader TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIComputeShader>
{
	typedef FMetalComputeShader TConcreteType;
};
#if PLATFORM_SUPPORTS_MESH_SHADERS
template<>
struct TMetalResourceTraits<FRHIMeshShader>
{
    typedef FMetalMeshShader TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIAmplificationShader>
{
    typedef FMetalAmplificationShader TConcreteType;
};
#endif
template<>
struct TMetalResourceTraits<FRHIRenderQuery>
{
	typedef FMetalRHIRenderQuery TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIUniformBuffer>
{
	typedef FMetalSuballocatedUniformBuffer TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIBuffer>
{
	typedef FMetalRHIBuffer TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIShaderResourceView>
{
	typedef FMetalShaderResourceView TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIUnorderedAccessView>
{
	typedef FMetalUnorderedAccessView TConcreteType;
};

template<>
struct TMetalResourceTraits<FRHIGraphicsPipelineState>
{
	typedef FMetalGraphicsPipelineState TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIGPUFence>
{
	typedef FMetalGPUFence TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIStagingBuffer>
{
	typedef FMetalRHIStagingBuffer TConcreteType;
};
#if METAL_RHI_RAYTRACING
template<>
struct TMetalResourceTraits<FRHIRayTracingScene>
{
	typedef FMetalRayTracingScene TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIRayTracingGeometry>
{
	typedef FMetalRayTracingGeometry TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIShaderBindingTable>
{
	typedef FMetalRayTracingShaderBindingTable TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIRayTracingShader>
{
	typedef FMetalRayShader TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIRayTracingPipelineState>
{
	typedef FMetalRayTracingPipelineState TConcreteType;
};
#endif // METAL_RHI_RAYTRACING
