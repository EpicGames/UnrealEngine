// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalShaderResources.h: Metal shader resource RHI definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "CrossCompilerCommon.h"
#include "RayTracingBuiltInResources.h"

THIRD_PARTY_INCLUDES_START
#include "metal_irconverter.h"
THIRD_PARTY_INCLUDES_END

/**
* Buffer data-types for MetalRHI & MetalSL
*/
enum class EMetalBufferFormat : uint8
{
	Unknown					=0,
	
	R8Sint					=1,
	R8Uint					=2,
	R8Snorm					=3,
	R8Unorm					=4,
	
	R16Sint					=5,
	R16Uint					=6,
	R16Snorm				=7,
	R16Unorm				=8,
	R16Half					=9,
	
	R32Sint					=10,
	R32Uint					=11,
	R32Float				=12,
	
	RG8Sint					=13,
	RG8Uint					=14,
	RG8Snorm				=15,
	RG8Unorm				=16,
	
	RG16Sint				=17,
	RG16Uint				=18,
	RG16Snorm				=19,
	RG16Unorm				=20,
	RG16Half				=21,
	
	RG32Sint				=22,
	RG32Uint				=23,
	RG32Float				=24,
	
	RGB8Sint				=25,
	RGB8Uint				=26,
	RGB8Snorm				=27,
	RGB8Unorm				=28,
	
	RGB16Sint				=29,
	RGB16Uint				=30,
	RGB16Snorm				=31,
	RGB16Unorm				=32,
	RGB16Half				=33,
	
	RGB32Sint				=34,
	RGB32Uint				=35,
	RGB32Float				=36,
	
	RGBA8Sint				=37,
	RGBA8Uint				=38,
	RGBA8Snorm				=39,
	RGBA8Unorm				=40,
	
	BGRA8Unorm				=41,
	
	RGBA16Sint				=42,
	RGBA16Uint				=43,
	RGBA16Snorm				=44,
	RGBA16Unorm				=45,
	RGBA16Half				=46,
	
	RGBA32Sint				=47,
	RGBA32Uint				=48,
	RGBA32Float				=49,
	
	RGB10A2Unorm			=50,
	
	RG11B10Half 			=51,
	
	R5G6B5Unorm         	=52,
	B5G5R5A1Unorm           =53,

	Max						=54
};

enum class EMetalBindingsFlags : uint8
{
	PixelDiscard = 1 << 0,
	UseMetalShaderConverter = 1 << 1,
};

static constexpr IRStaticSamplerDescriptor MakeStaticSampler(const IRFilter Filter, const IRTextureAddressMode WrapMode, const uint32 Register, const uint32 Space)
{
	IRStaticSamplerDescriptor Result = {};

	Result.Filter           = Filter;
	Result.AddressU         = WrapMode;
	Result.AddressV         = WrapMode;
	Result.AddressW         = WrapMode;
	Result.MipLODBias       = 0.0f;
	Result.MaxAnisotropy    = 1;
	Result.ComparisonFunc   = IRComparisonFunctionNever;
	Result.BorderColor      = IRStaticBorderColorTransparentBlack;
	Result.MinLOD           = 0.0f;
	Result.MaxLOD           = FLT_MAX;
	Result.ShaderRegister   = Register;
	Result.RegisterSpace    = Space;
	Result.ShaderVisibility = IRShaderVisibilityAll;

	return Result;
}

// Static sampler table must match MetalCommon.ush
static IRStaticSamplerDescriptor StaticSamplerDescs[] =
{
	MakeStaticSampler(IRFilterMinMagMipPoint,        IRTextureAddressModeWrap,  0, 1000),
	MakeStaticSampler(IRFilterMinMagMipPoint,        IRTextureAddressModeClamp, 1, 1000),
	MakeStaticSampler(IRFilterMinMagLinearMipPoint,  IRTextureAddressModeWrap,  2, 1000),
	MakeStaticSampler(IRFilterMinMagLinearMipPoint,  IRTextureAddressModeClamp, 3, 1000),
	MakeStaticSampler(IRFilterMinMagMipLinear,       IRTextureAddressModeWrap,  4, 1000),
	MakeStaticSampler(IRFilterMinMagMipLinear,       IRTextureAddressModeClamp, 5, 1000),
};

struct FMetalShaderBindings
{
	TArray<CrossCompiler::FPackedArrayInfo>			PackedGlobalArrays;
	TMap<uint8, TArray<uint8>>						ArgumentBufferMasks;
	CrossCompiler::FShaderBindingInOutMask			InOutMask;
    FString                                         IRConverterReflectionJSON;
    uint32                                          RSNumCBVs = 0;
    uint32                                          OutputSizeVS = 0;
    uint32                                          MaxInputPrimitivesPerMeshThreadgroupGS = 0;

	uint32 	ConstantBuffers = 0;
	uint32  ArgumentBuffers = 0;
	uint8	NumSamplers = 0;
	uint8	NumUniformBuffers = 0;
	uint8	NumUAVs = 0;
	EMetalBindingsFlags	Flags{};
	
	inline FArchive& Serialize(FArchive& Ar, FShaderResourceTable& SRT);
};

inline FArchive& FMetalShaderBindings::Serialize(FArchive& Ar, FShaderResourceTable& SRT)
{
	Ar << PackedGlobalArrays;
	Ar << SRT;
	Ar << ConstantBuffers;
	Ar << InOutMask;
	Ar << ArgumentBuffers;
	if (ArgumentBuffers)
	{
		Ar << ArgumentBufferMasks;
	}
	Ar << NumSamplers;
	Ar << NumUniformBuffers;
	Ar << NumUAVs;
	Ar << Flags;
	if (EnumHasAnyFlags(Flags, EMetalBindingsFlags::UseMetalShaderConverter))
	{
		Ar << IRConverterReflectionJSON;
		Ar << RSNumCBVs;
		Ar << OutputSizeVS;
		Ar << MaxInputPrimitivesPerMeshThreadgroupGS;
	}
	return Ar;
}

enum class EMetalOutputWindingMode : uint8
{
	Clockwise = 0,
	CounterClockwise = 1,
};

enum class EMetalPartitionMode : uint8
{
	Pow2 = 0,
	Integer = 1,
	FractionalOdd = 2,
	FractionalEven = 3,
};

enum class EMetalComponentType : uint8
{
	Uint = 0,
	Int,
	Half,
	Float,
	Bool,
	Max
};

struct FMetalRayTracingHeader
{
	uint32 PayloadType;
	uint32 PayloadSize;
	uint32 DefaultBytecodeStage = IRShaderStageInvalid;
	uint32 ExtendedBytecodeStage = IRShaderStageInvalid;
	FString EntryPoint;
	FString AnyHitEntryPoint;
	FString IntersectionEntryPoint;
	int32 ExtendedCodeDataOffset;
	int32 ExtendedCodeDataSize;
	
	TArray<IRRootParameter1> GlobalRootParams;
	TArray<IRRootParameter1> LocalRootParams;
	
	bool IsValid() const
	{
		return (PayloadType != UINT32_MAX
			&& PayloadSize != UINT32_MAX
			&& DefaultBytecodeStage != IRShaderStageInvalid
			&& !EntryPoint.IsEmpty());
	}

	FMetalRayTracingHeader()
		: PayloadType(UINT32_MAX)
		, PayloadSize(UINT32_MAX)
		, DefaultBytecodeStage(IRShaderStageInvalid)
		, ExtendedBytecodeStage(IRShaderStageInvalid)
		, ExtendedCodeDataOffset(-1)
		, ExtendedCodeDataSize(-1)
	{

	}

	friend FArchive& operator<<(FArchive& Ar, FMetalRayTracingHeader& Header)
	{
		Ar << Header.PayloadType;
		Ar << Header.PayloadSize;
		Ar << Header.DefaultBytecodeStage;
		Ar << Header.ExtendedBytecodeStage;
		Ar << Header.EntryPoint;
		Ar << Header.AnyHitEntryPoint;
		Ar << Header.IntersectionEntryPoint;
		Ar << Header.GlobalRootParams;
		Ar << Header.LocalRootParams;
		Ar << Header.ExtendedCodeDataOffset;
		Ar << Header.ExtendedCodeDataSize;

		return Ar;
	}
};

struct FMetalAttribute
{
	uint32 Index;
	uint32 Components;
	uint32 Offset;
	EMetalComponentType Type;
	uint32 Semantic;
	
	FMetalAttribute()
	: Index(0)
	, Components(0)
	, Offset(0)
	, Type(EMetalComponentType::Uint)
	, Semantic(0)
	{
		
	}
	
	friend FArchive& operator<<(FArchive& Ar, FMetalAttribute& Attr)
	{
		Ar << Attr.Index;
		Ar << Attr.Type;
		Ar << Attr.Components;
		Ar << Attr.Offset;
		Ar << Attr.Semantic;
		return Ar;
	}
};

struct FMetalCodeHeader
{
	FMetalShaderBindings Bindings;

	uint32 SourceLen;
	uint32 SourceCRC;
	uint32 Version;
	uint32 NumThreadsX;
	uint32 NumThreadsY;
	uint32 NumThreadsZ;
	uint32 CompileFlags;
	FMetalRayTracingHeader RayTracing;
	uint8 Frequency;
	int8 SideTable;
	
	
	FMetalCodeHeader()
	: SourceLen(0)
	, SourceCRC(0)
	, Version(0)
	, NumThreadsX(0)
	, NumThreadsY(0)
	, NumThreadsZ(0)
	, CompileFlags(0)
	, Frequency(0)
	, SideTable(-1)
	{
	}

	inline FArchive& Serialize(FArchive& Ar, FShaderResourceTable& SRT);
};


inline FArchive& FMetalCodeHeader::Serialize(FArchive& Ar, FShaderResourceTable& SRT)
{
	Bindings.Serialize(Ar, SRT);
	
	Ar << SourceLen;
	Ar << SourceCRC;
	Ar << Version;
	Ar << Frequency;
	if (Frequency == SF_Compute || IsRayTracingShaderFrequency((EShaderFrequency)Frequency))
	{
		Ar << NumThreadsX;
		Ar << NumThreadsY;
		Ar << NumThreadsZ;
		Ar << RayTracing;
	}
	Ar << CompileFlags;
	Ar << SideTable;
	
    return Ar;
}

struct FMetalShaderLibraryHeader
{
	FString Format;
	uint32 NumLibraries;
	uint32 NumShadersPerLibrary;
	
	friend FArchive& operator<<(FArchive& Ar, FMetalShaderLibraryHeader& Header)
	{
		return Ar << Header.Format << Header.NumLibraries << Header.NumShadersPerLibrary;
	}
};

inline FArchive& operator<<(FArchive &Ar, IRRootDescriptor1 &RootDescriptor)
{
	Ar << RootDescriptor.ShaderRegister;
	Ar << RootDescriptor.RegisterSpace;
	Ar << reinterpret_cast<uint32&>(RootDescriptor.Flags);
	
	return Ar;
}

inline FArchive& operator<<(FArchive &Ar, IRRootParameter1 &RootParameter)
{
	Ar << reinterpret_cast<uint32&>(RootParameter.ParameterType);
	Ar << RootParameter.Descriptor;
	Ar << reinterpret_cast<uint32&>(RootParameter.ShaderVisibility);

	return Ar;
}

struct FMetalShaderBytecode
{
	FString NativePath;
	TArray<uint8> OutputFile;
	TArray<uint8> ObjectFile;
	
	friend FArchive& operator<<( FArchive& Ar, FMetalShaderBytecode& Info )
	{
		Ar << Info.NativePath << Info.OutputFile << Info.ObjectFile;
		return Ar;
	}
};

struct FMetalBindlessShaderInfo
{
	FString RayEntryPoint;
	FString RayAnyHitEntryPoint;
	FString RayIntersectionEntryPoint;
	FString UniqueEntryPointName;
	FString UniqueRayAnyHitEntryPoint;
	FString UniqueRayIntersectionEntryPoint;
	
	uint32_t NumInstructions = 0;
	uint32_t NumCBVs = 0;
	
	uint32 CRCLen = 0;
	uint32 CRC = 0;
	uint32 SourceLen = 0;
	bool bUsesDualSourceBlending = false;

	TArray<IRRootParameter1> GlobalRootParams;
	TArray<IRRootParameter1> UnsortedLocalRootParams;
	TArray<IRRootParameter1> LocalRootParams;
	
	static constexpr uint32 NumShaderSpaces = (UE_HLSL_SPACE_STATIC_SHADER_BINDINGS + 1);
	
	// Temporary cache to remove dupes when processing a shader library. Key is the register index; Value is the binding name (for sanity check at compile time).
	TMap<uint32, FString> RootParamsCache[NumShaderSpaces];
	
	// Main entry point
	FMetalShaderBytecode MetalBytecode;
	IRShaderStage DefaultBytecodeShaderStage;
		
	// Extended entry point (only used by RayTracing IS/AHS)
	FMetalShaderBytecode MetalBytecodeExtended;
	IRShaderStage ExtendedBytecodeShaderStage = IRShaderStageInvalid;

	bool bIsProcedural = false; 
};
