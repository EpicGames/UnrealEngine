// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalState.h: Metal state definitions.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "RHI.h"
#include "RHIResources.h"

class FMetalSamplerState : public FRHISamplerState
{
public:
	
	/** 
	 * Constructor/destructor
	 */
	FMetalSamplerState(class FMetalDevice& Device, const FSamplerStateInitializerRHI& Initializer);
	~FMetalSamplerState();

	MTL::SamplerState* State;
#if !PLATFORM_MAC
    MTL::SamplerState* NoAnisoState;
#endif
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
    FRHIDescriptorHandle BindlessHandle;

    // TODO: Do we need to support NoAnisoState too? (or is it some leftover we don't care about anymore?)
    virtual FRHIDescriptorHandle GetBindlessHandle() const override final { return BindlessHandle; }
#endif
	
	FMetalDevice& Device;
};

class FMetalRasterizerState : public FRHIRasterizerState
{
public:

	/**
	 * Constructor/destructor
	 */
	FMetalRasterizerState(const FRasterizerStateInitializerRHI& Initializer);
	~FMetalRasterizerState();
	
	virtual bool GetInitializer(FRasterizerStateInitializerRHI& Init) override final;
	
	FRasterizerStateInitializerRHI State;
};

class FMetalDepthStencilState : public FRHIDepthStencilState
{
public:

	/**
	 * Constructor/destructor
	 */
	FMetalDepthStencilState(MTL::Device* Device, const FDepthStencilStateInitializerRHI& Initializer);
	~FMetalDepthStencilState();
	
	virtual bool GetInitializer(FDepthStencilStateInitializerRHI& Init) override final;
	
	FDepthStencilStateInitializerRHI Initializer;
	MTL::DepthStencilState* State;
	bool bIsDepthWriteEnabled;
	bool bIsStencilWriteEnabled;
};

class FMetalBlendState : public FRHIBlendState
{
public:

	/**
	 * Constructor/destructor
	 */
	FMetalBlendState(const FBlendStateInitializerRHI& Initializer);
	~FMetalBlendState();
	
	virtual bool GetInitializer(FBlendStateInitializerRHI& Init) override final;

	struct FBlendPerMRT
	{
		MTL::RenderPipelineColorAttachmentDescriptor* BlendState;
		uint8 BlendStateKey;
	};
	FBlendPerMRT RenderTargetStates[MaxSimultaneousRenderTargets];
	bool bUseIndependentRenderTargetBlendStates;
	bool bUseAlphaToCoverage;

private:
	// this tracks blend settings (in a bit flag) into a unique key that uses few bits, for PipelineState MRT setup
	static TMap<uint32, uint8> BlendSettingsToUniqueKeyMap;
	static uint8 NextKey;
	static FCriticalSection Mutex;
};

template<>
struct TMetalResourceTraits<FRHISamplerState>
{
    typedef FMetalSamplerState TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIRasterizerState>
{
    typedef FMetalRasterizerState TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIDepthStencilState>
{
    typedef FMetalDepthStencilState TConcreteType;
};
template<>
struct TMetalResourceTraits<FRHIBlendState>
{
    typedef FMetalBlendState TConcreteType;
};
