// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetalRHI.h"
#include "MetalBuffer.h"
#import "MetalThirdParty.h"
#include "Stats/Stats.h"

class FMetalCommandBuffer;
class FMetalDevice;
/*
   Simple Temporary allocator that allocates from heaps
 */
class FMetalTempAllocator : public IMetalBufferAllocator
{
public:
	FMetalTempAllocator(FMetalDevice& InDevice, uint32_t InMinAllocationSize, uint32_t InTargetAllocationLimit, uint32_t InAlignment);
	
	FMetalBufferPtr Allocate(const uint32_t Size);
	void Cleanup();
    
	virtual void ReleaseBuffer(FMetalBuffer*) override {}
	
private:
	struct FTempBufferInfo
	{
		MTL::Buffer* Buffer;
		uint32_t Offset;
		uint32_t Size;
	};
	
	FMetalDevice& Device;
	TArray<FTempBufferInfo> Buffers;
	
	FCriticalSection AllocatorLock;
	
	TStatId TotalAllocationStat;
	
	uint32_t TotalAllocated = 0;
	uint32_t MinAllocationSize = 0;
	uint32_t TargetAllocationLimit = 0;
	uint32_t Alignment;
};

