// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Need CoreMinimal to avoid collisions
#include "CoreMinimal.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

// Metal C++ wrapper
THIRD_PARTY_INCLUDES_START
    #include <Metal/Metal.h>
    #include "MetalInclude.h"
THIRD_PARTY_INCLUDES_END
