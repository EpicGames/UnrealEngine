// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//
//  Implements handles to linearly allocated per-frame constant buffers for shared memory systems.
//

#import "MetalThirdParty.h"
#include "MetalBuffer.h"
#include "RHIResources.h"

#define METAL_UNIFORM_BUFFER_VALIDATION !UE_BUILD_SHIPPING

class FMetalStateCache;
class FMetalDevice;

class FMetalSuballocatedUniformBuffer : public FRHIUniformBuffer
{
    friend class FMetalStateCache;
public:
    // The last render thread frame this uniform buffer updated or pushed contents to the GPU backing
    uint32 LastFrameUpdated;
	
    FMetalBufferPtr BackingBuffer;
    
	// CPU side shadow memory to hold updates for single-draw or multi-frame buffers.
    // This allows you to upload on a frame but actually use this UB later on
    void* Shadow;

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	FRHIDescriptorHandle BindlessHandle;
#endif
	
private:
	FMetalDevice& Device;
#if METAL_UNIFORM_BUFFER_VALIDATION
    EUniformBufferValidation Validation;
#endif

public:
    // Creates a uniform buffer.
    // If Usage is SingleDraw or MultiFrame we will keep a copy of the data
    FMetalSuballocatedUniformBuffer(FMetalDevice& Device, const void* Contents, const FRHIUniformBufferLayout* Layout,
									EUniformBufferUsage Usage, EUniformBufferValidation Validation);
    ~FMetalSuballocatedUniformBuffer();

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	virtual FRHIDescriptorHandle GetBindlessHandle() const;
#endif

	void Update(FRHICommandListBase& RHICmdList, const void* Contents);

private:
	// Copies the RDG resources to a resource table for a deferred update on the RHI thread.
	void CopyResourceTable(const void* Contents, TArray<TRefCountPtr<FRHIResource> >& OutResourceTable) const;

    // Pushes the data in Contents to the gpu.
    // Updates the frame counter to FrameNumber.
    // (this is to support the case where we create buffer and reuse it many frames later).
    // This acquires a region in the current frame's transient uniform buffer
    // and copies Contents into that backing.
    // The amount of data is determined by the Layout
    void PushToGPUBacking(const void* Contents);
};

typedef FMetalSuballocatedUniformBuffer FMetalUniformBuffer;
