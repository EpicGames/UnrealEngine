// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalViewport.h: Metal viewport RHI definitions.
=============================================================================*/

#pragma once

#include "MetalRHIPrivate.h"
#include "Apple/AppleStringUtils.h"
#include "PixelFormat.h"
#include "RHIResources.h"
#include "MetalResources.h"

#include "Widgets/SWindow.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#if PLATFORM_MAC
#include "Mac/CocoaTextView.h"
@interface FMetalView : FCocoaTextView
{
	bool bAllowMouseActionsPastLayerRange;
	bool bSlateWidgetHit;
	
	FMetalViewport* MetalViewport;
}

@property TInterval<int32> LayerRange;
@end
#endif
#include "HAL/PlatformFramePacer.h"

#if PLATFORM_VISIONOS
#import <CompositorServices/CompositorServices.h>
#endif

@class FCocoaWindow;
class FMetalSurface;
class FMetalDevice;

enum EMetalViewportAccessFlag
{
	EMetalViewportAccessRHI,
	EMetalViewportAccessRenderer,
	EMetalViewportAccessGame,
	EMetalViewportAccessDisplayLink
};

class FMetalCommandQueue;

#if PLATFORM_VISIONOS
namespace MetalRHIVisionOS
{
    struct BeginRenderingImmersiveParams;
    struct PresentImmersiveParams;
}
#endif

typedef void (^FMetalViewportPresentHandler)(uint32 CGDirectDisplayID, double OutputSeconds, double OutputDuration);

class FMetalViewport : public FRHIViewport
{
public:
	static FCriticalSection ViewportsMutex;
	static TSet<FMetalViewport*> Viewports;

	FMetalViewport(FMetalDevice& InDevice, void* WindowHandle, uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen,EPixelFormat InFormat);
	~FMetalViewport();

	void Resize(uint32 InSizeX, uint32 InSizeY, bool bInIsFullscreen,EPixelFormat Format,bool bWaitForRenderingThread = true);
	
	TRefCountPtr<FMetalSurface> GetBackBuffer(EMetalViewportAccessFlag Accessor, 
											  TOptional<int32> DrawLayerIndex = TOptional<int32>()) const;
	
	CA::MetalDrawable* GetDrawable(EMetalViewportAccessFlag Accessor, TOptional<int32> DrawLayerIndex = TOptional<int32>());
	MTL::Texture* GetDrawableTexture(EMetalViewportAccessFlag Accessor, TOptional<int32> DrawLayerIndex = TOptional<int32>());
	MTL::Texture* GetDrawableTexture(EMetalViewportAccessFlag Accessor, FMetalSurface* SurfaceWithDrawLayer);
	MTL::Texture* GetCurrentTexture(EMetalViewportAccessFlag Accessor);
	
	void ReleaseDrawable(void);
	void ReleaseDrawable(int32 DrawLayerKey);
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	virtual void CreateNativeLayer(int32 NewNativeLayer, void* NativeViewHandle, const TArray<TInterval<int32>>& LayerRanges) override;
	virtual void DeleteNativeLayer(int32 OldNativeLayer) override;
	virtual void SetDefaultNativeLayer(int32 NativeLayer) override;
#endif

	// supports pulling the raw MTLTexture
	virtual void* GetNativeBackBufferTexture() const override { return GetBackBuffer(EMetalViewportAccessRenderer, TOptional<int32>()).GetReference(); }
	virtual void* GetNativeBackBufferRT() const override { return (const_cast<FMetalViewport *>(this))->GetDrawableTexture(EMetalViewportAccessRenderer); }
	
#if PLATFORM_MAC
	NSWindow* GetWindow() const;
#endif
    
#if PLATFORM_MAC || PLATFORM_VISIONOS
	virtual void SetCustomPresent(FRHICustomPresent* InCustomPresent) override
	{
		CustomPresent = InCustomPresent;
	}

	virtual FRHICustomPresent* GetCustomPresent() const override { return CustomPresent; }
#endif
	
	void Present(FMetalCommandBuffer* CommandBuffer, bool bLockToVsync);
	
#if PLATFORM_VISIONOS
	void GetDrawableImmersiveTextures(EMetalViewportAccessFlag Accessor, cp_drawable_t SwiftDrawable, MTL::Texture*& OutColorTexture, MTL::Texture*& OutDepthTexture );
    void PresentImmersive(const MetalRHIVisionOS::PresentImmersiveParams& Params);
#endif
	
#if PLATFORM_MAC
	TSharedPtr<SWindow> GetBackingSlateWindow();
#endif
	
private:
#if PLATFORM_MAC
	FMetalView* CreateNewMetalView(bool bIsContentView, NSWindowOrderingMode ViewOrder, NSView* ViewRelativeTo,
								   TInterval<int32> LayerRange);
#endif
	uint32 GetViewportIndex(EMetalViewportAccessFlag Accessor) const;
	void PresentDrawLayers(FMetalCommandBuffer* CommandBuffer);
	
	CA::MetalDrawable* GetDrawableInternal(EMetalViewportAccessFlag Accessor, TOptional<int32> DrawLayerIndex,
										   bool bLockAlreadyAquired);

private:
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	struct DrawLayer
	{
		TRefCountPtr<FMetalSurface> BackBuffer[2];
		CA::MetalDrawable* Drawable;
#if PLATFORM_MAC
		FMetalView* View;
#else
	#error "Multi view slate window rendering mode not supported on non-mac platforms!"
#endif
	};
#endif
	
	struct CommittedDrawable
	{
		CA::MetalDrawable* Drawable;
		MTL::Texture* Texture;
		int32 Key;
	};
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	CommittedDrawable GetCommittedDrawableToPresent(int32 DrawLayer, FMetalCommandBuffer* CommandBuffer,
													bool& bLayersToDraw, FMetalView** TheView);
#else
	CommittedDrawable GetCommittedDrawableToPresent(int32 DrawLayer, FMetalCommandBuffer* CommandBuffer,
													bool& bLayersToDraw);
#endif
	
	void ResizeBackBuffer(int32 DrawLayerKey, uint32 Index, uint32 InSizeX, uint32 InSizeY, EPixelFormat InFormat);
	
#if PLATFORM_VISIONOS
	CP_OBJECT_cp_layer_renderer* SwiftLayer = nullptr;
#endif
	
	FMetalDevice& Device;
	mutable FCriticalSection Mutex;
	
	uint32 DisplayID;
	FMetalViewportPresentHandler Block;
	volatile int32 FrameAvailable;
	bool bIsFullScreen;


#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	uint32 SizeX;
	uint32 SizeY;
	EPixelFormat Format;
#endif
#if PLATFORM_IOS && !PLATFORM_TVOS && !PLATFORM_VISIONOS	
	UIInterfaceOrientationMask OrientationMask;
	UIInterfaceOrientationMask OldOrientationMask;
    FDelegateHandle OrientationChangedHandle;
#endif

	int32 DefaultNativeLayer;
	
#if PLATFORM_MAC
	FCocoaWindow* WindowHandle;
#endif
	
#if WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT
	// all views with slate layers >= the key get drawn in the value metal view
	TMap<int32, DrawLayer> DrawLayers;
	TArray<NSView*> ViewsAdded;
#else
	CA::MetalDrawable* Drawable;
	TRefCountPtr<FMetalSurface> BackBuffer[2];
#if PLATFORM_MAC
	FMetalView* View;
#endif
#endif
	MTL::Texture* DrawableTextures[2];
    
#if PLATFORM_MAC || PLATFORM_VISIONOS
    FCustomPresentRHIRef CustomPresent;
#endif
	
	TWeakPtr<SWindow> BackingSlateWindow;
};

template<>
struct TMetalResourceTraits<FRHIViewport>
{
	typedef FMetalViewport TConcreteType;
};
